[CoreRedirects]
+PackageRedirects=(OldName="/USDImporter/Blueprint/DefaultLiveLinkAnimBP", NewName="/USDCore/Blueprint/DefaultLiveLinkAnimBP")
+PackageRedirects=(OldName="/USDImporter/Materials/CardTextureMaterial", NewName="/USDCore/Materials/CardTextureMaterial")
+PackageRedirects=(OldName="/USDImporter/Materials/ChannelSelector", NewName="/USDCore/Materials/ChannelSelector")
+PackageRedirects=(OldName="/USDImporter/Materials/DisplayColor", NewName="/USDCore/Materials/DisplayColor")
+PackageRedirects=(OldName="/USDImporter/Materials/DisplayColorAndOpacity", NewName="/USDCore/Materials/DisplayColorAndOpacity")
+PackageRedirects=(OldName="/USDImporter/Materials/DisplayColorAndOpacityTwoSided", NewName="/USDCore/Materials/DisplayColorAndOpacityTwoSided")
+PackageRedirects=(OldName="/USDImporter/Materials/DisplayColorTwoSided", NewName="/USDCore/Materials/DisplayColorTwoSided")
+PackageRedirects=(OldName="/USDImporter/Materials/UsdPreviewSurface", NewName="/USDCore/Materials/UsdPreviewSurface")
+PackageRedirects=(OldName="/USDImporter/Materials/UsdPreviewSurfaceTranslucent", NewName="/USDCore/Materials/UsdPreviewSurfaceTranslucent")
+PackageRedirects=(OldName="/USDImporter/Materials/UsdPreviewSurfaceTranslucentTwoSided", NewName="/USDCore/Materials/UsdPreviewSurfaceTranslucentTwoSided")
+PackageRedirects=(OldName="/USDImporter/Materials/UsdPreviewSurfaceTranslucentTwoSidedVT", NewName="/USDCore/Materials/UsdPreviewSurfaceTranslucentTwoSidedVT")
+PackageRedirects=(OldName="/USDImporter/Materials/UsdPreviewSurfaceTranslucentVT", NewName="/USDCore/Materials/UsdPreviewSurfaceTranslucentVT")
+PackageRedirects=(OldName="/USDImporter/Materials/UsdPreviewSurfaceTwoSided", NewName="/USDCore/Materials/UsdPreviewSurfaceTwoSided")
+PackageRedirects=(OldName="/USDImporter/Materials/UsdPreviewSurfaceTwoSidedVT", NewName="/USDCore/Materials/UsdPreviewSurfaceTwoSidedVT")
+PackageRedirects=(OldName="/USDImporter/Materials/UsdPreviewSurfaceVT", NewName="/USDCore/Materials/UsdPreviewSurfaceVT")
+PackageRedirects=(OldName="/USDImporter/Materials/UsdTransform2d", NewName="/USDCore/Materials/UsdTransform2d")
+PackageRedirects=(OldName="/USDImporter/Materials/WorldSpaceNormalsToTangentSpace", NewName="/USDCore/Materials/WorldSpaceNormalsToTangentSpace")
+PackageRedirects=(OldName="/USDImporter/Textures/DefaultColorVT", NewName="/USDCore/Textures/DefaultColorVT")
+PackageRedirects=(OldName="/USDImporter/Textures/DefaultLinearColorVT", NewName="/USDCore/Textures/DefaultLinearColorVT")
+PackageRedirects=(OldName="/USDImporter/Textures/DefaultNormalVT", NewName="/USDCore/Textures/DefaultNormalVT")
