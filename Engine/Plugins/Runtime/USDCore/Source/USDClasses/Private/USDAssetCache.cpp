// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDAssetCache.h"

#include "USDLog.h"

#include "Misc/ScopeLock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(USDAssetCache)

UUsdAssetCache::UUsdAssetCache()
	: bAllowPersistentStorage(true)
{
}

#if WITH_EDITOR
void UUsdAssetCache::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UUsdAssetCache, bAllowPersistentStorage))
	{
		if (!bAllowPersistentStorage)
		{
			TransientStorage.Append(PersistentStorage);
			PersistentStorage.Empty();
		}
	}
}
#endif	  // #if WITH_EDITOR

void UUsdAssetCache::CacheAsset(const FString& Hash, UObject* Asset, const FString& PrimPath /*= FString() */)
{
	if (!Asset)
	{
		UE_LOGF(LogUsd, Warning, "Attempted to add a null asset to USD Asset Cache with hash '%ls' and PrimPath '%ls'!", *Hash, *PrimPath);
		return;
	}

	FScopeLock Lock(&CriticalSection);

	if (!bAllowPersistentStorage || Asset->HasAnyFlags(RF_Transient) || Asset->GetOutermost() == GetTransientPackage())
	{
		if (UObject* ExistingAsset = TransientStorage.FindRef(Hash))
		{
			if (ExistingAsset != Asset)
			{
				UE_LOGF(
					LogUsd,
					Log,
					"Overwriting asset '%ls' with '%ls' (prim path '%ls') in the asset cache's transient storage",
					*ExistingAsset->GetPathName(),
					*Asset->GetPathName(),
					*PrimPath
				);
				DiscardAsset(Hash);
			}
		}

		TransientStorage.Add(Hash, Asset);
	}
	else
	{
		Modify();

		if (UObject* ExistingAsset = PersistentStorage.FindRef(Hash))
		{
			if (ExistingAsset != Asset)
			{
				UE_LOGF(
					LogUsd,
					Log,
					"Overwriting asset '%ls' with '%ls' (prim path '%ls') in the asset cache's persistent storage",
					*ExistingAsset->GetPathName(),
					*Asset->GetPathName(),
					*PrimPath
				);
				DiscardAsset(Hash);
			}
		}

		PersistentStorage.Add(Hash, Asset);
	}

	OwnedAssets.Add(Asset);
	if (!PrimPath.IsEmpty())
	{
		PrimPathToAssets.Add(PrimPath, Asset);
	}

	ActiveAssets.Add(Asset);
}

void UUsdAssetCache::DiscardAsset(const FString& Hash)
{
	FScopeLock Lock(&CriticalSection);

	TObjectPtr<UObject>* FoundObject = TransientStorage.Find(Hash);

	if (!FoundObject)
	{
		FoundObject = PersistentStorage.Find(Hash);

		if (FoundObject)
		{
			Modify();
		}
	}

	if (FoundObject)
	{
		for (TMap<FString, TWeakObjectPtr<UObject>>::TIterator PrimPathToAssetIt = PrimPathToAssets.CreateIterator(); PrimPathToAssetIt;
			 ++PrimPathToAssetIt)
		{
			if (*FoundObject == PrimPathToAssetIt.Value().Get())
			{
				PrimPathToAssetIt.RemoveCurrent();
			}
		}

		ActiveAssets.Remove(*FoundObject);
		TransientStorage.Remove(Hash);
		PersistentStorage.Remove(Hash);
		OwnedAssets.Remove(*FoundObject);
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UObject* UUsdAssetCache::GetCachedAsset(const FString& Hash) const
{
	FScopeLock Lock(&CriticalSection);

	TObjectPtr<UObject> const* FoundObject = TransientStorage.Find(Hash);

	if (!FoundObject)
	{
		FoundObject = PersistentStorage.Find(Hash);
	}

	if (FoundObject)
	{
		ActiveAssets.Add(*FoundObject);
		return *FoundObject;
	}

	return nullptr;
}

void UUsdAssetCache::LinkAssetToPrim(const FString& PrimPath, UObject* Asset)
{
	if (!Asset)
	{
		return;
	}

	if (!OwnedAssets.Contains(Asset))
	{
		UE_LOGF(
			LogUsd,
			Warning,
			"Tried to set prim path '%ls' to asset '%ls', but it is not currently owned by the USD stage cache!",
			*PrimPath,
			*Asset->GetName()
		);
		return;
	}

	FScopeLock Lock(&CriticalSection);

	PrimPathToAssets.Add(PrimPath, Asset);
}

void UUsdAssetCache::RemoveAssetPrimLink(const FString& PrimPath)
{
	FScopeLock Lock(&CriticalSection);

	PrimPathToAssets.Remove(PrimPath);
}

UObject* UUsdAssetCache::GetAssetForPrim(const FString& PrimPath) const
{
	FScopeLock Lock(&CriticalSection);

	if (TWeakObjectPtr<UObject> const* FoundObjectPtr = PrimPathToAssets.Find(PrimPath))
	{
		if (UObject* FoundObject = FoundObjectPtr->Get())
		{
			ActiveAssets.Add(FoundObject);
			return FoundObject;
		}
	}

	return nullptr;
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

FString UUsdAssetCache::GetPrimForAsset(UObject* Asset) const
{
	FString PrimForAsset;

	{
		FScopeLock Lock(&CriticalSection);

		if (const FString* KeyPtr = PrimPathToAssets.FindKey(Asset))
		{
			PrimForAsset = *KeyPtr;
		}
	}

	return PrimForAsset;
}

FString UUsdAssetCache::GetHashForAsset(UObject* Asset) const
{
	if (!Asset || !OwnedAssets.Contains(Asset))
	{
		return {};
	}

	FScopeLock Lock(&CriticalSection);

	if (const FString* KeyPtr = TransientStorage.FindKey(Asset))
	{
		return *KeyPtr;
	}

	if (const FString* KeyPtr = PersistentStorage.FindKey(Asset))
	{
		return *KeyPtr;
	}

	return {};
}

void UUsdAssetCache::Reset()
{
	FScopeLock Lock(&CriticalSection);

	Modify();

	TransientStorage.Reset();
	OwnedAssets.Reset();
	PrimPathToAssets.Reset();
	ActiveAssets.Reset();
	PersistentStorage.Reset();
}

void UUsdAssetCache::MarkAssetsAsStale()
{
	FScopeLock Lock(&CriticalSection);

	ActiveAssets.Reset();
}

TSet<UObject*> UUsdAssetCache::GetActiveAssets() const
{
	return ActiveAssets;
}

void UUsdAssetCache::Serialize(FArchive& Ar)
{
	FScopeLock Lock(&CriticalSection);

	Super::Serialize(Ar);

	if (Ar.GetPortFlags() & PPF_DuplicateForPIE)
	{
		Ar << TransientStorage;
		Ar << ActiveAssets;
	}
}
