// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDAssetCache2.h"

#include "USDClassesModule.h"
#include "USDLog.h"

#include "Animation/Skeleton.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture2D.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Materials/MaterialInterface.h"
#include "MaterialShared.h"
#include "Misc/App.h"
#include "Misc/ScopeRWLock.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"
#include "Serialization/ObjectReader.h"
#include "Serialization/ObjectWriter.h"
#include "StaticMeshResources.h"
#include "UObject/ObjectRedirector.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(USDAssetCache2)

PRAGMA_DISABLE_DEPRECATION_WARNINGS

static bool GOnDemandCachedAssetLoading = true;
static FAutoConsoleVariableRef COnDemandCachedAssetLoading(
	TEXT("USD.OnDemandCachedAssetLoading"),
	GOnDemandCachedAssetLoading,
	TEXT("When true will cause the USD Asset Cache to only load cached UObjects from disk once they are needed, regardless of when the actual Asset "
		 "Cache asset itself is loaded.")
);

static int32 GCurrentCacheVersion = 1;
static int32 GCurrentAssetInfoVersion = 2;

namespace UE::AssetCache::Private
{
	UClass* FindClass(const TCHAR* ClassName)
	{
		check(ClassName);

		if (UClass* Result = FindFirstObject<UClass>(ClassName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("USDAssetCache2")))
		{
			return Result;
		}

		if (UObjectRedirector* RenamedClassRedirector = FindFirstObject<UObjectRedirector>(
				ClassName,
				EFindFirstObjectOptions::None,
				ELogVerbosity::Warning,
				TEXT("USDAssetCache2")
			))
		{
			return CastChecked<UClass>(RenamedClassRedirector->DestinationObject);
		}

		return nullptr;
	}

	bool AssetCanBePersisted(const TCHAR* ClassName)
	{
		if (!ClassName)
		{
			return false;
		}

		// Note we may fail to find a class even if the name is correct if we're running this on
		// engine exit
		UClass* AssetClass = FindClass(ClassName);
		if (!AssetClass)
		{
			return false;
		}

		static TArray<const UClass*> AllowedPersistentClasses = {
			UTexture2D::StaticClass(),
			UMaterialInterface::StaticClass(),
			UStaticMesh::StaticClass(),
			UPhysicsAsset::StaticClass(),
			USkeleton::StaticClass(),
			USkeletalMesh::StaticClass()};

		for (const UClass* AllowedClass : AllowedPersistentClasses)
		{
			if (AssetClass->IsChildOf(AllowedClass))
			{
				return true;
			}
		}

		return false;
	}

	uint64 GetAssetSizeBytes(UObject* Asset, bool bOnDisk)
	{
		if (!Asset)
		{
			return 0;
		}

		if (UTexture2D* Texture = Cast<UTexture2D>(Asset))
		{
#if WITH_EDITOR
			if (bOnDisk)
			{
				return static_cast<uint64>(Texture->Source.GetSizeOnDisk());
			}
			else
#endif	  // WITH_EDITOR
			{
				// TODO: Try doing this without calling GetPlatformData() if this is ever a speed bottleneck,
				// as apparently that can stall.
				// We can't use the texture Source here as it's not available at runtime.
				if (FTexturePlatformData* PlatformData = Texture->GetPlatformData())
				{
					const uint32 MipCount = 1;
					return GPixelFormats[PlatformData->PixelFormat].Get2DTextureSizeInBytes(PlatformData->SizeX, PlatformData->SizeY, MipCount);
				}
			}
		}

		return Asset->GetResourceSizeBytes(EResourceSizeMode::EstimatedTotal);
	}

	// Extends FObjectAndNameAsStringProxyArchive to support FLazyObjectPtr
	// Copied from FSnapshotCustomArchive
	struct FArchiveObjectPtrAsStringWrapper : public FObjectAndNameAsStringProxyArchive
	{
		FArchiveObjectPtrAsStringWrapper(FArchive& InInnerArchive)
			: FObjectAndNameAsStringProxyArchive(InInnerArchive, false)
		{
		}

		virtual FArchive& operator<<(FLazyObjectPtr& Obj) override
		{
			if (IsLoading())
			{
				// Reset before serializing to clear the internal weak pointer.
				Obj.Reset();
			}
			InnerArchive << Obj.GetUniqueID();

			return *this;
		}

		// Override FObjectAndNameAsStringProxyArchive::operator<< because we don't want to blindly call LoadObject
		// on 'None' strings, which is what gets serialized in case we have a nullptr (check
		// UObjectBaseUtility::GetPathName)
		virtual FArchive& operator<<(UObject*& Obj) override
		{
			if (IsLoading())
			{
				// load the path name to the object
				FString LoadedString;
				InnerArchive << LoadedString;

				const static FString NullptrString = TEXT("None");
				if (LoadedString == NullptrString || LoadedString.IsEmpty())
				{
					Obj = nullptr;
				}
				else
				{
					// look up the object by fully qualified pathname
					Obj = FindObject<UObject>(nullptr, *LoadedString, EFindObjectFlags::None);
					// If we couldn't find it, and we want to load it, do that
					if (!Obj && bLoadIfFindFails)
					{
						Obj = LoadObject<UObject>(nullptr, *LoadedString);
					}
				}
			}
			else
			{
				// save out the fully qualified object name
				FString SavedString(Obj->GetPathName());
				InnerArchive << SavedString;
			}
			return *this;
		}
	};

	// The main archiver used to write UObjects to byte buffers
	class FAssetCacheObjectWriter final : public FObjectWriter
	{
	public:
		FAssetCacheObjectWriter(UObject& Object, TArray<uint8>& Bytes)
			: FObjectWriter(Bytes)
		{
			SetIsLoading(false);
			SetIsSaving(true);
			SetIsPersistent(true);
		}
	};

	// The main archiver used to read UObjects from byte buffers
	class FAssetCacheObjectReader final : public FObjectReader
	{
	public:
		FAssetCacheObjectReader(UObject& Object, const TArray<uint8>& Bytes)
			: FObjectReader(Bytes)
		{
			SetIsLoading(true);
			SetIsSaving(false);
			SetIsPersistent(true);
		}
	};

	// Copied from DataprepSnapshot.cpp, the "WriteSnapshotData" function
	void SerializeObjectAndSubObjects(UObject* Object, TArray<uint8>& Buffer, FArchive& DestArchive)
	{
		if (!Object)
		{
			return;
		}

		// Helper struct to identify dependency of a UObject on other UObject(s) except given one (its outer)
		struct FObjectDependencyAnalyzer : public FArchiveUObject
		{
			FObjectDependencyAnalyzer(UObject* InSourceObject, const TSet<UObject*>& InValidObjects)
				: SourceObject(InSourceObject)
				, ValidObjects(InValidObjects)
			{
			}

			virtual FArchive& operator<<(UObject*& Obj) override
			{
				if (Obj != nullptr)
				{
					// Limit serialization to sub-object of source object
					if (Obj == SourceObject->GetOuter() || Obj->IsA<UPackage>() || (Obj->HasAnyFlags(RF_Public) && Obj->GetOuter()->IsA<UPackage>()))
					{
						return FArchiveUObject::operator<<(Obj);
					}
					// Stop serialization when a dependency is found or has been found
					else if (Obj != SourceObject && !ConsumerObjects.Contains(Obj) && ValidObjects.Contains(Obj))
					{
						ConsumerObjects.Add(Obj);
					}
				}

				return *this;
			}

			UObject* SourceObject;
			const TSet<UObject*>& ValidObjects;
			TSet<UObject*> ConsumerObjects;
		};

		FAssetCacheObjectWriter Writer{*Object, Buffer};
		FArchiveObjectPtrAsStringWrapper Ar{Writer};

		// Transfer over whether we're cooking or not, but otherwise leave at the default settings, as that is
		// how we'll try to deserialize on the other end, which is potentially at runtime
		Ar.SetSavePackageData(DestArchive.GetSavePackageData());
		Ar.SetFilterEditorOnly(DestArchive.IsFilterEditorOnly());
		Ar.SetUEVer(DestArchive.UEVer());
		Ar.SetLicenseeUEVer(DestArchive.LicenseeUEVer());
		Ar.SetCustomVersions(DestArchive.GetCustomVersions());

		ensure(!Object->IsEditorOnly() || !Ar.IsCooking());

		// Collect sub-objects depending on input object including nested objects
		TArray<UObject*> SubObjectsArray;
		GetObjectsWithOuter(Object, SubObjectsArray, EGetObjectsFlags::IncludeNestedObjects);
		TSet<UObject*> AllowedSubObjectsSet(SubObjectsArray);

		// Keep track of which subobjects are actually referenced by our UObject tree.
		// Other objects may just be hiding inside of them if they're referenced by an external UObject, and so
		// shouldn't be serialized to disk (this happens with UAssetImportData: Some UObjects are created with them
		// already, and those importdatas remain there even when we swap in our own UUsdAssetImportData, as they're
		// referenced by the transaction buffer);
		TSet<UObject*> ActualDependencies;
		{
			FObjectDependencyAnalyzer Analyzer(Object, AllowedSubObjectsSet);
			Object->Serialize(Analyzer);
			ActualDependencies.Append(Analyzer.ConsumerObjects);
		}

		// Sort array of sub-objects based on their inter-dependency
		{
			// Create and initialize graph of dependency between sub-objects
			TMap<UObject*, TSet<UObject*>> SubObjectDependencyGraph;
			SubObjectDependencyGraph.Reserve(SubObjectsArray.Num());

			// Build graph of dependency: each entry contains the set of sub-objects to create before itself.
			// Note that we're not using ActualDependencies for this because we want to map out
			// all transitive dependencies as well, and FObjectDependencyAnalyzer is not recursive
			for (UObject* SubObject : SubObjectsArray)
			{
				FObjectDependencyAnalyzer SubAnalyzer(SubObject, AllowedSubObjectsSet);
				SubObject->Serialize(SubAnalyzer);

				SubObjectDependencyGraph.FindOrAdd(SubObject).Append(SubAnalyzer.ConsumerObjects);
				ActualDependencies.Append(SubAnalyzer.ConsumerObjects);
			}

			// Sort array of sub-objects: first objects do not depend on ones below
			int32 Count = ActualDependencies.Num();
			SubObjectsArray.Empty(Count);
			while (Count != SubObjectsArray.Num())
			{
				for (auto& Entry : SubObjectDependencyGraph)
				{
					if (Entry.Value.Num() == 0)
					{
						UObject* SubObject = Entry.Key;

						SubObjectDependencyGraph.Remove(SubObject);

						if (ActualDependencies.Contains(SubObject))
						{
							SubObjectsArray.Add(SubObject);
						}

						for (auto& SubEntry : SubObjectDependencyGraph)
						{
							if (SubEntry.Value.Num() > 0)
							{
								SubEntry.Value.Remove(SubObject);
							}
						}

						break;
					}
				}
			}
		}

		// Filter objects we'll be serializing beforehand. This is a significant difference from the reference code,
		// but essentially if we aren't going to be serializing an object, we shouldn't serialize the info about it
		// either: If it's a default subobject we'll find another instance of it with the same name when deserializing
		// anyway. If it's not, it probably shouldn't be serialized anyway
		{
			TArray<UObject*> FilteredSubObjects;
			FilteredSubObjects.Reserve(SubObjectsArray.Num());
			for (UObject* SubObject : SubObjectsArray)
			{
				if (SubObject && !SubObject->HasAnyFlags(RF_DefaultSubObject) && (!SubObject->IsEditorOnly() || !Ar.IsCooking()))
				{
					FilteredSubObjects.Add(SubObject);
				}
			}
			Swap(FilteredSubObjects, SubObjectsArray);
		}

		// Serialize size of array
		int32 SubObjectsCount = SubObjectsArray.Num();
		Ar << SubObjectsCount;

		// Serialize class and path of each sub-object
		for (int32 Index = SubObjectsArray.Num() - 1; Index >= 0; --Index)
		{
			const UObject* SubObject = SubObjectsArray[Index];

			UClass* SubObjectClass = SubObject->GetClass();

			FString ClassName = SubObjectClass->GetPathName();
			Ar << ClassName;

			int32 ObjectFlags = SubObject->GetFlags();
			Ar << ObjectFlags;
		}

		// Serialize sub-objects' outer path and name
		// Done in reverse order since a sub-object can be the outer of another sub-object
		// it depends on. Not the opposite
		for (int32 Index = SubObjectsArray.Num() - 1; Index >= 0; --Index)
		{
			const UObject* SubObject = SubObjectsArray[Index];

			FSoftObjectPath SoftPath(SubObject->GetOuter());

			FString SoftPathString = SoftPath.ToString();
			Ar << SoftPathString;

			FString SubObjectName = SubObject->GetName();
			Ar << SubObjectName;
		}

		for (UObject* SubObject : SubObjectsArray)
		{
			// Ensure these objects are serialized with flags as if they were regular
			// persistent assets, in case they have branching on their serialization functions that watch
			// out for this
			SubObject->ClearFlags(RF_Transient);
			SubObject->SetFlags(RF_Public);
			{
				SubObject->Serialize(Ar);
			}
			SubObject->SetFlags(RF_Transient);
		}

		Object->ClearFlags(RF_Transient);
		Object->SetFlags(RF_Public);
		{
			Object->Serialize(Ar);
		}
		Object->SetFlags(RF_Transient);
	}

	// Copied from DataprepSnapshot.cpp, the "ReadSnapshotData" function
	void DeserializeObjectAndSubObjects(UObject* Object, const TArray<uint8>& InBuffer)
	{
		if (!Object)
		{
			return;
		}

		// Remove all objects created by default that InObject is an outer of.
		// This method must obviously be called just after the InObject is created
		auto RemoveDefaultDependencies = [](UObject* InObject)
		{
			TArray<UObject*> ObjectsWithOuter;
			GetObjectsWithOuter(InObject, ObjectsWithOuter, EGetObjectsFlags::IncludeNestedObjects);

			for (UObject* ObjectWithOuter : ObjectsWithOuter)
			{
				// Do not delete default sub-objects
				if (!ObjectWithOuter->HasAnyFlags(RF_DefaultSubObject))
				{
					if (ObjectWithOuter != nullptr)
					{
						ObjectWithOuter
							->Rename(nullptr, GetTransientPackage(), REN_NonTransactional | REN_DontCreateRedirectors);
					}
					ObjectWithOuter->MarkAsGarbage();
				}
			}
		};

		RemoveDefaultDependencies(Object);

		FAssetCacheObjectReader Reader{*Object, InBuffer};
		FArchiveObjectPtrAsStringWrapper Ar{Reader};
		Ar.bLoadIfFindFails = true;

		// Deserialize count of sub-objects
		int32 SubObjectsCount = 0;
		Ar << SubObjectsCount;

		// Create empty sub-objects based on class and patch
		TArray<UObject*> SubObjectsArray;
		SubObjectsArray.SetNumZeroed(SubObjectsCount);

		// Create root name to avoid name collision
		FString RootName = FGuid::NewGuid().ToString();

		for (int32 Index = SubObjectsCount - 1; Index >= 0; --Index)
		{
			FString ClassName;
			Ar << ClassName;

			UClass* SubObjectClass = FindClass(*ClassName);
			check(SubObjectClass);

			int32 ObjectFlags;
			Ar << ObjectFlags;

			FString SubObjectName = RootName + FString::FromInt(Index);

			UObject* SubObject = NewObject<UObject>(Object, SubObjectClass, *SubObjectName, EObjectFlags(ObjectFlags));
			SubObjectsArray[Index] = SubObject;

			RemoveDefaultDependencies(SubObject);
		}

		// Restore sub-objects' outer if original outer differs from Object
		// Restoration is done in the order the serialization was done: reverse order
		for (int32 Index = SubObjectsArray.Num() - 1; Index >= 0; --Index)
		{
			FString SoftPathString;
			Ar << SoftPathString;

			FString SubObjectName;
			Ar << SubObjectName;

			const FSoftObjectPath SoftPath(SoftPathString);

			UObject* NewOuter = SoftPath.ResolveObject();
			ensure(NewOuter);

			if (UObject* SubObject = SubObjectsArray[Index])
			{
				if (NewOuter != SubObject->GetOuter())
				{
					const TCHAR* NewName = nullptr;
					SubObject->Rename(NewName, NewOuter, REN_NonTransactional | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
				}

				UObject* NoNewOuter = nullptr;
				if (SubObjectName != SubObject->GetName() && SubObject->Rename(*SubObjectName, NoNewOuter, REN_Test))
				{
					SubObject->Rename(*SubObjectName, NoNewOuter, REN_NonTransactional | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
				}
			}
		}

		TArray<UTexture*> DeserializedTextures;
		TArray<UStaticMesh*> DeserializedStaticMeshes;
		TArray<USkeletalMesh*> DeserializedSkeletalMeshes;
		TArray<UMaterialInterface*> DeserializedMaterials;

		for (UObject* SubObject : SubObjectsArray)
		{
			ensure(SubObject && !SubObject->HasAnyFlags(RF_DefaultSubObject));

			SubObject->Serialize(Ar);
			SubObject->SetFlags(RF_Transient | RF_Public);
		}

		Object->Serialize(Ar);
		Object->SetFlags(RF_Transient | RF_Public);

		if (UTexture* Texture = Cast<UTexture>(Object))
		{
			Texture->UpdateResource();
		}
#if WITH_EDITOR
		else if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(Object))
		{
			// These steps are copied from USDGeomMeshTranslator.cpp, from the PreBuildStaticMesh, BuildStaticMesh
			// and PostBuildStaticMesh functions, although we will have serialized our body setup and extended bounds
			// already. Ideally this will hit DDC and should be fast.
			// We only need these in the editor: When we're cooking we'll just save the cooked mesh data into the
			// asset itself, so at runtime there is no need for an additional "build" step that we have to handle here

			if (FApp::CanEverRender() || !FPlatformProperties::RequiresCookedData())
			{
				StaticMesh->SetRenderData(MakeUnique<FStaticMeshRenderData>());
				ITargetPlatformManagerModule& TargetPlatformManager = GetTargetPlatformManagerRef();
				ITargetPlatform* RunningPlatform = TargetPlatformManager.GetRunningTargetPlatform();
				const FStaticMeshLODSettings& LODSettings = RunningPlatform->GetStaticMeshLODSettings();
				StaticMesh->GetRenderData()->Cache(RunningPlatform, StaticMesh, LODSettings);
				StaticMesh->InitResources();
			}
		}
		else if (USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Object))
		{
			SkeletalMesh->CalculateInvRefMatrices();
			SkeletalMesh->InitMorphTargetsAndRebuildRenderData();
		}
#endif	  // WITH_EDITOR
		if (UMaterialInterface* Material = Cast<UMaterialInterface>(Object))
		{
			// TODO: If this presents a performance issue we could possibly just do part of the work that this seems
			// to be doing, but until then it's probably good to run the entire FMaterialUpdateContext for robustness
			FMaterialUpdateContext UpdateContext;
			UpdateContext.AddMaterialInterface(Material);
		}
	}
}	 // namespace UE::AssetCache::Private

FArchive& operator<<(FArchive& Ar, UUsdAssetCache2::ECacheStorageType& Type)
{
	Ar << (uint8&)Type;
	return Ar;
}

void UUsdAssetCache2::FCachedAssetInfo::Serialize(FArchive& Ar, UObject* Owner)
{
	int32 SavedAssetInfoVersion = INDEX_NONE;
	if (Ar.IsSaving())
	{
		Ar << GCurrentAssetInfoVersion;
		SavedAssetInfoVersion = GCurrentAssetInfoVersion;
	}
	else if (Ar.IsLoading())
	{
		Ar << SavedAssetInfoVersion;
		ensure(SavedAssetInfoVersion <= GCurrentAssetInfoVersion);
	}

	UE_LOGF(
		LogUsd,
		Verbose,
		"Serializing Asset Info '%ls' (%ls) with version %d, storage type: %d, disk bytes: %lld, memory bytes: %lld, bulkdata size: %lld, saving? %d, "
			 "persistent? %d, "
			 "transacting? %d, duplicating? %d, cooking? %d",
		*AssetName,
		*Hash,
		SavedAssetInfoVersion,
		int(CurrentStorageType),
		SizeOnDiskInBytes,
		SizeOnMemoryInBytes,
		BulkData.GetBulkDataSize(),
		Ar.IsSaving(),
		Ar.IsPersistent(),
		Ar.IsTransacting(),
		(Ar.GetPortFlags() & (PPF_DuplicateForPIE | PPF_Duplicate)),
		Ar.IsCooking()
	);

	Ar << Hash;
	Ar << AssetClassName;
	Ar << AssetName;
	Ar << (uint32&)AssetFlags;

	Ar << SizeOnDiskInBytes;
	Ar << SizeOnMemoryInBytes;
	Ar << Dependencies;
	Ar << Consumers;

	// We do need to serialize this in case we do too many undo/redo operations without a cache resync between them
	if (SavedAssetInfoVersion > 1)
	{
		Ar << CurrentStorageType;
	}

	// We only need to read/write to our bulkdata when saving to disk
	if (Ar.IsPersistent())
	{
		// Very important: This flag ensures that when we call Serialize below we won't actually
		// load the full bulkdata right away, so that we can do it on-demand
		BulkData.SetBulkDataFlags(BULKDATA_Force_NOT_InlinePayload);
		BulkData.Serialize(Ar, Owner);
	}
}

UUsdAssetCache2::UUsdAssetCache2()
	: LRUCache(-1)	  // Negative MaxElements as we don't want the cache to drop elements by itself
{
}

#if WITH_EDITOR
void UUsdAssetCache2::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UUsdAssetCache2, PersistentAssetStorageSizeMB))
	{
		// We won't be persisting anything if we're transient anyway
		if (GetOutermost() != GetTransientPackage())
		{
			RefreshStorage();
		}
	}
	else if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UUsdAssetCache2, UnreferencedAssetStorageSizeMB))
	{
		RefreshStorage();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif	  // #if WITH_EDITOR

void UUsdAssetCache2::CacheAsset(const FString& Hash, UObject* Asset, const UObject* Referencer)
{
	if (!Asset)
	{
		UE_LOGF(LogUsd, Warning, "Attempted to add a null asset to USD Asset Cache with hash '%ls'!", *Hash);
		return;
	}

	FCachedAssetInfo* FoundInfo = nullptr;
	{
		FReadScopeLock Lock(RWLock);

		FoundInfo = LRUCache.Find(Asset);
		if (FoundInfo)
		{
			if (FoundInfo->Hash != Hash)
			{
				UE_LOGF(
					LogUsd,
					Warning,
					"Attempted to store asset '%ls' more than once into the USD Asset Cache '%ls'! (old hash: '%ls', new hash: '%ls')",
					*Asset->GetPathName(),
					*GetPathName(),
					*FoundInfo->Hash,
					*Hash
				);
				return;
			}
		}
	}

	const UObject* ReferencerToUse = Referencer ? Referencer : CurrentScopedReferencer;

	if (UObject* ExistingAsset = AssetStorage.FindRef(Hash))
	{
		if (Asset == ExistingAsset)
		{
			UE_LOGF(
				LogUsd,
				Verbose,
				"Recaching asset '%ls' with hash '%ls' into the USD Asset Cache '%ls'",
				*Asset->GetPathName(),
				*Hash,
				*GetPathName()
			);

			// We're trying to cache an asset that's already in the cache with this same hash.
			// Just record that we "touched" this asset and return
			Modify();

			FWriteScopeLock Lock(RWLock);
			TouchAssetInternal(Asset, ReferencerToUse);
			return;
		}
		else
		{
			bool bRemoved = false;
			if (CanRemoveAsset(Hash))
			{
				if (UObject* OldAsset = RemoveAsset(Hash))
				{
					bRemoved = true;
					UE_LOGF(
						LogUsd,
						Log,
						"Overwriting asset '%ls' with '%ls' (for hash '%ls') in the USD Asset Cache '%ls'",
						*OldAsset->GetPathName(),
						*Asset->GetPathName(),
						*Hash,
						*GetPathName()
					);
				}
			}

			if (!bRemoved)
			{
				UE_LOGF(
					LogUsd,
					Error,
					"Irrecoverable hash collision! Asset '%ls' cannot be cached into USD Asset Cache '%ls' as the existing asset '%ls' with the "
						 "same hash '%ls' could not be discarded!",
					*Asset->GetPathName(),
					*GetPathName(),
					*ExistingAsset->GetPathName(),
					*Hash
				);
				return;
			}
		}
	}

	UE_LOGF(LogUsd, Verbose, "Caching asset '%ls' with hash '%ls' into the USD Asset Cache '%ls'", *Asset->GetPathName(), *Hash, *GetPathName());

	Modify();

	FWriteScopeLock Lock(RWLock);

	// We only cache assets that are originally transient, and calling code should know this... we shouldn't try
	// caching some random persistent asset
	ensure(Asset->HasAnyFlags(RF_Transient));
	ensure(Asset->GetOutermost() == GetTransientPackage() || Asset->GetOuter() == this);

	// Rename before adding to our maps as we track some stuff with SoftObjectPaths
	if (Asset->GetOuter() != this)
	{
		UObject* NewOuter = this;
		const FName NewName = MakeUniqueObjectName(NewOuter, Asset->GetClass(), Asset->GetFName());
		Asset->Rename(*NewName.ToString(), NewOuter, REN_DontCreateRedirectors | REN_NonTransactional | REN_AllowPackageLinkerMismatch);
	}

	FCachedAssetInfo NewInfo;
	NewInfo.Hash = Hash;
	NewInfo.AssetClassName = Asset->GetClass()->GetPathName();
	NewInfo.AssetName = Asset->GetName();
	NewInfo.AssetFlags = Asset->GetFlags();
	if (ReferencerToUse)
	{
		NewInfo.Referencers.Add(FObjectKey{ReferencerToUse});
	}

	AssetStorage.Add(Hash, Asset);
	LRUCache.Add(Asset, MoveTemp(NewInfo));
	ActiveAssets.Add(Asset);
}

bool UUsdAssetCache2::CanRemoveAsset(const FString& Hash)
{
	FReadScopeLock Lock(RWLock);

	FCachedAssetInfo* Info = nullptr;

	if (UObject* FoundAsset = AssetStorage.FindRef(Hash))
	{
		Info = LRUCache.Find(FoundAsset);
	}
	else if (FSoftObjectPath* FoundPendingPath = PendingPersistentStorage.Find(Hash))
	{
		Info = LRUCache.Find(*FoundPendingPath);
	}

	if (!Info)
	{
		return true;
	}

	if (Info->Referencers.Num() > 0 || Info->Consumers.Num() > 0)
	{
		return false;
	}

	return true;
}

UObject* UUsdAssetCache2::RemoveAsset(const FString& Hash)
{
	FSoftObjectPath FoundObjectPath;
	TObjectPtr<UObject> FoundObject;
	{
		FWriteScopeLock Lock(RWLock);

		if (AssetStorage.RemoveAndCopyValue(Hash, FoundObject))
		{
			FoundObjectPath = FoundObject;
			ActiveAssets.Remove(FoundObject);
		}
		else
		{
			PendingPersistentStorage.RemoveAndCopyValue(Hash, FoundObjectPath);
		}
	}

	if (!FoundObjectPath.IsNull())
	{
		Modify();

		// Modify will call Serialize, so we need to release and get a new lock
		FWriteScopeLock Lock(RWLock);

		if (FCachedAssetInfo* Info = LRUCache.Find(FoundObjectPath))
		{
			ensure(Info->Referencers.Num() == 0);
			ensure(Info->Consumers.Num() == 0);

			// Make sure all of our dependencies know we're no longer Consumer on them
			for (const FSoftObjectPath& Dependency : Info->Dependencies)
			{
				if (FCachedAssetInfo* DependencyInfo = LRUCache.Find(Dependency))
				{
					DependencyInfo->Consumers.Remove(FoundObjectPath);
				}
			}
			Info->Dependencies.Empty();

			TryUnloadAsset(*Info);
			Info->BulkData.RemoveBulkData();
			LRUCache.Remove(FoundObjectPath);
		}
	}

	return FoundObject;
}

UObject* UUsdAssetCache2::GetCachedAsset(const FString& Hash)
{
	// Try to fetch a loaded asset from the cache with this hash
	UObject* FoundAsset = nullptr;
	{
		FReadScopeLock Lock{RWLock};
		FoundAsset = AssetStorage.FindRef(Hash);
	}
	UE_LOGF(
		LogUsd,
		Verbose,
		"Fetching cached asset with hash '%ls' for cache '%ls'. Loaded in AssetStorage? %d. Size of LRU cache: %d, Pending persistent assets: %d",
		*Hash,
		*GetPathName(),
		FoundAsset != nullptr,
		LRUCache.Num(),
		PendingPersistentStorage.Num()
	);
	if (FoundAsset)
	{
		FWriteScopeLock Lock{RWLock};
		TouchAssetInternal(FoundAsset, CurrentScopedReferencer);
		return FoundAsset;
	}

	// Check if we know about this persistent asset but just haven't loaded it yet
	FSoftObjectPath* FoundAssetPath = nullptr;
	{
		FReadScopeLock Lock{RWLock};
		FoundAssetPath = PendingPersistentStorage.Find(Hash);
		if (!FoundAssetPath)
		{
			UE_LOGF(LogUsd, Verbose, "Full cache miss: We do not know about any assets with hash '%ls' in cache '%ls'", *Hash, *GetPathName());
			return nullptr;
		}
	}

	TFunction<UObject*(FCachedAssetInfo * ObjectInfo)> LoadAssetFromBulkData = [this](FCachedAssetInfo* ObjectInfo) -> UObject*
	{
		if (!ObjectInfo)
		{
			return nullptr;
		}

		const FString ClassNameString = ObjectInfo->AssetClassName.ToString();
		UClass* FoundClass = UE::AssetCache::Private::FindClass(*ClassNameString);
		if (!FoundClass)
		{
			UE_LOGF(
				LogUsd,
				Warning,
				"Failed to find object class '%ls' when deserializing asset '%ls' with hash '%ls'",
				*ClassNameString,
				*ObjectInfo->AssetName,
				*ObjectInfo->Hash
			);
			return nullptr;
		}

		// Deserialize the referenced asset from the Info's bulkdata
		// TODO: If performance is an issue here we can probably prevent this copy by having our own
		// FObjectReader that uses a pointer and offset and reads it off the bulkdata directly
		TArray<uint8> BulkDataCopy;
		const void* BulkDataBytes = ObjectInfo->BulkData.LockReadOnly();
		{
			BulkDataCopy.SetNumUninitialized(ObjectInfo->BulkData.GetBulkDataSize());
			FMemory::Memcpy(BulkDataCopy.GetData(), BulkDataBytes, BulkDataCopy.Num());
		}
		ObjectInfo->BulkData.Unlock();

		FName AssetName = ObjectInfo->AssetName.IsEmpty() ? NAME_None : FName{*ObjectInfo->AssetName};

		// They're always "transient", since we'll persist them ourselves
		UObject* NewAsset = NewObject<UObject>(this, FoundClass, AssetName, ObjectInfo->AssetFlags | RF_Transient);
		UE::AssetCache::Private::DeserializeObjectAndSubObjects(NewAsset, BulkDataCopy);
		ensure(NewAsset);

		UE_LOGF(
			LogUsd,
			Log,
			"Deserialized asset at '%ls' (%ls, hash '%ls', storage %d) from bulkdata (%.3f MB)",
			*ObjectInfo->AssetName,
			NewAsset ? *NewAsset->GetPathName() : TEXT("nullptr"),
			*ObjectInfo->Hash,
			int(ObjectInfo->CurrentStorageType),
			BulkDataCopy.Num() / 1000000.0
		);

		AssetStorage.Add(ObjectInfo->Hash, NewAsset);
		PendingPersistentStorage.Remove(ObjectInfo->Hash);
		return NewAsset;
	};

	// Since LoadAssetFromBulkData will modify our member maps, the safest is to collect all hashes that we'll be
	// loading first and then do a pass
	TArray<FSoftObjectPath> AssetsToLoad;
	TFunction<void(const FSoftObjectPath&)> RecursivelyCollectAssetsToLoad;
	RecursivelyCollectAssetsToLoad = [this, &RecursivelyCollectAssetsToLoad, &AssetsToLoad](const FSoftObjectPath& AssetPath)
	{
		if (FCachedAssetInfo* FoundCachedInfo = LRUCache.Find(AssetPath))
		{
			if (PendingPersistentStorage.Contains(FoundCachedInfo->Hash))
			{
				// Before we deserialize the asset, we need to make sure all of it's dependencies are also
				// deserialized, so push our dependency assets first
				for (const FSoftObjectPath& DependencyPath : FoundCachedInfo->Dependencies)
				{
					RecursivelyCollectAssetsToLoad(DependencyPath);
				}

				AssetsToLoad.Add(AssetPath);
			}
		}
		else
		{
			UE_LOGF(LogUsd, Warning, "Failed to find info about asset '%ls' when deserializing on-demand", *AssetPath.ToString());
		}
	};
	{
		FReadScopeLock Lock{RWLock};
		RecursivelyCollectAssetsToLoad(*FoundAssetPath);
	}

	ensure(AssetsToLoad.Num() > 0);
	UE_LOGF(
		LogUsd,
		Verbose,
		"Partial cache miss: Will attempt to load %d assets from bulk data of cache '%ls'",
		AssetsToLoad.Num(),
		*GetPathName()
	);
	{
		FWriteScopeLock Lock{RWLock};
		for (const FSoftObjectPath& AssetToLoad : AssetsToLoad)
		{
			if (FCachedAssetInfo* FoundCachedInfo = LRUCache.Find(AssetToLoad))
			{
				UObject* LoadedAsset = LoadAssetFromBulkData(FoundCachedInfo);
				ensure(LoadedAsset);

				TouchAssetInternal(LoadedAsset, CurrentScopedReferencer);
				FoundAsset = LoadedAsset;	 // The very last asset we loaded is our main asset, as it's pushed last into AssetsToLoad
			}
		}
	}

	return FoundAsset;
}

bool UUsdAssetCache2::TouchAsset(const UObject* Asset, const UObject* Referencer)
{
	if (!Asset)
	{
		return false;
	}

	if (IsAssetOwnedByCache(Asset->GetPathName()))
	{
		Modify();

		FWriteScopeLock Lock(RWLock);
		TouchAssetInternal(Asset, Referencer);

		return true;
	}

	return false;
}

bool UUsdAssetCache2::AddAssetReference(const UObject* Asset, const UObject* Referencer)
{
	FWriteScopeLock Lock(RWLock);
	return AddAssetReferenceInternal(Asset, Referencer);
}

bool UUsdAssetCache2::RemoveAssetReference(const UObject* Asset, const UObject* Referencer)
{
	if (!Asset)
	{
		return false;
	}

	FWriteScopeLock Lock(RWLock);

	bool bRemovedSomething = false;
	if (FCachedAssetInfo* Info = LRUCache.Find(Asset))
	{
		if (Referencer)
		{
			const int32 NumRemoved = Info->Referencers.Remove(FObjectKey{Referencer});
			if (NumRemoved > 0)
			{
				UE_LOGF(LogUsd, Verbose, "Removed referencer '%ls' from asset '%ls'", *Referencer->GetPathName(), *Asset->GetPathName());

				bRemovedSomething = true;
			}
		}
		else
		{
			const int32 NumRemoved = Info->Referencers.Num();
			Info->Referencers.Reset();

			if (NumRemoved > 0)
			{
				UE_LOGF(LogUsd, Verbose, "Removed all '%d' referencers for asset '%ls'", NumRemoved, *Asset->GetPathName());

				bRemovedSomething = true;
			}
		}
	}

	return bRemovedSomething;
}

bool UUsdAssetCache2::RemoveAllAssetReferences(const UObject* Referencer)
{
	if (!Referencer)
	{
		return false;
	}

	FWriteScopeLock Lock(RWLock);

	FObjectKey Key{Referencer};

	bool bRemovedSomething = false;
	for (TLruCache<FSoftObjectPath, FCachedAssetInfo>::TIterator Iter{LRUCache}; Iter; ++Iter)
	{
		FCachedAssetInfo& Info = Iter.Value();
		int32 NumRemoved = Info.Referencers.Remove(Key);
		if (NumRemoved > 0)
		{
			bRemovedSomething = true;
		}
	}

	return bRemovedSomething;
}

FString UUsdAssetCache2::GetHashForAsset(const UObject* Asset) const
{
	if (!Asset)
	{
		return {};
	}

	FReadScopeLock Lock(RWLock);

	if (const FCachedAssetInfo* Info = LRUCache.Find(Asset))
	{
		return Info->Hash;
	}

	return {};
}

bool UUsdAssetCache2::IsAssetOwnedByCache(const FString& AssetPath) const
{
	FReadScopeLock Lock(RWLock);

	return IsAssetOwnedByCacheInternal(AssetPath);
}

int32 UUsdAssetCache2::GetNumAssets() const
{
	return LRUCache.Num();
}

TArray<FString> UUsdAssetCache2::GetAllAssetHashes() const
{
	FReadScopeLock Lock(RWLock);

	TArray<FString> Hashes;
	Hashes.Reserve(LRUCache.Num());

	for (TLruCache<FSoftObjectPath, FCachedAssetInfo>::TConstIterator Iter{LRUCache}; Iter; ++Iter)
	{
		Hashes.Add(Iter.Value().Hash);
	}

	return Hashes;
}

TArray<UObject*> UUsdAssetCache2::GetAllLoadedAssets() const
{
	FReadScopeLock Lock(RWLock);

	TArray<UObject*> Objects;
	Objects.Reserve(AssetStorage.Num());

	for (const TPair<FString, TObjectPtr<UObject>>& Pair : AssetStorage)
	{
		Objects.Add(Pair.Value.Get());
	}

	return Objects;
}

TArray<FString> UUsdAssetCache2::GetAllCachedAssetPaths() const
{
	FReadScopeLock Lock(RWLock);

	TArray<FString> Paths;
	Paths.Reserve(LRUCache.Num());

	for (TLruCache<FSoftObjectPath, FCachedAssetInfo>::TConstIterator Iter{LRUCache}; Iter; ++Iter)
	{
		Paths.Add(Iter.Key().ToString());
	}

	return Paths;
}

void UUsdAssetCache2::Reset()
{
	Modify();

	FWriteScopeLock Lock(RWLock);

	// We never want the LRUCache to drop items by itself
	const int32 MaxElements = -1;
	AssetStorage.Reset();
	ActiveAssets.Reset();
	for (TLruCache<FSoftObjectPath, FCachedAssetInfo>::TIterator Iter{LRUCache}; Iter; ++Iter)
	{
		FCachedAssetInfo& Info = Iter.Value();
		Info.BulkData.RemoveBulkData();
	}
	LRUCache.Empty(MaxElements);
	PendingPersistentStorage.Reset();
}

void UUsdAssetCache2::RefreshStorage()
{
	using namespace UE::AssetCache::Private;

	const bool bIsTransientAssetCache = GetOutermost() == GetTransientPackage();

	const uint64 MaxUnreferencedTransientBytes = UnreferencedAssetStorageSizeMB < 0.0
													 ? TNumericLimits<uint64>::Max()
													 : static_cast<uint64>(FMath::Max(UnreferencedAssetStorageSizeMB, 0.0) + 0.5) * 1000000;

	const uint64 MaxPersistentBytes = PersistentAssetStorageSizeMB < 0.0
										  ? TNumericLimits<uint64>::Max()
										  : static_cast<uint64>(FMath::Max(PersistentAssetStorageSizeMB, 0.0) + 0.5) * 1000000;

	uint64 UnreferencedAssetsSumBytes = 0;
	uint64 PersistentAssetSumBytes = 0;

	TMap<FString, TObjectPtr<UObject>> NewAssetStorage;
	TMap<FString, FSoftObjectPath> NewPendingPersistentStorage;

	NewAssetStorage.Reserve(AssetStorage.Num());
	NewPendingPersistentStorage.Reserve(PendingPersistentStorage.Num());

	TSet<FSoftObjectPath> VisitedAssets;

	Modify();

	FWriteScopeLock Lock(RWLock);

	double StartTime = FPlatformTime::Cycles64();

	// Do an initial pass to reset our current storage types and fetch some remaining data
	for (TLruCache<FSoftObjectPath, FCachedAssetInfo>::TIterator Iter{LRUCache}; Iter; ++Iter)
	{
		FCachedAssetInfo& Info = Iter.Value();

		Info.CurrentStorageType = ECacheStorageType::None;

		if (Info.SizeOnDiskInBytes == 0)
		{
			const FSoftObjectPath& AssetPath = Iter.Key();
			UObject* Asset = AssetStorage.FindRef(Info.Hash);

			// If we have no size yet, then we must have just cached the asset, and so it must still be loaded
			ensure(Asset);

			// We defer fetching asset sizes to here as some assets are not "complete" by the time we originally cache
			// them, but should be fine now (e.g. static meshes that still need to be built).
			// We should always do one RefreshStorage after we fully finish any operation that adds assets
			// (e.g. loading a stage, reloading, etc.), so we'll always have at least have one pass here
			Info.SizeOnDiskInBytes = UE::AssetCache::Private::GetAssetSizeBytes(Asset, /*bOnDisk*/ true);
			Info.SizeOnMemoryInBytes = UE::AssetCache::Private::GetAssetSizeBytes(Asset, /*bOnDisk*/ false);

			// We also wait to fetch dependencies only here so that we know we have cached everything we want,
			// and can ensure that IsAssetOwnedByCache wouldn't reject some asset that would have eventually be cached
			// (e.g. if we cached a SkeletalMesh and only later cached the Skeleton that it uses)
			TSet<UObject*> Dependencies = IUsdClassesModule::GetAssetDependencies(Asset);
			Info.Dependencies.Empty(Dependencies.Num());
			for (UObject* Dependency : Dependencies)
			{
				// We need to check this here because sometimes our assets refer to other persistent assets
				// (e.g. unreal materials)
				if (IsAssetOwnedByCacheInternal(Dependency->GetPathName()))
				{
					Info.Dependencies.Add(Dependency);
					if (FCachedAssetInfo* DependencyInfo = LRUCache.Find(Dependency))
					{
						DependencyInfo->Consumers.Add(AssetPath);
					}
				}
			}
		}
	}

	TFunction<bool(const FCachedAssetInfo&)> ConsumerTreeHasReferencedAsset;
	ConsumerTreeHasReferencedAsset = [this, &ConsumerTreeHasReferencedAsset](const FCachedAssetInfo& Info) -> bool
	{
		for (const FSoftObjectPath& Consumer : Info.Consumers)
		{
			FCachedAssetInfo* ConsumerInfo = LRUCache.Find(Consumer);
			if (ensure(ConsumerInfo))
			{
				if (ConsumerInfo->Referencers.Num() > 0)
				{
					return true;
				}

				if (ConsumerTreeHasReferencedAsset(*ConsumerInfo))
				{
					return true;
				}
			}
		}

		return false;
	};

	TFunction<ECacheStorageType(FCachedAssetInfo&, FSoftObjectPath&)> VisitEntryRecursively;
	VisitEntryRecursively = [this,
							 &ConsumerTreeHasReferencedAsset,
							 &VisitEntryRecursively,
							 &VisitedAssets,
							 bIsTransientAssetCache,
							 MaxUnreferencedTransientBytes,
							 MaxPersistentBytes,
							 &UnreferencedAssetsSumBytes,
							 &PersistentAssetSumBytes,
							 &NewAssetStorage,
							 &NewPendingPersistentStorage](FCachedAssetInfo& Info, FSoftObjectPath& AssetPath) -> ECacheStorageType
	{
		if (AssetPath.IsNull())
		{
			ensure(false);
			return ECacheStorageType::None;
		}

		if (VisitedAssets.Contains(AssetPath))
		{
			return Info.CurrentStorageType;
		}
		VisitedAssets.Add(AssetPath);

		// Always visit dependencies first: We can only ever persist an asset if its dependencies are also
		// persisted. Here we'll also track the "worst" storage any of our dependencies got (e.g. if one of our
		// textures managed Persistent, but the other got Referenced, we'll track Referenced)
		ECacheStorageType WorstDependencyStorage = ECacheStorageType::Persistent;
		for (FSoftObjectPath& Dependency : Info.Dependencies)
		{
			FCachedAssetInfo* DependencyInfo = LRUCache.Find(Dependency);
			if (!ensure(DependencyInfo))
			{
				continue;
			}

			ECacheStorageType DependencyStorage = VisitEntryRecursively(*DependencyInfo, Dependency);
			WorstDependencyStorage = static_cast<ECacheStorageType>(
				FMath::Min(static_cast<uint8>(WorstDependencyStorage), static_cast<uint8>(DependencyStorage))
			);
		}

		const bool bAssetFitsInPersistent = PersistentAssetSumBytes + Info.SizeOnDiskInBytes <= MaxPersistentBytes;
		const bool bAssetFitsInUnreferenced = UnreferencedAssetsSumBytes + Info.SizeOnMemoryInBytes <= MaxUnreferencedTransientBytes;
		const bool bDependenciesArePersistent = WorstDependencyStorage == ECacheStorageType::Persistent;
		const bool bAllDependenciesInStorage = WorstDependencyStorage != ECacheStorageType::None;
		const bool bConsumerIsReferenced = ConsumerTreeHasReferencedAsset(Info);

		const FString AssetPathStr = AssetPath.ToString();
		const FString AssetClassNameStr = Info.AssetClassName.ToString();
		UObject* Asset = AssetStorage.FindRef(Info.Hash);

		// Priority 1: Asset can be moved to persistent storage
		if (!bIsTransientAssetCache && bAssetFitsInPersistent && bDependenciesArePersistent && AssetCanBePersisted(*AssetClassNameStr))
		{
			Info.CurrentStorageType = ECacheStorageType::Persistent;
		}

		// Priority 2: Asset is kept on transient storage because it (or a Consumer) is referenced
		else if (Info.Referencers.Num() > 0 || bConsumerIsReferenced)
		{
			Info.CurrentStorageType = ECacheStorageType::Referenced;
		}

		// Priority 3: Asset fits on transient storage even while unreferenced, as long as all dependencies also
		// made it somewhere and weren't evicted
		else if (bAssetFitsInUnreferenced && bAllDependenciesInStorage)
		{
			Info.CurrentStorageType = ECacheStorageType::Unreferenced;
		}

		UE_LOGF(
			LogUsd,
			Verbose,
			"Asset '%ls' (class %ls, hash '%ls', size %.3f MB on memory, %.3f MB on disk) fits in persistent? %d, fits in unreferenced? %d, "
				 "referencer count %d, dependency count %d. Target storage: %d",
			*AssetPathStr,
			*AssetClassNameStr,
			*Info.Hash,
			Info.SizeOnMemoryInBytes / 1000000.0,
			Info.SizeOnDiskInBytes / 1000000.0,
			bAssetFitsInPersistent,
			bAssetFitsInUnreferenced,
			Info.Referencers.Num(),
			Info.Dependencies.Num(),
			int(Info.CurrentStorageType)
		);

		switch (Info.CurrentStorageType)
		{
			case ECacheStorageType::None:
			{
				Info.BulkData.UnloadBulkData();
				break;
			}
			case ECacheStorageType::Referenced:
			{
				// We'll at least unload the bulkdata here, but not actually remove it from disk yet, in case we
				// put this asset back in persistent later and can reuse the same bulkdata.
				// We'll get rid of the bulkdata for sure on UUsdAssetCache2::Serialize(), if the asset is not
				// persistent by then
				Info.BulkData.UnloadBulkData();

				ensure(Asset);
				NewAssetStorage.Add(Info.Hash, Asset);
				break;
			}
			case ECacheStorageType::Unreferenced:
			{
				Info.BulkData.UnloadBulkData();

				if (Asset)
				{
					ensure(AssetStorage.Contains(Info.Hash));
					NewAssetStorage.Add(Info.Hash, Asset);
				}
				// It's possible to end up with an asset that is currently persistent but living in
				// Unreferenced storage if we load an asset cache and discover the asset fits in the latter
				// but not in the former (e.g. if size on disk > size in memory somehow, or if the storage
				// sizes are tweaked to cause that)
				else
				{
					ensure(!AssetStorage.Contains(Info.Hash));
					NewPendingPersistentStorage.Add(Info.Hash, AssetPath);
				}

				UnreferencedAssetsSumBytes += Info.SizeOnMemoryInBytes;
				UE_LOGF(
					LogUsd,
					Verbose,
					"Updated unreferenced sum to %.3f out of max %llu MB",
					UnreferencedAssetsSumBytes / 1000000.0,
					MaxUnreferencedTransientBytes / 1000000
				);
				break;
			}
			case ECacheStorageType::Persistent:
			{
				if (Asset)
				{
					ensure(AssetStorage.Contains(Info.Hash));
					NewAssetStorage.Add(Info.Hash, Asset);
				}
				else
				{
					ensure(!AssetStorage.Contains(Info.Hash));
					NewPendingPersistentStorage.Add(Info.Hash, AssetPath);
				}

				PersistentAssetSumBytes += Info.SizeOnDiskInBytes;
				UE_LOGF(
					LogUsd,
					Verbose,
					"Updated persistent sum to %.3f out of max %llu MB",
					PersistentAssetSumBytes / 1000000.0,
					MaxPersistentBytes / 1000000
				);
				break;
			}
			default:
			{
				ensure(false);
				break;
			}
		}

		return Info.CurrentStorageType;
	};

	for (TLruCache<FSoftObjectPath, FCachedAssetInfo>::TIterator Iter{LRUCache}; Iter; ++Iter)
	{
		FSoftObjectPath& AssetPath = Iter.Key();
		FCachedAssetInfo& Info = Iter.Value();

		VisitEntryRecursively(Info, AssetPath);
	}

	Swap(AssetStorage, NewAssetStorage);
	Swap(PendingPersistentStorage, NewPendingPersistentStorage);

	// Evict from the LRUCache assets that never made it into any storage: These assets are unreferenced and don't fit
	// anymore
	{
		TLruCache<FSoftObjectPath, FCachedAssetInfo>::TIterator Iter{LRUCache};
		while (Iter)
		{
			FSoftObjectPath& AssetPath = Iter.Key();
			FCachedAssetInfo& Info = Iter.Value();

			if (Info.CurrentStorageType == ECacheStorageType::None)
			{
				UE_LOGF(
					LogUsd,
					Verbose,
					"Evicting asset '%ls' (hash '%ls') from the cache '%ls'",
					*AssetPath.ToString(),
					*Info.Hash,
					*GetPathName()
				);

				// Discard data on disk
				TryUnloadAsset(Info);
				Info.BulkData.RemoveBulkData();

				// Make sure all of our dependencies know we're no longer a Consumer
				for (const FSoftObjectPath& Dependency : Info.Dependencies)
				{
					if (FCachedAssetInfo* DepInfo = LRUCache.Find(Dependency))
					{
						DepInfo->Consumers.Remove(AssetPath);
					}
				}

				Iter.RemoveCurrentAndIncrement();
			}
			else
			{
				++Iter;
			}
		}
	}

	// Now that we evicted all potential dependencies we don't care about anymore,
	// unload all persistent assets that aren't being currently used
	if (GOnDemandCachedAssetLoading)
	{
		for (TLruCache<FSoftObjectPath, FCachedAssetInfo>::TIterator Iter{LRUCache}; Iter; ++Iter)
		{
			FCachedAssetInfo& Info = Iter.Value();
			TryUnloadAsset(Info);
		}
	}

	double ElapsedSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);

	UE_LOGF(
		LogUsd,
		Log,
		"Finished asset cache '%ls' storage refresh in %.3f seconds: %.3f out of max %llu MB of unreferenced assets, %.3f out of max %llu MB of "
			 "persistent assets",
		*GetName(),
		ElapsedSeconds,
		UnreferencedAssetsSumBytes / 1000000.0,
		MaxUnreferencedTransientBytes / 1000000,
		PersistentAssetSumBytes / 1000000.0,
		MaxPersistentBytes / 1000000
	);
}

void UUsdAssetCache2::MarkAssetsAsStale()
{
	FWriteScopeLock Lock(RWLock);

	ActiveAssets.Reset();
}

TSet<UObject*> UUsdAssetCache2::GetActiveAssets() const
{
	return ActiveAssets;
}

void UUsdAssetCache2::PostLoad()
{
	Super::PostLoad();

	// Finish loading all of our persistent assets right away if we should.
	// We do this here and not inside ::Serialize() because this involves renaming UObjects, which can't be done
	// during load
	if (!GOnDemandCachedAssetLoading)
	{
		TMap<FString, FSoftObjectPath> PendingStorageCopy = PendingPersistentStorage;
		for (const TPair<FString, FSoftObjectPath>& PendingAsset : PendingStorageCopy)
		{
			UObject* AssetFromBulkData = GetCachedAsset(PendingAsset.Key);
			ensure(AssetFromBulkData);
		}
	}
}

bool UUsdAssetCache2::TryUnloadAsset(FCachedAssetInfo& InOutInfo)
{
	if (GOnDemandCachedAssetLoading && InOutInfo.BulkData.CanLoadFromDisk() && InOutInfo.Referencers.Num() == 0)
	{
		// Asset is currently loaded
		if (TObjectPtr<UObject> Asset = AssetStorage.FindRef(InOutInfo.Hash))
		{
			// Check to see if all of its Consumers can be unloaded too
			for (auto It = InOutInfo.Consumers.CreateIterator(); It; ++It)
			{
				const FSoftObjectPath& Consumer = *It;
				if (FCachedAssetInfo* ConsumerInfo = LRUCache.Find(Consumer))
				{
					ensure(ConsumerInfo->Dependencies.Contains(Asset));

					bool bConsumerIsUnloaded = TryUnloadAsset(*ConsumerInfo);
					if (!bConsumerIsUnloaded)
					{
						// Our Consumer must stay loaded -> So must we
						return false;
					}
				}
				// We don't know about this Consumer anymore (maybe it was evicted?)
				else
				{
					It.RemoveCurrent();
				}
			}

			PendingPersistentStorage.Add(InOutInfo.Hash, Asset);
			AssetStorage.Remove(InOutInfo.Hash);

			const TCHAR* NewName = nullptr;
			UObject* NewOuter = GetTransientPackage();
			Asset->Rename(NewName, NewOuter, REN_DontCreateRedirectors | REN_NonTransactional | REN_AllowPackageLinkerMismatch);
		}

		return InOutInfo.BulkData.UnloadBulkData();
	}

	return false;
}

bool UUsdAssetCache2::IsAssetOwnedByCacheInternal(const FString& AssetPath) const
{
	return !AssetPath.IsEmpty() && LRUCache.Contains(AssetPath);
}

bool UUsdAssetCache2::AddAssetReferenceInternal(const UObject* Asset, const UObject* Referencer)
{
	if (!Asset || !Referencer)
	{
		return false;
	}

	if (FCachedAssetInfo* Info = LRUCache.Find(Asset))
	{
		Info->Referencers.Add(FObjectKey{Referencer});

		UE_LOGF(LogUsd, Verbose, "Added referencer '%ls' for asset '%ls'", *Referencer->GetPathName(), *Asset->GetPathName());
		return true;
	}

	return false;
}

void UUsdAssetCache2::TouchAssetInternal(const UObject* Asset, const UObject* Referencer)
{
	if (!Asset)
	{
		return;
	}

	ActiveAssets.Add(const_cast<UObject*>(Asset));
	ensure(LRUCache.FindAndTouch(Asset));

	if (Referencer)
	{
		ensure(AddAssetReferenceInternal(Asset, Referencer));
	}
}

void UUsdAssetCache2::Serialize(FArchive& Ar)
{
	if (Ar.IsSaving())
	{
		Ar << GCurrentCacheVersion;
	}
	else if (Ar.IsLoading())
	{
		int32 SavedCacheVersion = INDEX_NONE;
		Ar << SavedCacheVersion;
		ensure(SavedCacheVersion <= GCurrentCacheVersion);
	}

	Super::Serialize(Ar);

	if (Ar.IsPersistent() || Ar.IsTransacting() || (Ar.GetPortFlags() & (PPF_DuplicateForPIE | PPF_Duplicate)))
	{
		if (Ar.IsSaving())
		{
			// When we cook we must generate a new cooked bulkdata as it may need to filter editor-only stuff during
			// the serialization process, so make sure none of these assets are pending
			if (Ar.IsCooking())
			{
				// Take a copy here because GetCachedAsset will modify PendingPersistentStorage
				TMap<FString, FSoftObjectPath> PendingStorageCopy;
				{
					FReadScopeLock Lock(RWLock);
					PendingStorageCopy = PendingPersistentStorage;
				}
				for (const TPair<FString, FSoftObjectPath>& PendingAsset : PendingStorageCopy)
				{
					UObject* AssetFromBulkData = GetCachedAsset(PendingAsset.Key);
					ensure(AssetFromBulkData);
				}
			}

			FWriteScopeLock Lock(RWLock);

			TSet<FSoftObjectPath> PersistentAssetsWithInfo;
			if (Ar.IsPersistent())
			{
				for (TLruCache<FSoftObjectPath, FCachedAssetInfo>::TIterator Iter{LRUCache}; Iter; ++Iter)
				{
					const FCachedAssetInfo& Info = Iter.Value();
					if (Info.CurrentStorageType == ECacheStorageType::Persistent)
					{
						PersistentAssetsWithInfo.Add(Iter.Key());
					}
				}
			}

			int32 NumInfos = Ar.IsPersistent() ? PersistentAssetsWithInfo.Num() : LRUCache.Num();
			Ar << NumInfos;

			UE_LOGF(
				LogUsd,
				Verbose,
				"Serializing USD Asset Cache '%ls' with %d asset infos. Persistent? %d, transacting? %d, duplicating? %d, cooking? %d",
				*GetPathName(),
				NumInfos,
				Ar.IsPersistent(),
				Ar.IsTransacting(),
				(Ar.GetPortFlags() & (PPF_DuplicateForPIE | PPF_Duplicate)),
				Ar.IsCooking()
			);

			// By using the iterators we'll traverse the most recently used object first
			for (TLruCache<FSoftObjectPath, FCachedAssetInfo>::TIterator Iter{LRUCache}; Iter; ++Iter)
			{
				FCachedAssetInfo& Info = Iter.Value();
				FSoftObjectPath& AssetPath = Iter.Key();
				ensure(!AssetPath.IsNull());

				if (Ar.IsPersistent() && Info.CurrentStorageType != ECacheStorageType::Persistent)
				{
					UE_LOGF(
						LogUsd,
						Verbose,
						"Skipping serialization of info on asset '%ls' (%ls) as it will not be persisted",
						*Info.AssetName,
						*Info.Hash
					);
					continue;
				}

				// This is the main way in which assets get serialized into the memory bulkdata, which we only ever need to do
				// when serializing to disk (which will happen within the Info.Serialize call below)
				if (Ar.IsPersistent())
				{
					// The asset should either be loaded and in AssetStorage, or have been unloaded back into persistent storage
					UObject* Asset = AssetStorage.FindRef(Info.Hash);
					ensure(Asset || PendingPersistentStorage.Contains(Info.Hash));

					// We're only going to serialize it back to disk if it is already loaded of course. We could prevent overwriting
					// on disk an already serialized asset that happens to be loaded right now since we don't really support
					// modifying our generated assets, but I suppose this could be useful if the user just changed a couple of
					// settings on his StaticMesh or whatever and expects them to persist.
					if (Asset)
					{
						// Serialize the referenced asset into a byte buffer
						TArray<uint8> Buffer;
						UE::AssetCache::Private::SerializeObjectAndSubObjects(Asset, Buffer, Ar);

						// Copy the byte buffer into the Info's bulkdata in memory
						// TODO: Maybe it's possible to avoid this copy by having a custom object writer
						// that can write directly to the realloc'd bulkdata bytes
						Info.BulkData.Lock(LOCK_READ_WRITE);
						{
							void* BulkDataBytes = Info.BulkData.Realloc(Buffer.Num());
							FMemory::Memcpy(BulkDataBytes, Buffer.GetData(), Buffer.Num());
						}
						Info.BulkData.Unlock();

						// Now we know *exactly* how big the asset is on disk
						Info.SizeOnDiskInBytes = Buffer.Num();
						ensure(Info.SizeOnDiskInBytes > 0);

						UE_LOGF(
							LogUsd,
							Verbose,
							"Serializing asset '%ls' (%ls) to %lld bulk data bytes",
							*Info.AssetName,
							*Info.Hash,
							Info.SizeOnDiskInBytes
						);
					}
				}

				Ar << AssetPath;

				// When persisting, we only want to serialize dependencies on persistent assets, or else when we deserialize this
				// we'll have a bunch of dependencies on assets that don't exist
				TSet<FSoftObjectPath> PersistentDependencies;
				TSet<FSoftObjectPath> PersistentConsumers;
				if (Ar.IsPersistent())
				{
					PersistentDependencies = Info.Dependencies.Intersect(PersistentAssetsWithInfo);
					PersistentConsumers = Info.Consumers.Intersect(PersistentAssetsWithInfo);
					Swap(Info.Dependencies, PersistentDependencies);
					Swap(Info.Consumers, PersistentConsumers);
				}

				Info.Serialize(Ar, this);

				if (Ar.IsPersistent())
				{
					Swap(Info.Dependencies, PersistentDependencies);
					Swap(Info.Consumers, PersistentConsumers);
				}
			}
		}
		else if (Ar.IsLoading())
		{
			FWriteScopeLock Lock(RWLock);

			int32 NumInfos = INDEX_NONE;
			Ar << NumInfos;
			ensure(NumInfos >= 0);

			TArray<FCachedAssetInfo> MostToLeastRecent;
			MostToLeastRecent.Reserve(NumInfos);

			TArray<FSoftObjectPath> AssetPaths;
			AssetPaths.Reserve(NumInfos);

			UE_LOGF(LogUsd, Verbose, "Deserializing USD Asset Cache '%ls' with %d asset infos", *GetPathName(), NumInfos);

			for (int32 Index = 0; Index < NumInfos; ++Index)
			{
				FSoftObjectPath& AssetPath = AssetPaths.Emplace_GetRef();
				Ar << AssetPath;

				FCachedAssetInfo& NewEl = MostToLeastRecent.Emplace_GetRef();
				NewEl.Serialize(Ar, this);

				PendingPersistentStorage.Add(NewEl.Hash, AssetPath);
			}

			// We never want the cache to drop items by itself
			const int32 MaxElements = -1;
			LRUCache.Empty(MaxElements);

			// Push them back-to-front into the LRU cache to make sure the most recent one is added last
			for (int32 Index = NumInfos - 1; Index >= 0; --Index)
			{
				LRUCache.Add(MoveTemp(AssetPaths[Index]), MoveTemp(MostToLeastRecent[Index]));
			}
		}
	}
}

FUsdScopedAssetCacheReferencer::FUsdScopedAssetCacheReferencer(UUsdAssetCache2* InAssetCache, const UObject* Referencer)
{
	// For now we're assuming you can't nest these objects
	if (ensure(InAssetCache))
	{
		ensure(Referencer && (!InAssetCache->CurrentScopedReferencer || InAssetCache->CurrentScopedReferencer == Referencer));
		InAssetCache->CurrentScopedReferencer = Referencer;
	}

	AssetCache = InAssetCache;
}

FUsdScopedAssetCacheReferencer::~FUsdScopedAssetCacheReferencer()
{
	UUsdAssetCache2* ValidCache = AssetCache.Get();
	if (ensure(ValidCache))
	{
		ValidCache->CurrentScopedReferencer = nullptr;
	}
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS
