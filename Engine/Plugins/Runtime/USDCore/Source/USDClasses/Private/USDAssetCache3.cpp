// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDAssetCache3.h"

#include "USDAssetUserData.h"
#include "USDLog.h"
#include "USDObjectUtils.h"

#include "Animation/AnimData/IAnimationDataController.h"
#include "Animation/AnimData/IAnimationDataModel.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Async/Async.h"
#include "ComponentRecreateRenderStateContext.h"
#include "Containers/Ticker.h"
#include "Engine/Blueprint.h"
#include "GeometryCache.h"
#include "GeometryCacheTrack.h"
#include "GroomAsset.h"
#include "HAL/FileManager.h"
#include "Interfaces/Interface_AssetUserData.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "UObject/Package.h"
#include "UObject/ReferencerFinder.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectIterator.h"

#if WITH_EDITOR
#include "AssetToolsModule.h"
#include "Editor.h"
#include "Editor/Transactor.h"
#include "ObjectTools.h"
#include "PackageTools.h"
#include "Subsystems/ImportSubsystem.h"
#endif	  // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(USDAssetCache3)

#define LOCTEXT_NAMESPACE "USDAssetCache3"

namespace UE::USDAssetCache3::Private
{
	// Note that this is not thread-safe, so it should only be called after acquiring a write lock
	UObject* SilentTryLoad(const FSoftObjectPath& Path)
	{
		if (Path.IsValid())
		{
			// Check if the package exists on disk first to try and avoid some ugly warnings if we try calling TryLoad with a broken path
			const FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
			FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(Path);
			if (!AssetData.IsValid())
			{
				return nullptr;
			}

			// We can't load objects from disk from an async thread
			if (!AssetData.IsAssetLoaded())
			{
				check(IsInGameThread());
			}
			return Path.TryLoad();
		}

		return nullptr;
	}

	bool HasObjectBeenSavedToDisk(UObject* Object)
	{
		if (!Object)
		{
			return false;
		}

		UPackage* Outermost = Object->GetOutermost();
		if (Outermost && Outermost->GetFileSize() > 0)
		{
			return true;
		}

		return false;
	}

	// Deletes the assets, but only if they're unreferenced by *other* external objects.
	// This will ignore references between the provided assets (e.g. a Skeleton and AnimSequence that are just referenced by each other
	// will be both deleted).
	// This should only be called with memory-only objects (i.e. don't call this with assets that have been previously saved to disk).
	// This aims to be more safe and fast than comprehensive: We'll run GC at the end which can wipe some references on its own, so even if
	// we leave some objects behind there's always the chance that closing the next stage may clear them anyway
	void SafeDeleteObjects(const TSet<UObject*>& ObjectsToDelete, const UUsdAssetCache3& AssetCache)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UUsdAssetCache3::SafeDeleteObjects);

		if (ObjectsToDelete.Num() == 0)
		{
			return;
		}

		FScopedSlowTask OuterProgress(100.0f, LOCTEXT("CleaningUpAssets", "Cleaning up assets"));
		const float ThresholdTime = 1.0f;
		OuterProgress.MakeDialogDelayed(ThresholdTime);

		double StartTime = FPlatformTime::Cycles64();

		TSet<UObject*> AllObjectsToDelete = ObjectsToDelete;

		// Expand ObjectsToDelete with all the subobjects to each object as well. We can't delete an outer
		// if the inner is referenced by an external object.
		// This is in part replicating some of what GatherObjectReferencersForDeletion does, but it's nice
		// for us to track all the UObjects ourselves for our ExternalReferencers step below, and for runtime parity
		{
			TArray<UObject*> ExtraObjects;
			ExtraObjects.Reserve(ObjectsToDelete.Num());
			for (UObject* Object : ObjectsToDelete)
			{
				// This only ever appends to the InnerObjects array, so we can pile them up
				GetObjectsWithOuter(Object, ExtraObjects, EGetObjectsFlags::IncludeNestedObjects);

				UBlueprint* Blueprint = Cast<UBlueprint>(Object);
				if (Blueprint && Blueprint->GeneratedClass)
				{
					ExtraObjects.Add(Blueprint->GeneratedClass);
				}
			}
			AllObjectsToDelete.Append(ExtraObjects);
		}

		TSet<UObject*> ExternalReferencers;

		// Prepare a map tracking any UObject that was referencing any of the objects we'll delete.
		// Note that we'll be skipping internal references here: We're trying to find out blockers that would
		// prevent us from deleting something, and internal references never do that
		TMap<UObject*, TSet<UObject*>> ReferencerToReferenced;
		{
			OuterProgress.EnterProgressFrame(95.0f);
			FScopedSlowTask Progress(AllObjectsToDelete.Num(), LOCTEXT("FindingReferencesOuter", "Finding references"));

#if WITH_EDITOR
			if (GEditor && GEditor->Trans)
			{
				// This makes it so that FReferencerFinder::GetAllReferencers doesn't find any references from the transaction buffer.
				// We're going to clear the transaction buffer before we actually delete anyway, so its references don't matter
				GEditor->Trans->DisableObjectSerialization();
			}
#endif	  // WITH_EDITOR

			for (UObject* ObjectToDelete : AllObjectsToDelete)
			{
				Progress.EnterProgressFrame(
					1,
					FText::Format(LOCTEXT("FindingReferences", "Finding references to {0}"), FText::FromString(ObjectToDelete->GetPathName()))
				);

				TSet<UObject*> ObjectOuterChain;
				{
					UObject* Outer = ObjectToDelete->GetOuter();

					if (Outer)
					{
						// The inner should count as a referencer to the outer, because if there are any external referencers to this inner
						// then we cannot delete the outer
						ReferencerToReferenced.FindOrAdd(ObjectToDelete).Add(Outer);
					}

					while (Outer && !Outer->IsA<UPackage>())
					{
						ObjectOuterChain.Add(Outer);
						Outer = Outer->GetOuter();
					}
				}

				// This is a bit less comprehensive than ObjectTools::GatherObjectReferencersForDeletion but hopefully is good enough to cover
				// our use cases. We can skip some stuff from GatherObjectReferencersForDeletion like tracking inner referencers or how
				// having *any* external reference here is enough to keep objects alive: We don't have to call FindObjectsRoots, which
				// is expensive and the majority of the time spent in GatherObjectReferencersForDeletion.
				// TODO: This could potentially be made faster, as FAssetDeleteModel seemingly achieves the same task significantly faster
				TArray<UObject*> ReferencedObject{ObjectToDelete};
				TSet<UObject*>* ObjectsToIgnore = nullptr;
				EReferencerFinderFlags Flags = EReferencerFinderFlags::SkipWeakReferences | EReferencerFinderFlags::SkipInnerReferences;
				TArray<UObject*> Referencers = FReferencerFinder::GetAllReferencers(ReferencedObject, ObjectsToIgnore, Flags);

				for (UObject* Referencer : Referencers)
				{
					bool bOuterWillBeGCd = false;
					bool bReferencerIsOuter = false;

					// It doesn't matter if an object is referenced by one of its outers, because if we ever delete anything, it
					// will be the outermost directly (the package), so this "referencer" and ObjectToDelete will both be deleted anyway
					if (ObjectOuterChain.Contains(Referencer))
					{
						bReferencerIsOuter = true;
					}
					else
					{
						// Check to see if the referencer is pending kill or will be GC'd anyway: We don't care about those, since
						// we'll run GC after deleting anyway
						UObject* ReferencerOuter = Referencer;
						while (ReferencerOuter && !ReferencerOuter->IsA<UPackage>())
						{
							if (!IsValid(ReferencerOuter))
							{
								bOuterWillBeGCd = true;
							}
							ReferencerOuter = ReferencerOuter->GetOuter();
						}
					}

					if (!bOuterWillBeGCd && !bReferencerIsOuter)
					{
						ReferencerToReferenced.FindOrAdd(Referencer).Add(ObjectToDelete);

						// This referencer is very important: It's a valid external referencer pointing at one of our objects to delete
						// (or one of its subobjects), and will mean we can't delete anything it is referencing
						if (!AllObjectsToDelete.Contains(Referencer) || Referencer->IsRooted() || HasObjectBeenSavedToDisk(Referencer))
						{
							// Manually ignore UAnimSequencerControllers: They are created by UAnimationSequencerDataModel::GetController
							// and exclusively used by UAnimationSequencerDataModel, both classes being private. UAnimSequencerController
							// really seems to be an internal class of the AnimSequence and so we shouldn't consider it an external referencer,
							// but our current filters don't work for it because for whatever reason it is placed within the transient package
							// and doesn't have any flags
							if (Cast<IAnimationDataModel>(ObjectToDelete) && Cast<IAnimationDataController>(Referencer))
							{
								continue;
							}

							// Manually ignore the references from UGeometryCacheTrack to UGeometryCaches. The tracks are owned by the caches
							// themselves, they just happen to have the transient package as their outer instead of the UGeometryCache asset,
							// so our mechanism here considers them external referencers
							if (Cast<UGeometryCache>(ObjectToDelete) && Cast<UGeometryCacheTrack>(Referencer))
							{
								continue;
							}

							// GeometryCaches can be fully owned by the asset cache now, so they would count as referencers here.
							// Of course, we don't care about those references either
							if (Referencer == &AssetCache)
							{
								continue;
							}

							ExternalReferencers.Add(Referencer);
						}
					}
				}
			}

#if WITH_EDITOR
			if (GEditor && GEditor->Trans)
			{
				GEditor->Trans->EnableObjectSerialization();
			}
#endif	  // WITH_EDITOR
		}

		// Here we'll collect everything that we cannot delete due to an external referencer, which can be tricky
		// (e.g. There shouldn't originally be any problem if AssetC references AssetD, and AssetD references AssetC: We'll delete
		// both anyway... Except that if we have an ExternalAsset referencing AssetC, then we can't delete either anymore).
		//
		// To solve this we'll start from our known external referencers: Anything referenced by them we know
		// we cannot delete. Then we'll push those referenced assets into the stack and also mark the assets that they in
		// turn are referencing, and so on until we visited the entire "undeletable tree" and know everything we can't delete
		TSet<UObject*> Undeletable;
		{
			Undeletable.Reserve(AllObjectsToDelete.Num());

			TArray<UObject*> Stack = ExternalReferencers.Array();
			Stack.Reserve(AllObjectsToDelete.Num());

			while (Stack.Num() > 0)
			{
				UObject* Referencer = Stack.Pop();
				if (TSet<UObject*>* ReferencedObjects = ReferencerToReferenced.Find(Referencer))
				{
					for (UObject* ReferencedObject : *ReferencedObjects)
					{
						if (Undeletable.Contains(ReferencedObject))
						{
							// Already visited, don't push it into the stack again
							continue;
						}
						UE_LOGF(
							LogUsd,
							Verbose,
							"Not trying to clean up '%ls' because it is referenced by '%ls'",
							*ReferencedObject->GetPathName(),
							*Referencer->GetPathName()
						);
						Undeletable.Add(ReferencedObject);
						Stack.Push(ReferencedObject);
					}
				}
			}
		}

		TSet<UObject*> DeletableAssets = ObjectsToDelete.Difference(Undeletable);
		DeletableAssets.Remove(nullptr);
		TArray<UObject*> DeletableAssetsArray = DeletableAssets.Array();

#if WITH_EDITOR
		// Prepare for actual deletion
		// Reference: ObjectTools::DeleteObjects
		{
			FCanDeleteAssetResult CanDeleteResult;
			FEditorDelegates::OnAssetsCanDelete.Broadcast(DeletableAssetsArray, CanDeleteResult);
			if (!CanDeleteResult.Get())
			{
				UE_LOGF(
					LogUsd,
					Warning,
					"Cancelling the deletion of '%d' assets as the deletion operation was blocked by an engine event",
					DeletableAssetsArray.Num()
				);
				return;
			}

			GEditor->ClearPreviewComponents();

			FOnPreDestructiveAssetActionArgs PreDestructiveArgs(DeletableAssetsArray, EDestructiveAssetActions::AssetDelete);
			FEditorDelegates::OnPreDestructiveAssetActionWithArgs.Broadcast(PreDestructiveArgs);
			if (!PreDestructiveArgs.Result.bSuccess)
			{
				UE_LOGF(LogUsd, Warning, "Cancelling the deletion of '%d' assets as the deletion operation was blocked by an editor event", DeletableAssetsArray.Num());
				UE_LOGF(LogUsd, Warning, "%ls", *PreDestructiveArgs.Result.ErrorMessage);
				return;
			}

			FEditorDelegates::OnAssetsPreDelete.Broadcast(DeletableAssetsArray);
		}
#endif	  // WITH_EDITOR

		// Finally actually delete the assets that we can
		{
#if WITH_EDITOR
			OuterProgress.EnterProgressFrame(2.5f);
			FScopedSlowTask Progress(DeletableAssets.Num(), LOCTEXT("DeletingAssets", "Deleting assets"));

			if (DeletableAssets.Num() > 0)
			{
				// Preemptively clear undo/redo buffer (it seems the norm to clear it when deleting assets)
				GEditor->ResetTransaction(LOCTEXT("ResetBeforeDelete", "Reset before cleaning up unreferenced assets"));
			}

			for (UObject* DeletableAsset : DeletableAssets)
			{
				UE_LOGF(LogUsd, Verbose, "Deleting '%ls'", *DeletableAsset->GetPathName());

				Progress.EnterProgressFrame();

				// Call ObjectTools here as that is the correct/complete thing to do.
				// We don't need to perform the reference check here though (and it shouldn't show any warnings)
				// because we already did a referencer check ourselves
				const bool bPerformReferenceCheck = false;
				const bool bDeleted = ObjectTools::DeleteSingleObject(DeletableAsset, bPerformReferenceCheck);
				if (bDeleted)
				{
					// This is good as it allows weak pointers to instantly start failing
					DeletableAsset->MarkAsGarbage();
				}
				else
				{
					UE_LOGF(LogUsd, Warning, "Failed to delete asset '%ls'", *DeletableAsset->GetPathName());
				}
			}
#else
			for (UObject* DeletableAsset : DeletableAssets)
			{
				UE_LOGF(LogUsd, Verbose, "Deleting '%ls'", *DeletableAsset->GetPathName());

				// These are essentially the internals of ObjectTools::DeleteSingleObject that actually do the deletion
				DeletableAsset->MarkPackageDirty();
				FAssetRegistryModule::AssetDeleted(DeletableAsset);
				DeletableAsset->ClearFlags(RF_Standalone | RF_Public);
				DeletableAsset->MarkAsGarbage();
			}
#endif
		}

		// Run GC if we deleted anything
		if (DeletableAssets.Num() > 0)
		{
			OuterProgress.EnterProgressFrame(2.5f);
			FScopedSlowTask Progress(1, LOCTEXT("GC", "Collecting garbage"));

			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

			double ElapsedSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);
			UE_LOGF(
				LogUsd,
				Verbose,
				"Deleted %d out of %d assets in %.3f s (including GC and transaction reset)",
				DeletableAssets.Num(),
				ObjectsToDelete.Num(),
				ElapsedSeconds
			);
		}
	}
}	 // namespace UE::USDAssetCache3::Private

UObject* UUsdAssetCache3::GetOrCreateCachedAsset(
	const FString& Hash,
	UClass* Class,
	const FString& DesiredName,
	int32 DesiredFlags,
	bool& bOutCreatedAsset,
	const UObject* Referencer
)
{
	return GetOrCreateCustomCachedAsset(
		Hash,
		Class,
		DesiredName,
		static_cast<EObjectFlags>(DesiredFlags),
		[Class](UPackage* PackageOuter, FName SanitizedName, EObjectFlags FlagsToUse) -> UObject*
		{
			return NewObject<UObject>(PackageOuter, Class, SanitizedName, FlagsToUse);
		},
		&bOutCreatedAsset,
		Referencer
	);
}

UObject* UUsdAssetCache3::GetOrCreateCustomCachedAsset(
	const FString& Hash,
	UClass* Class,
	const FString& DesiredName,
	EObjectFlags DesiredFlags,
	TFunctionRef<UObject*(UPackage* PackageOuter, FName SanitizedName, EObjectFlags FlagsToUse)> ObjectCreationFunc,
	bool* bOutCreatedAsset,
	const UObject* Referencer
)
{
	if (!ensure(Class))
	{
		return nullptr;
	}

	check(IsInGameThread());

	Modify();

	// We have a single scope lock here in order to avoid a race condition where two threads simultaneously calling this
	// function would both fail to find an existing asset and end up creating identical assets, causing trouble downstream.
	//
	// For simplicity we'll also do a bit of copy pasting of the implementation of GetCachedAssetPath and TouchAsset too,
	// otherwise we'd need some internal intermediate functions in order to avoid deadlocking with this lock right here
	FWriteScopeLock Lock(RWLock);

	if (bOutCreatedAsset)
	{
		*bOutCreatedAsset = false;
	}

	// Check for an existing asset
	FSoftObjectPath CachedPath = HashToAssetPaths.FindRef(Hash);
	if (bOnlyHandleAssetsWithinAssetDirectory)
	{
		const FString NewPathString = CachedPath.GetAssetPathString();
		if (!NewPathString.StartsWith(AssetDirectory.Path))
		{
			CachedPath.Reset();
			StopTrackingAssetInternal(Hash);
		}
	}
	UObject* Asset = UE::USDAssetCache3::Private::SilentTryLoad(CachedPath);
	if (Asset)
	{
		if (!Asset->IsA(Class))
		{
			UE_LOGF(
				LogUsd,
				Warning,
				"Asset cache '%ls' stopped tracking asset '%ls' for hash '%ls' as its class ('%ls') differs from the requested class '%ls'. A new asset of the requested class will be instantiated for that hash.",
				*GetPathName(),
				*Asset->GetPathName(),
				*Hash,
				*Asset->GetClass()->GetPathName(),
				*Class->GetPathName()
			);

			StopTrackingAssetInternal(Hash);
			Asset = nullptr;
		}
		else
		{
			TouchAssetInternal(CachedPath);
			return Asset;
		}
	}

	ForceValidAssetDirectoryInternal();

	FString PrefixedDesiredName = UsdUnreal::ObjectUtils::GetPrefixedAssetName(DesiredName, Class);

	FName UniqueAssetName;
	UPackage* Package = nullptr;

	const bool bHadTransientFlag = static_cast<bool>(EObjectFlags(DesiredFlags) & EObjectFlags::RF_Transient);
	EObjectFlags FlagsToUse = DesiredFlags | EObjectFlags::RF_Public | EObjectFlags::RF_Standalone;

	const bool bIsTransientCache = IsTransientCache();
	bool bIsTransientAsset = false;

	{
		// We never want to create new assets into the transaction buffer, as we don't want them to disappear when we undo
		TGuardValue<ITransaction*> SuppressTransaction{GUndo, nullptr};

		// If we're in the transient package (or have been told to create a transient asset, or are at runtime),
		// we want to place assets also in the transient package
		if (bIsTransientCache || bHadTransientFlag || !GIsEditor)
		{
			UniqueAssetName = MakeUniqueObjectName(GetTransientPackage(), Class, *UsdUnreal::ObjectUtils::SanitizeObjectName(PrefixedDesiredName));
			Package = GetTransientPackage();
			FlagsToUse |= EObjectFlags::RF_Transient;
		}
#if WITH_EDITOR
		// If we're a regular asset cache on the content browser in the editor, we want to place our assets in individual
		// standalone packages inside AssetDirectory.Path
		else
		{
			// Create unique names for the package and asset inside of it (CreateUniqueAssetName also internally sanitizes them)
			FString DesiredPath = FPaths::Combine(AssetDirectory.Path, PrefixedDesiredName);
			FString Suffix = TEXT("");
			FString UniquePackageName;
			FString UniqueAssetNameStr;
			const FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));
			AssetToolsModule.Get().CreateUniqueAssetName(DesiredPath, Suffix, UniquePackageName, UniqueAssetNameStr);
			if (UniquePackageName.EndsWith(UniqueAssetNameStr))
			{
				UniquePackageName = UniquePackageName.LeftChop(UniqueAssetNameStr.Len() + 1);
			}

			const FString PackageName = UPackageTools::SanitizePackageName(UniquePackageName + TEXT("/") + UniqueAssetNameStr);

			UniqueAssetName = FName{*UniqueAssetNameStr};
			Package = CreatePackage(*PackageName);
			FlagsToUse &= ~EObjectFlags::RF_Transient;
		}
#endif	  // WITH_EDITOR

		Package->FullyLoad();

		// Actually create the asset itself
		Asset = ObjectCreationFunc(Package, UniqueAssetName, FlagsToUse);
		if (!Asset)
		{
			return nullptr;
		}

		// Broadcast events
		bIsTransientAsset = Asset->GetOutermost() == GetTransientPackage();
		if (!bIsTransientAsset)
		{
			// It seems the Content Browser has trouble displaying the assets if their FName is different from their package FName.
			// We're providing the correct SanitizedName and an Outer to ObjectCreationFunc, but it's possible that the function itself
			// didn't follow this rule, so here we'll show a warning in that case
			const FString& AssetName = Asset->GetName();
			const FString& PackageName = FPackageName::GetShortName(Asset->GetOutermost());
			if (AssetName != PackageName)
			{
				UE_LOGF(
					LogUsd,
					Warning,
					"Asset '%ls' has a different name than it's package short name '%ls' (full name '%ls'), which could cause issues",
					*AssetName,
					*PackageName,
					*Asset->GetOutermost()->GetPathName()
				);
			}

			// We definitely want to mark the package as dirty, but we can't do that in the context of loading packages (e.g.
			// while loading into a level with a loaded stage actor), so delay it to the game thread.
			// Note that we're already in the game thread here, we just want to get the MarkPackageDirty call to happen
			// outside of the callstack of package loading. The async task is technically better for us than waiting for the next
			// tick via FTSTicker because UUsdAssetCache3::RequestDelayedAssetAutoCleanup can potentially (but unlikely) trigger
			// a cleanup on that next tick, however
			TWeakObjectPtr<UObject> WeakAsset = Asset;
			AsyncTask(
				ENamedThreads::GameThread,
				[WeakAsset]()
				{
					if (UObject* Asset = WeakAsset.Get())
					{
						Asset->MarkPackageDirty();
					}
				}
			);

#if WITH_EDITOR
			UFactory* Factory = nullptr;
			GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(Factory, Asset);
#endif	  // WITH_EDITOR
		}

		FAssetRegistryModule::AssetCreated(Asset);

		// Setup AssetUserData so we can immediately record its original hash
		TSubclassOf<UUsdAssetUserData> AssetUserDataClass = UsdUnreal::ObjectUtils::GetAssetUserDataClassForObject(Class);
		if (AssetUserDataClass)
		{
			if (UUsdAssetUserData* UserData = UsdUnreal::ObjectUtils::GetOrCreateAssetUserData(Asset, AssetUserDataClass))
			{
				UserData->OriginalHash = Hash;
			}
		}
	}

	// Our reverse map is just one to one, so we can't allow associating two hashes to the same asset
	if (FString* OldHash = AssetPathToHashes.Find(Asset))
	{
		// There are some scenarios in which we have to recache the same asset to the same hash (UE-214909), and that should
		// be fine. We only care if we somehow have different hashes for the same asset
		if (*OldHash != Hash)
		{
			UE_LOGF(
				LogUsd,
				Warning,
				"An asset can only be associated with a single hash! Discarding old hash '%ls' mapped to recently cached asset '%ls' (new hash '%ls')",
				**OldHash,
				*Asset->GetPathName(),
				*Hash
			);
			StopTrackingAssetInternal(*OldHash);
		}
	}

	// We don't want to inherit any old referencers in case we happened to have some old data in the referencer
	// maps when creating this new asset
	RemoveAllAssetReferencersInternal(Hash);

	// Cache asset
	HashToAssetPaths.Add(Hash, Asset);
	AssetPathToHashes.Add(Asset, Hash);
	TouchAssetInternal(Asset, Referencer);
	if (bIsTransientAsset)
	{
		TransientObjectStorage.Add(Hash, Asset);
	}
	else
	{
		TransientObjectStorage.Remove(Hash);
	}
	DeletableAssetKeys.Add(FObjectKey{Asset});

	if (bOutCreatedAsset)
	{
		*bOutCreatedAsset = true;
	}
	return Asset;
}

void UUsdAssetCache3::CacheAsset(const FString& Hash, const FSoftObjectPath& AssetPath, const UObject* Referencer)
{
	if (!AssetPath.IsValid() || Hash.IsEmpty())
	{
		return;
	}

	Modify();

	FWriteScopeLock Lock(RWLock);

	// We don't want to inherit any old referencers in case we are overwriting a hash entry with a new asset
	FSoftObjectPath* OldCachedPath = HashToAssetPaths.Find(Hash);
	if (OldCachedPath && *OldCachedPath != AssetPath)
	{
		RemoveAllAssetReferencersInternal(Hash);
	}

	// Setup AssetUserData if the asset doesn't have any, so that we can set its hash.
	// This is important for MaterialX materials for example: They're produced all in one go when the interchange translator
	// handles the mtlx files, and we must be able to add these hashes
	if (UObject* LoadedObject = AssetPath.TryLoad())
	{
		TSubclassOf<UUsdAssetUserData> AssetUserDataClass = UsdUnreal::ObjectUtils::GetAssetUserDataClassForObject(LoadedObject->GetClass());
		if (AssetUserDataClass)
		{
			if (UUsdAssetUserData* UserData = UsdUnreal::ObjectUtils::GetOrCreateAssetUserData(LoadedObject, AssetUserDataClass))
			{
				UserData->OriginalHash = Hash;
			}
		}
	}

	HashToAssetPaths.Add(Hash, AssetPath);
	AssetPathToHashes.Add(AssetPath, Hash);
	TouchAssetInternal(AssetPath, Referencer);

	const bool bIsTransientAsset = AssetPath.ToString().StartsWith(GetTransientPackage()->GetPathName());
	if (bIsTransientAsset)
	{
		// If the asset is in the transient package then it must already be loaded, so this shouldn't
		// actually cause any loading
		UObject* LoadedObject = FindObjectFast<UObject>(GetTransientPackage(), *AssetPath.GetAssetName());
		if (ensure(LoadedObject))
		{
			TransientObjectStorage.Add(Hash, LoadedObject);
		}
	}
	else
	{
		TransientObjectStorage.Remove(Hash);
	}
}

FSoftObjectPath UUsdAssetCache3::StopTrackingAsset(const FString& Hash)
{
	Modify();

	FWriteScopeLock Lock(RWLock);

	return StopTrackingAssetInternal(Hash);
}

UObject* UUsdAssetCache3::GetCachedAsset(const FString& Hash) const
{
	check(IsInGameThread());

	// We lock for writing here instead of calling GetCachedAssetPath because the FAssetRegistryModule is not
	// really thread safe, so we need to protect against calling SilentTryLoad concurrently from multiple threads
	FWriteScopeLock Lock(RWLock);

	FSoftObjectPath CachedPath = HashToAssetPaths.FindRef(Hash);
	if (CachedPath.IsValid())
	{
		ActiveAssets.Add(CachedPath);
	}

	if (UObject* LoadedObject = UE::USDAssetCache3::Private::SilentTryLoad(CachedPath))
	{
		return LoadedObject;
	}

	return nullptr;
}

FSoftObjectPath UUsdAssetCache3::GetCachedAssetPath(const FString& Hash) const
{
	FWriteScopeLock Lock(RWLock);

	FSoftObjectPath CachedPath = HashToAssetPaths.FindRef(Hash);
	if (CachedPath.IsValid())
	{
		ActiveAssets.Add(Hash);
	}

	return CachedPath;
}

FString UUsdAssetCache3::GetHashForAsset(const FSoftObjectPath& AssetPath) const
{
	FWriteScopeLock Lock(RWLock);

	if (AssetPath.IsValid())
	{
		ActiveAssets.Add(AssetPath);
	}

	return AssetPathToHashes.FindRef(AssetPath);
}

bool UUsdAssetCache3::IsAssetTrackedByCache(const FSoftObjectPath& AssetPath) const
{
	FReadScopeLock Lock(RWLock);

	return AssetPathToHashes.Contains(AssetPath);
}

int32 UUsdAssetCache3::GetNumAssets() const
{
	FReadScopeLock Lock(RWLock);

	return HashToAssetPaths.Num();
}

TMap<FString, FSoftObjectPath> UUsdAssetCache3::GetAllTrackedAssets() const
{
	FReadScopeLock Lock(RWLock);
	return HashToAssetPaths;
}

TMap<FString, UObject*> UUsdAssetCache3::LoadAndGetAllTrackedAssets() const
{
	FReadScopeLock Lock(RWLock);

	TMap<FString, UObject*> Result;
	Result.Reserve(HashToAssetPaths.Num());

	for (const TPair<FString, FSoftObjectPath>& HashToAssetPath : HashToAssetPaths)
	{
		Result.Add(HashToAssetPath.Key, HashToAssetPath.Value.TryLoad());
	}

	return Result;
}

bool UUsdAssetCache3::AddAssetReferencer(const UObject* Asset, const UObject* Referencer)
{
	if (!Asset || !Referencer)
	{
		return false;
	}

	Modify();

	FWriteScopeLock Lock(RWLock);

	FString* Hash = AssetPathToHashes.Find(Asset);
	if (!Hash || Hash->IsEmpty())
	{
		return false;
	}

	AddReferenceInternal(*Hash, Referencer);
	return true;
}

bool UUsdAssetCache3::RemoveAssetReferencer(const UObject* Asset, const UObject* Referencer)
{
	if (!Asset || !Referencer)
	{
		return false;
	}

	Modify();

	FWriteScopeLock Lock(RWLock);

	FString* Hash = AssetPathToHashes.Find(Asset);
	if (!Hash || Hash->IsEmpty())
	{
		return false;
	}

	FObjectKey ReferencerKey{Referencer};

	bool bRemovedSomething = false;
	if (TArray<FString>* FoundReferencerAssets = ReferencerToHash.Find(ReferencerKey))
	{
		FoundReferencerAssets->Remove(*Hash);
		bRemovedSomething = true;
	}

	if (TArray<FObjectKey>* FoundAssetReferencers = HashToReferencer.Find(*Hash))
	{
		FoundAssetReferencers->Remove(ReferencerKey);
		bRemovedSomething = true;
	}

	return bRemovedSomething;
}

bool UUsdAssetCache3::RemoveAllReferencersForAsset(const UObject* Asset)
{
	Modify();

	FWriteScopeLock Lock(RWLock);

	if (!Asset)
	{
		return false;
	}

	FString* Hash = AssetPathToHashes.Find(Asset);
	if (!Hash || Hash->IsEmpty())
	{
		return false;
	}

	return RemoveAllAssetReferencersInternal(*Hash);
}

bool UUsdAssetCache3::RemoveAllReferencerAssets(const UObject* Referencer)
{
	if (!Referencer)
	{
		return false;
	}

	Modify();

	FWriteScopeLock Lock(RWLock);

	FObjectKey ReferencerKey{Referencer};

	TArray<FString> ReferencerAssets;
	bool bRemovedSomething = ReferencerToHash.RemoveAndCopyValue(ReferencerKey, ReferencerAssets);

	for (const FString& AssetHash : ReferencerAssets)
	{
		if (TArray<FObjectKey>* FoundAssetReferencers = HashToReferencer.Find(AssetHash))
		{
			FoundAssetReferencers->Remove(ReferencerKey);
		}
	}

	return bRemovedSomething;
}

bool UUsdAssetCache3::RemoveAllAssetReferencers()
{
	Modify();

	FWriteScopeLock Lock(RWLock);

	bool bHadSomething = ReferencerToHash.Num() > 0 && HashToReferencer.Num() > 0;

	ReferencerToHash.Reset();
	HashToReferencer.Reset();

	return bHadSomething;
}

void UUsdAssetCache3::SetAssetDeletable(const UObject* Asset, bool bIsDeletable)
{
	Modify();

	FWriteScopeLock Lock(RWLock);

	FObjectKey Key{Asset};
	if (bIsDeletable)
	{
		DeletableAssetKeys.Add(Key);
	}
	else
	{
		DeletableAssetKeys.Remove(Key);
	}
}

bool UUsdAssetCache3::IsAssetDeletable(const UObject* Asset) const
{
	FReadScopeLock Lock(RWLock);

	FObjectKey Key{Asset};

	return DeletableAssetKeys.Contains(Key);
}

void UUsdAssetCache3::DeleteUnreferencedAssets(bool bShowConfirmation)
{
	Modify();

	TMap<FString, TWeakObjectPtr<UObject>> AbandonedHashesToObjects;
	TMap<FString, FObjectKey> AbandonedHashesToObjectKeys;
	TSet<UObject*> ObjectsToDelete;
	TArray<FAssetData> AssetsToDelete;

#if !WITH_EDITOR
	bShowConfirmation = false;
#endif	  // !WITH_EDITOR

	{
		// Write lock here because the asset registry is not thread-safe
		FWriteScopeLock Lock(RWLock);

		const FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

		for (const TPair<FString, FSoftObjectPath>& Pair : HashToAssetPaths)
		{
			const FString& Hash = Pair.Key;
			const FSoftObjectPath& Path = Pair.Value;
			const FString AssetPathString = Path.ToString();

			// This should convert from '/Game/UsdAssets/mesh.mesh' to '/Game/UsdAssets/mesh'. Possibly overkill but
			// likely best than manually searching for the dot ourselves
			FString ClassName;
			FString PackageName;
			FString ObjectName;
			FString SubObjectName;
			const bool bDetectClassName = false;
			FPackageName::SplitFullObjectPath(AssetPathString, ClassName, PackageName, ObjectName, SubObjectName, bDetectClassName);

			// Skip assets that have been saved to disk before (without actually loading them to check).
			// Note that no package means it's an invalid path or something we only know about because it
			// was saved in the past. In both cases we want to ignore it here.
			UObject* Outer = nullptr;
			UPackage* Package = FindObjectFast<UPackage>(Outer, *PackageName);
			if (!Package || Package->GetFileSize() > 0)
			{
				UE_LOGF(
					LogUsd,
					Verbose,
					"Not trying to clean up '%ls' because the path doesn't resolve, or resolve to a saved asset",
					*AssetPathString
				);
				continue;
			}

			// If we're here, it means the asset has never been saved. This means it must be loaded, if it exists
			// at all: Let's quickly fetch it and see if we can actually delete it
			UObject* LoadedObject = Path.TryLoad();
			if (!LoadedObject || !DeletableAssetKeys.Contains(FObjectKey{LoadedObject}))
			{
				// We never want to delete assets that the user manually added to the asset cache
				// (Only assets added via GetOrCreateCachedAsset/CacheAsset are considered Deletable)
				UE_LOGF(LogUsd, Verbose, "Not trying to clean up '%ls' because it hasn't been set as deletable", *AssetPathString);
				continue;
			}

			// Check if asset is referenced by any stage actor or UObject
			if (TArray<FObjectKey>* FoundReferencers = HashToReferencer.Find(Hash))
			{
				if (FoundReferencers->Num() > 0)
				{
					UE_LOGF(LogUsd, Verbose, "Not trying to clean up '%ls' because it has object referencers", *AssetPathString);
					continue;
				}
			}

			AbandonedHashesToObjects.Add(Hash, LoadedObject);
			AbandonedHashesToObjectKeys.Add(Hash, {LoadedObject});

			if (bShowConfirmation)
			{
				AssetsToDelete.Add(AssetRegistryModule.Get().GetAssetByObjectPath(Path));
			}
			else
			{
				ObjectsToDelete.Add(Path.TryLoad());
			}
		}
	}

#if WITH_EDITOR
	if (bShowConfirmation)
	{
		int32 NumDeleted = ObjectTools::DeleteAssets(AssetsToDelete, bShowConfirmation);
		if (NumDeleted == 0)
		{
			return;
		}
	}
	else
#endif	  // WITH_EDITOR
	{
		// We choose our own asset deletion function here for a few reasons:
		// 	- It works at runtime (even the FAssetDeleteModel that ObjectTools::DeleteAssets uses is editor-only)
		//  - If ObjectTools::DeleteAssets finds a reference to any asset when bShowConfirmation==false, it won't delete *any* asset at all.
		// 	  This means that as soon as the user actually makes an external reference to any of the assets we generated, we won't be able
		//    to clear anything anymore, which defeats the entire purpose
		// Note that SafeDeleteObjects should be a bit slower than ObjectTools::DeleteAssets though, unfortunately...
		ObjectsToDelete.Remove(nullptr);
		UE::USDAssetCache3::Private::SafeDeleteObjects(ObjectsToDelete, *this);
	}

	FWriteScopeLock Lock(RWLock);
	for (const TPair<FString, TWeakObjectPtr<UObject>>& Pair : AbandonedHashesToObjects)
	{
		const FString& Hash = Pair.Key;
		UObject* Asset = Pair.Value.Get();

		// ObjectTools::DeleteAssets may not have deleted everything, so make sure we only ever stop tracking
		// the records about assets that have actually been deleted. This is probably a good idea to do in general
		// too, even if we're not using ObjectTools::DeleteAssets
		if (!Asset)
		{
			StopTrackingAssetInternal(Hash);

			// Manually remove the entry on DeletableAssetKeys here or else they will just pile up in there every
			// time we delete assets, as StopTrackingAssetInternal won't be able to remove the entries itself
			if (FObjectKey* KeyToDelete = AbandonedHashesToObjectKeys.Find(Hash))
			{
				DeletableAssetKeys.Remove(*KeyToDelete);
			}
		}
	}
}

void UUsdAssetCache3::DeleteUnreferencedAssetsWithConfirmation()
{
	const bool bShowConfirmation = true;
	DeleteUnreferencedAssets(bShowConfirmation);
}

void UUsdAssetCache3::RescanAssetDirectory()
{
	TArray<FAssetData> AssetDatas;
	bool bGotAssets = false;
	{
		const bool bAlwaysMarkDirty = false;
		Modify(bAlwaysMarkDirty);
		FWriteScopeLock Lock(RWLock);
		ForceValidAssetDirectoryInternal();

		// If we renamed our folder to a new location, automatically check the assets there to see if we can auto cache them
		const FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		const bool bRecursive = true;
		bGotAssets = AssetRegistryModule.Get().GetAssetsByPath(*AssetDirectory.Path, AssetDatas, bRecursive);
	}

	if (bGotAssets)
	{
		for (const FAssetData& ExistingAssetData : AssetDatas)
		{
			if (!AssetPathToHashes.Contains(ExistingAssetData.GetSoftObjectPath()))
			{
				TryCachingAssetFromAssetUserData(ExistingAssetData);
			}
		}
	}
}

UUsdAssetCache3::FUsdScopedReferencer::FUsdScopedReferencer(UUsdAssetCache3* InAssetCache, const UObject* Referencer)
{
	if (!InAssetCache || !Referencer)
	{
		return;
	}

	AssetCache = InAssetCache;
	OldReferencer = InAssetCache->SetCurrentScopedReferencer(Referencer);
}

UUsdAssetCache3::FUsdScopedReferencer::~FUsdScopedReferencer()
{
	if (UUsdAssetCache3* ValidCache = AssetCache.Get())
	{
		ValidCache->SetCurrentScopedReferencer(OldReferencer);
	}
}

UUsdAssetCache3::UUsdAssetCache3()
{
	// The CDO shouldn't be listening to the asset registry events...
	if (IsTemplate())
	{
		return;
	}

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistryModule.Get().OnAssetRenamed().AddUObject(this, &UUsdAssetCache3::OnRegistryAssetRenamed);
}

void UUsdAssetCache3::PostLoad()
{
	Super::PostLoad();

	const bool bEmitWarning = false;
	ForceValidAssetDirectoryInternal(bEmitWarning);

	// There is nothing to load us whenever an asset is added to our AssetDirectory while we were unloaded, so let's
	// make sure we do a new scan whenever we do get loaded to pick up on any new assets that may have been added.
	// We delay this to the next tick though, because we may need to mark ourselves as dirty if we found anything,
	// and we can't do that within the callstack that calls PostLoad on us.
	//
	// Note that this was originally within an AsyncTask, but given that RescanAssetDirectory() locks the RWLock, it's
	// possible to get a deadlock here if the async task is resumed from some unknown point within the callstack of another
	// asset cache call, so we use the ticker instead.
	TWeakObjectPtr<UUsdAssetCache3> WeakThis{this};
	ExecuteOnGameThread(
		UE_SOURCE_LOCATION,
		[WeakThis]()
		{
			if (UUsdAssetCache3* AssetCache = WeakThis.Get())
			{
				AssetCache->RescanAssetDirectory();
			}
		}
	);
}

void UUsdAssetCache3::BeginDestroy()
{
	if (!IsTemplate())
	{
		if (FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>(TEXT("AssetRegistry")))
		{
			AssetRegistryModule->Get().OnAssetRenamed().RemoveAll(this);
		}
	}

	Super::BeginDestroy();
}

void UUsdAssetCache3::Serialize(FArchive& Ar)
{
	FWriteScopeLock Lock(RWLock);

	Super::Serialize(Ar);

	if (!Ar.IsPersistent())
	{
		Ar << HashToReferencer;
		Ar << ReferencerToHash;
		Ar << DeletableAssetKeys;
		Ar << ActiveAssets;
	}
}

#if WITH_EDITOR
void UUsdAssetCache3::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// If we're changing a property inside a struct, like "bCollectMetadata" inside our MetadataOptions, then
	// "MemberProperty" will point to "MetadataOptions", and "Property" is the thing that will point to "bCollectMetadata"
	const FName PropertyName = PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	const FName MemberPropertyName = PropertyChangedEvent.MemberProperty ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;

	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UUsdAssetCache3, HashToAssetPaths))
	{
		FWriteScopeLock Lock(RWLock);

		// If the user changed our HashToAssetPaths map directly, we need to update the reverse map to match it
		AssetPathToHashes.Reset();
		AssetPathToHashes.Reserve(HashToAssetPaths.Num());
		for (const TPair<FString, FSoftObjectPath>& HashToAsset : HashToAssetPaths)
		{
			AssetPathToHashes.Add(HashToAsset.Value, HashToAsset.Key);
		}

		// Cleanup old entries from HashToReferencer
		for (TMap<FString, TArray<FObjectKey>>::TIterator Iter = HashToReferencer.CreateIterator(); Iter; ++Iter)
		{
			if (!HashToAssetPaths.Contains(Iter->Key))
			{
				Iter.RemoveCurrent();
			}
		}

		ActiveAssets.Reset();
	}
	else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UUsdAssetCache3, AssetDirectory))
	{
		RescanAssetDirectory();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif	  // WITH_EDITOR

void UUsdAssetCache3::OnRegistryAssetRenamed(const FAssetData& NewAssetData, const FString& OldName)
{
	// Don't check if NewAssetData is valid or not, as during a move this is called before it becomes valid
	if (OldName.IsEmpty())
	{
		return;
	}

	const FSoftObjectPath OldPath{OldName};
	const FSoftObjectPath NewPath = NewAssetData.GetSoftObjectPath();

	if (!IsAssetTrackedByCache(OldPath))
	{
		// An asset was dragged into the AssetDirectory, let's see if it knows its own hash, and then
		// automatically cache it if we can
		const FString NewPathString = NewPath.GetAssetPathString();
		if (NewPathString.StartsWith(AssetDirectory.Path))
		{
			TryCachingAssetFromAssetUserData(NewAssetData);
		}

		return;
	}

	FWriteScopeLock Lock(RWLock);

	// We only need to update ActiveAssets: The asset registry will itself reserialize us and fix up
	// any FSoftObjectPath non-transient property to the new path, and most of our other internal maps
	// are based on asset hashes and not FSoftObjectPath
	const bool bWasActive = ActiveAssets.Contains(OldPath);
	ActiveAssets.Remove(OldPath);
	if (bWasActive)
	{
		ActiveAssets.Add(NewPath);
	}
}

void UUsdAssetCache3::RequestDelayedAssetAutoCleanup()
{
	FWriteScopeLock Lock(RWLock);

	if (bCleanUpUnreferencedAssets && !bPendingCleanup)
	{
		// Only actually run the asset cache cleanup on the next engine tick.
		// This because we may be unloading due to just wanting to open a different stage. That stage
		// could potentially reuse some of the (currently unreferenced) assets in the asset cache.
		// Delaying the cleanup gives us the chance to reuse and add new "referencers" to those assets,
		// preventing them from being dropped
		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda(
			[this](float TickerTime)
			{
				const bool bShowConfirmation = false;
				DeleteUnreferencedAssets(bShowConfirmation);

				bPendingCleanup = false;
				return false;	 // Don't run again
			}
		));
		bPendingCleanup = true;
	}
}

void UUsdAssetCache3::TouchAsset(const FString& Hash, const UObject* Referencer)
{
	FSoftObjectPath CachedPath = GetCachedAssetPath(Hash);
	if (!CachedPath.IsValid())
	{
		return;
	}

	Modify();

	FWriteScopeLock Lock(RWLock);
	TouchAssetInternal(CachedPath, Referencer);
}

void UUsdAssetCache3::TouchAssetPath(const FSoftObjectPath& AssetPath, const UObject* Referencer)
{
	if (!AssetPath.IsValid())
	{
		return;
	}

	Modify();

	FWriteScopeLock Lock(RWLock);
	TouchAssetInternal(AssetPath, Referencer);
}

void UUsdAssetCache3::MarkAssetsAsStale()
{
	FWriteScopeLock Lock(RWLock);

	ActiveAssets.Reset();
}

TSet<FSoftObjectPath> UUsdAssetCache3::GetActiveAssets() const
{
	FReadScopeLock Lock(RWLock);

	// Return a copy for thread safety
	return ActiveAssets;
}

const UObject* UUsdAssetCache3::SetCurrentScopedReferencer(const UObject* NewReferencer)
{
	FWriteScopeLock Lock(RWLock);

	Swap(NewReferencer, CurrentScopedReferencer);
	return NewReferencer;
}

void UUsdAssetCache3::AddReferenceInternal(const FString& Hash, const UObject* Referencer)
{
	if (Hash.IsEmpty() || !Referencer)
	{
		return;
	}

	FObjectKey ReferencerKey{Referencer};

	HashToReferencer.FindOrAdd(Hash).Add(ReferencerKey);
	ReferencerToHash.FindOrAdd(ReferencerKey).Add(Hash);
}

FSoftObjectPath UUsdAssetCache3::StopTrackingAssetInternal(const FString& Hash)
{
	if (Hash.IsEmpty())
	{
		return {};
	}

	FSoftObjectPath Removed;
	HashToAssetPaths.RemoveAndCopyValue(Hash, Removed);

	AssetPathToHashes.Remove(Removed);

	TransientObjectStorage.Remove(Hash);

	UObject* LoadedObject = nullptr;
	if (Removed.IsValid())
	{
		FString RemovedStr = Removed.GetAssetPathString();

		FString ClassName;
		FString PackageName;
		FString ObjectName;
		FString SubObjectName;
		const bool bDetectClassName = false;
		FPackageName::SplitFullObjectPath(RemovedStr, ClassName, PackageName, ObjectName, SubObjectName, bDetectClassName);

		// Note: PackageName may be something like "/Game/UsdAssets/MyMaterial" but also "/Engine/Transient" for the transient package
		if (!PackageName.IsEmpty())
		{
			UObject* Outer = nullptr;
			UPackage* AssetPackage = FindObjectFast<UPackage>(Outer, *PackageName);
			if (AssetPackage && AssetPackage->GetFileSize() == 0 && AssetPackage->IsFullyLoaded())
			{
				// Note: This may fail and return nullptr if we're publishing assets
				// from a direct import
				LoadedObject = FindObjectFast<UObject>(AssetPackage, *ObjectName);
			}
		}
	}
	if (LoadedObject)
	{
		UE_LOGF(LogUsd, Verbose, "Setting '%ls' as undeletable", *LoadedObject->GetPathName());
		DeletableAssetKeys.Remove(FObjectKey{LoadedObject});
	}

	RemoveAllAssetReferencersInternal(Hash);

	return Removed;
}

bool UUsdAssetCache3::RemoveAllAssetReferencersInternal(const FString& Hash)
{
	TArray<FObjectKey> ReferencerKeys;
	bool bRemovedSomething = HashToReferencer.RemoveAndCopyValue(Hash, ReferencerKeys);

	for (const FObjectKey& ReferencerKey : ReferencerKeys)
	{
		if (TArray<FString>* FoundReferencerAssets = ReferencerToHash.Find(ReferencerKey))
		{
			FoundReferencerAssets->Remove(Hash);
		}
	}

	return bRemovedSomething;
}

void UUsdAssetCache3::TouchAssetInternal(const FSoftObjectPath& AssetPath, const UObject* Referencer)
{
	ActiveAssets.Add(AssetPath);

	const UObject* ReferencerToUse = Referencer ? Referencer : CurrentScopedReferencer;
	if (ReferencerToUse)
	{
		if (FString* Hash = AssetPathToHashes.Find(AssetPath))
		{
			AddReferenceInternal(*Hash, ReferencerToUse);
		}
	}
}

void UUsdAssetCache3::TryCachingAssetFromAssetUserData(const FAssetData& ExistingAssetData)
{
	const FSoftObjectPath ExistingAssetPath = ExistingAssetData.GetSoftObjectPath();
	const FString& ExistingAssetPathStr = ExistingAssetPath.GetAssetPathString();

	if (!ExistingAssetData.IsAssetLoaded())
	{
		UE_LOGF(
			LogUsd,
			Verbose,
			"Loading existing asset '%ls' to check if it can be automatically added to the asset cache '%ls'",
			*ExistingAssetPathStr,
			*GetPathName()
		);
	}

	UObject* LoadedAsset = ExistingAssetPath.TryLoad();
	if (LoadedAsset && LoadedAsset->GetClass()->ImplementsInterface(UInterface_AssetUserData::StaticClass()))
	{
		if (UUsdAssetUserData* UserData = UsdUnreal::ObjectUtils::GetAssetUserData(LoadedAsset))
		{
			if (!UserData->OriginalHash.IsEmpty())
			{
				UE_LOGF(
					LogUsd,
					Verbose,
					"Automatically caching asset '%ls' into asset cache '%ls' with hash '%ls'",
					*ExistingAssetPathStr,
					*GetPathName(),
					*UserData->OriginalHash
				);

				// Never overwrite an existing cached asset with something we pick up from the scan, that way
				// scanning is never "destructive"
				if (!HashToAssetPaths.Contains(UserData->OriginalHash))
				{
					CacheAsset(UserData->OriginalHash, ExistingAssetPath);
				}
			}
		}
	}
}

bool UUsdAssetCache3::IsTransientCache()
{
	return GetOutermost() == GetTransientPackage();
}

void UUsdAssetCache3::ForceValidAssetDirectoryInternal(bool bEmitWarning)
{
	if (IsTemplate())
	{
		return;
	}

	const bool bIsTransientCache = IsTransientCache();

	// Transient caches should always be pointing at the transient package
	bool bCurrentIsValid = true;
	if (bIsTransientCache)
	{
		bCurrentIsValid = AssetDirectory.Path == GetTransientPackage()->GetPathName();
	}
	if (bCurrentIsValid)
	{
		// We'll want to spawn packages inside of AssetDirectory.Path, so it itself should be a valid package name already
		bCurrentIsValid &= FPackageName::IsValidLongPackageName(AssetDirectory.Path);
	}

	// If our asset path is invalid, reset to something reasonable
	if (!bCurrentIsValid)
	{
		FString NewPath;

		// If we're a transient asset cache, let's cache our assets in the transient package.
		// This is used for direct importing (import from content browser, Import Into Level, etc.)
		if (bIsTransientCache)
		{
			NewPath = GetTransientPackage()->GetPathName();
		}
		// Opening stages and Actions->Import will use a non-transient asset cache that will
		// make individual packages for each asset
		else
		{
			NewPath = FPaths::Combine(FPaths::GetPath(GetPathName()), TEXT("UsdAssets"));
		}

		if (bEmitWarning)
		{
			UE_LOGF(
				LogUsd,
				Log,
				"Resetting %ls's AssetDirectory to '%ls', as '%ls' is not a valid content folder path for this asset cache",
				*GetPathName(),
				*NewPath,
				*AssetDirectory.Path
			);
		}
		AssetDirectory.Path = NewPath;
	}

	// Make sure that AssetDirectory.Path actually exists on disk if we need it to, otherwise the content
	// folder itself won't actually show up on the content browser
	if (!bIsTransientCache)
	{
		const static FString GamePrefix = TEXT("/Game/");
		FString AssetDiskDirectory = AssetDirectory.Path;
		if (AssetDiskDirectory.StartsWith(GamePrefix))
		{
			AssetDiskDirectory = AssetDiskDirectory.RightChop(GamePrefix.Len());
		}
		AssetDiskDirectory = FPaths::Combine(FPaths::ProjectContentDir(), AssetDiskDirectory);

		const bool bMakeEntireTree = true;
		IFileManager::Get().MakeDirectory(*AssetDiskDirectory, bMakeEntireTree);
	}
}

#undef LOCTEXT_NAMESPACE
