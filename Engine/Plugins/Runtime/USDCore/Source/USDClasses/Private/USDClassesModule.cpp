// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDClassesModule.h"

#include "USDAssetCache3.h"
#include "USDLog.h"
#include "USDObjectUtils.h"
#include "USDProjectSettings.h"

#include "AnalyticsEventAttribute.h"
#include "Animation/AnimBlueprint.h"
#include "Animation/AnimSequence.h"
#include "Animation/Skeleton.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/Engine.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture.h"
#include "EngineAnalytics.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GeometryCache.h"
#include "GroomAsset.h"
#include "GroomBindingAsset.h"
#include "GroomCache.h"
#include "HAL/FileManager.h"
#include "LevelSequence.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInterface.h"
#include "Misc/EngineVersion.h"
#include "Misc/FileHelper.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Misc/SecureHash.h"
#include "Modules/ModuleManager.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "Serialization/JsonSerializer.h"
#include "Sound/SoundBase.h"
#include "SparseVolumeTexture/SparseVolumeTexture.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "Widgets/Notifications/SNotificationList.h"

#if WITH_EDITOR
#include "AssetToolsModule.h"
#include "MaterialEditingLibrary.h"
#endif	  // WITH_EDITOR

DEFINE_LOG_CATEGORY(LogUsd);

#define LOCTEXT_NAMESPACE "USDClassesModule"

namespace UE::USDClasses::Private
{
	TSharedPtr<FJsonObject> ParseJSON(const FString& FileContents)
	{
		if (FileContents.IsEmpty())
		{
			return nullptr;
		}

		const TSharedRef<TJsonReader<>>& Reader = TJsonReaderFactory<>::Create(FileContents);

		TSharedPtr<FJsonObject> DescriptorObject;
		if (FJsonSerializer::Deserialize(Reader, DescriptorObject) && DescriptorObject.IsValid())
		{
			return DescriptorObject;
		}
		else
		{
			UE_LOGF(LogUsd, Warning, "Failed to parse plugInfo.json file: '%ls'", *Reader->GetErrorMessage());
		}

		return nullptr;
	}

	TMap<FString, int32> PackagePathNameToDirtyCounter;

	TMap<FString, TArray<FAnalyticsEventAttribute>> AccumulatedEventAttributes;
	int AnalyticsAllowedWhenZero = 0;
}	 // namespace UE::USDClasses::Private

void IUsdClassesModule::UpdatePlugInfoFiles(const FString& PluginDirectory, const FString& TargetDllFolder)
{
	// Prevent cooker worker processes from trying to patch the plugInfo.json files: Only the director
	// process should do that
	if (UE::GetMultiprocessId() != 0)
	{
		return;
	}

	// Traverse all USD plugins
	TArray<FString> JsonPaths;
	const bool bFiles = true;
	const bool bDirectories = false;
	const bool bClearFileNames = false;	   // Whether it needs to reset the output array or not
	IFileManager::Get().FindFilesRecursive(JsonPaths, *PluginDirectory, TEXT("plugInfo.json"), bFiles, bDirectories, bClearFileNames);
	for (const FString& JsonFilePath : JsonPaths)
	{
		FString FileContent;
		if (!FFileHelper::LoadFileToString(FileContent, *JsonFilePath))
		{
			continue;
		}

		// Strip a preamble that looks like this:
		//		# Portions of this file auto-generated by usdGenSchema.
		//		# Edits will survive regeneration except for comments and
		//		# changes to types with autoGenerated=true.
		// We will not save this preamble back into the file, but that's not bad as the file is already in the binaries folder
		// and the preamble is actually invalid json anyway
		int32 FirstOpenBracketPos = FileContent.Find(TEXT("{"));
		int32 NewLen = FileContent.Len() - FirstOpenBracketPos;
		FileContent.RightInline(NewLen);

		TSharedPtr<FJsonObject> DescriptorObject = UE::USDClasses::Private::ParseJSON(FileContent);
		if (!DescriptorObject.IsValid())
		{
			continue;
		}

		TSharedPtr<FJsonValue> PluginsField = DescriptorObject->TryGetField(TEXT("Plugins"));
		if (!PluginsField.IsValid())
		{
			continue;
		}

		bool bUpdatedFile = false;

		const TArray<TSharedPtr<FJsonValue>>* PluginEntriesPtr = nullptr;
		if (!PluginsField->TryGetArray(PluginEntriesPtr) || !PluginEntriesPtr)
		{
			continue;
		}

		for (const TSharedPtr<FJsonValue>& PluginEntry : *PluginEntriesPtr)
		{
			const TSharedPtr<FJsonObject>* PluginEntryObjectPtr = nullptr;
			if (!PluginEntry->TryGetObject(PluginEntryObjectPtr) || !PluginEntryObjectPtr || !PluginEntryObjectPtr->IsValid())
			{
				continue;
			}

			TSharedPtr<FJsonValue> LibraryPathField = (*PluginEntryObjectPtr)->TryGetField(TEXT("LibraryPath"));
			if (!LibraryPathField.IsValid())
			{
				continue;
			}

			// e.g. "../../../../../../Win64/sdf.dll"
			FString LibraryPath;
			if (!LibraryPathField->TryGetString(LibraryPath))
			{
				continue;
			}

			// e.g. "sdf.dll"
			FString DllName = FPaths::GetCleanFilename(LibraryPath);
			if (DllName.IsEmpty())
			{
				continue;
			}

			// e.g. ".."
			FString RootPath;
			if (TSharedPtr<FJsonValue> RootPathField = (*PluginEntryObjectPtr)->TryGetField(TEXT("Root")))
			{
				RootPathField->TryGetString(RootPath);
			}

			// e.g. "E:/Folder/Workspace/Engine/Binaries/ThirdParty/USD/UsdResources/Win64/plugins/ar/resources/.."
			RootPath = FPaths::Combine(FPaths::GetPath(JsonFilePath), RootPath);
			FPaths::NormalizeDirectoryName(RootPath);
			FPaths::CollapseRelativeDirectories(RootPath);

			FString PathToDll = TargetDllFolder;

			// Ensure directories end with a slash, otherwise FPaths::MakePathRelativeTo may strip the last folder thinking it's a filename
			if (!PathToDll.EndsWith(TEXT("/")))
			{
				PathToDll.AppendChar(TEXT('/'));
			}
			if (!RootPath.EndsWith(TEXT("/")))
			{
				RootPath.AppendChar(TEXT('/'));
			}

			// e.g. "../../../../../../Win64"
			// This can fail if we're in different drives. Then we're forced to assume that CurrentDllLocation is an absolute path and use it directly
			if (!FPaths::MakePathRelativeTo(PathToDll, *RootPath))
			{
				if (FPaths::IsRelative(PathToDll) && !FPaths::DirectoryExists(PathToDll))
				{
					UE_LOGF(LogUsd, Warning, "Failed to handle Current DLL Location '%ls' when trying to update plugInfo.json paths", *PathToDll);
					continue;
				}
			}

			// e.g. "../../../../../../Win64/sdf.dll"
			FString NewLibraryPath = FPaths::Combine(PathToDll, DllName);
			if (!FPaths::IsSamePath(NewLibraryPath, LibraryPath))
			{
				bUpdatedFile = true;
				(*PluginEntryObjectPtr)->SetField(TEXT("LibraryPath"), MakeShared<FJsonValueString>(NewLibraryPath));
			}
		}

		// Serialize them back to JSON if we changed anything
		if (bUpdatedFile)
		{
			FString OutputString;
			TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&OutputString);
			FJsonSerializer::Serialize(DescriptorObject.ToSharedRef(), Writer);

			if (FFileHelper::SaveStringToFile(
					OutputString,
					*JsonFilePath,
					FFileHelper::EEncodingOptions::AutoDetect,
					&IFileManager::Get(),
					FILEWRITE_EvenIfReadOnly
				))
			{
				UE_LOGF(LogUsd, Log, "Updated LibraryPaths for USD plugInfo.json file '%ls'", *JsonFilePath);
			}
			else
			{
				UE_LOGF(LogUsd, Warning, "Failed to update LibraryPath for USD plugInfo.json file '%ls'", *JsonFilePath);
			}
		}
	}
}

void IUsdClassesModule::SendAnalytics(
	TArray<FAnalyticsEventAttribute>&& InAttributes,
	const FString& EventName,
	bool bAutomated,
	double ElapsedSeconds,
	double NumberOfFrames,
	const FString& Extension
)
{
	if (!FEngineAnalytics::IsAvailable())
	{
		return;
	}

	// Send the event right away
	if (UE::USDClasses::Private::AnalyticsAllowedWhenZero == 0)
	{
		TArray<FAnalyticsEventAttribute> Attributes(InAttributes);
		Attributes.Emplace(TEXT("Automated"), LexToString(bAutomated));
		Attributes.Emplace(TEXT("FileExtension"), Extension);
		Attributes.Emplace(TEXT("NumberOfFrames"), LexToString(FMath::Abs(NumberOfFrames)));
		Attributes.Emplace(TEXT("TimeTaken.Seconds"), ElapsedSeconds);
		Attributes.Emplace(TEXT("Platform"), FPlatformProperties::IniPlatformName());
		Attributes.Emplace(TEXT("EngineVersion"), FEngineVersion::Current().ToString());
		Attributes.Emplace(TEXT("EngineMode"), FPlatformMisc::GetEngineMode());

		const FString EventFullName = FString::Printf(TEXT("Engine.Usage.USD.%s"), *EventName);
		FEngineAnalytics::GetProvider().RecordEvent(EventFullName, Attributes);
	}
	// Something is suppressing analytics, so just accumulate the events
	else
	{
		UE::USDClasses::Private::AccumulatedEventAttributes.FindOrAdd(EventName, InAttributes);
	}
}

void IUsdClassesModule::SendAnalytics(TArray<FAnalyticsEventAttribute>&& InAttributes, const FString& EventName)
{
	if (!FEngineAnalytics::IsAvailable())
	{
		return;
	}

	// Send the event right away
	if (UE::USDClasses::Private::AnalyticsAllowedWhenZero == 0)
	{
		TArray<FAnalyticsEventAttribute> Attributes(InAttributes);
		Attributes.Emplace(TEXT("Platform"), FPlatformProperties::IniPlatformName());
		Attributes.Emplace(TEXT("EngineVersion"), FEngineVersion::Current().ToString());
		Attributes.Emplace(TEXT("EngineMode"), FPlatformMisc::GetEngineMode());

		const FString EventFullName = FString::Printf(TEXT("Engine.Usage.USD.%s"), *EventName);
		FEngineAnalytics::GetProvider().RecordEvent(EventFullName, Attributes);
	}
	// Something is suppressing analytics, so just accumulate the events
	else
	{
		UE::USDClasses::Private::AccumulatedEventAttributes.FindOrAdd(EventName, InAttributes);
	}
}

FString IUsdClassesModule::GetClassNameForAnalytics(const UObject* Object)
{
	if (!Object)
	{
		return TEXT("None");
	}
	else if (Object->IsA<UStaticMesh>())
	{
		return TEXT("StaticMesh");
	}
	else if (Object->IsA<USkeletalMesh>())
	{
		return TEXT("SkeletalMesh");
	}
	else if (Object->IsA<UAnimSequence>())
	{
		return TEXT("AnimSequence");
	}
	else if (Object->IsA<UMaterialInstance>())
	{
		return TEXT("MaterialInstance");
	}
	else if (Object->IsA<UMaterial>())
	{
		return TEXT("Material");
	}
	else if (Object->IsA<UGeometryCache>())
	{
		return TEXT("GeometryCache");
	}
	else if (Object->IsA<ULevelSequence>())
	{
		return TEXT("LevelSequence");
	}
	else if (Object->IsA<UTexture>())
	{
		return TEXT("Texture");
	}
	else if (Object->IsA<UGroomAsset>())
	{
		return TEXT("GroomAsset");
	}
	else if (Object->IsA<UGroomBindingAsset>())
	{
		return TEXT("GroomBindingAsset");
	}
	else if (Object->IsA<UGroomCache>())
	{
		return TEXT("GroomCache");
	}
	else if (Object->IsA<USparseVolumeTexture>())
	{
		return TEXT("SparseVolumeTexture");
	}
	else if (Object->IsA<USoundBase>())
	{
		return TEXT("SoundBase");
	}
	else
	{
		return TEXT("Other");
	}
}

void IUsdClassesModule::AddAssetCountAttributes(const TSet<UObject*>& Assets, TArray<FAnalyticsEventAttribute>& InOutAttributes)
{
	int32 NumStaticMeshes = 0;
	int32 NumSkeletalMeshes = 0;
	int32 NumMaterials = 0;
	int32 NumAnimSequences = 0;
	int32 NumLevelSequences = 0;
	int32 NumTextures = 0;
	int32 NumGeometryCaches = 0;
	int32 NumGroomAssets = 0;
	int32 NumGroomBindings = 0;
	int32 NumGroomCaches = 0;
	int32 NumSparseVolumeTextures = 0;
	int32 NumSounds = 0;

	for (UObject* ImportedAsset : Assets)
	{
		if (!ImportedAsset)
		{
			continue;
		}

		if (ImportedAsset->IsA<UStaticMesh>())
		{
			++NumStaticMeshes;
		}
		else if (ImportedAsset->IsA<USkeletalMesh>())
		{
			++NumSkeletalMeshes;
		}
		else if (ImportedAsset->IsA<UMaterialInterface>())
		{
			++NumMaterials;
		}
		else if (ImportedAsset->IsA<UAnimSequence>())
		{
			++NumAnimSequences;
		}
		else if (ImportedAsset->IsA<ULevelSequence>())
		{
			++NumLevelSequences;
		}
		else if (ImportedAsset->IsA<UTexture>())
		{
			++NumTextures;
		}
		else if (ImportedAsset->IsA<UGeometryCache>())
		{
			++NumGeometryCaches;
		}
		else if (ImportedAsset->IsA<UGroomAsset>())
		{
			++NumGroomAssets;
		}
		else if (ImportedAsset->IsA<UGroomBindingAsset>())
		{
			++NumGroomBindings;
		}
		else if (ImportedAsset->IsA<UGroomCache>())
		{
			++NumGroomCaches;
		}
		else if (ImportedAsset->IsA<USparseVolumeTexture>())
		{
			++NumSparseVolumeTextures;
		}
		else if (ImportedAsset->IsA<USoundBase>())
		{
			++NumSounds;
		}
	}

	InOutAttributes.Emplace(TEXT("NumStaticMeshes"), LexToString(NumStaticMeshes));
	InOutAttributes.Emplace(TEXT("NumSkeletalMeshes"), LexToString(NumSkeletalMeshes));
	InOutAttributes.Emplace(TEXT("NumMaterials"), LexToString(NumMaterials));
	InOutAttributes.Emplace(TEXT("NumAnimSequences"), LexToString(NumAnimSequences));
	InOutAttributes.Emplace(TEXT("NumLevelSequences"), LexToString(NumLevelSequences));
	InOutAttributes.Emplace(TEXT("NumTextures"), LexToString(NumTextures));
	InOutAttributes.Emplace(TEXT("NumGeometryCaches"), LexToString(NumGeometryCaches));
	InOutAttributes.Emplace(TEXT("NumGroomAssets"), LexToString(NumGroomAssets));
	InOutAttributes.Emplace(TEXT("NumGroomBindings"), LexToString(NumGroomBindings));
	InOutAttributes.Emplace(TEXT("NumGroomCaches"), LexToString(NumGroomCaches));
	InOutAttributes.Emplace(TEXT("NumSparseVolumeTextures"), LexToString(NumSparseVolumeTextures));
	InOutAttributes.Emplace(TEXT("NumSounds"), LexToString(NumSounds));
}

void IUsdClassesModule::BlockAnalyticsEvents()
{
	// Clear our accumulated events when first going from unblocked to blocked
	if (UE::USDClasses::Private::AnalyticsAllowedWhenZero == 0)
	{
		UE::USDClasses::Private::AccumulatedEventAttributes.Reset();
	}

	UE::USDClasses::Private::AnalyticsAllowedWhenZero += 1;
}

void IUsdClassesModule::ResumeAnalyticsEvents()
{
	UE::USDClasses::Private::AnalyticsAllowedWhenZero -= 1;
}

const TMap<FString, TArray<FAnalyticsEventAttribute>>& IUsdClassesModule::GetAccumulatedAnalytics()
{
	return UE::USDClasses::Private::AccumulatedEventAttributes;
}

bool IUsdClassesModule::HashObjectPackage(const UObject* Object, FSHA1& HashToUpdate)
{
#if WITH_EDITOR
	if (!Object)
	{
		return false;
	}

	UPackage* Package = Object->GetOutermost();
	if (!Package)
	{
		return false;
	}

	// Hash package's persistent Guid
	const FGuid& Guid = Package->GetPersistentGuid();
	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Guid), sizeof(Guid));

	// Hash last modified date
	FString PackageFullName = Package->GetPathName();
	FString FileName;
	if (FPackageName::TryConvertLongPackageNameToFilename(PackageFullName, FileName))
	{
		FFileStatData StatData = IFileManager::Get().GetStatData(*FileName);
		if (StatData.bIsValid)
		{
			FString ModifiedTimeString = StatData.ModificationTime.ToString();
			HashToUpdate.UpdateWithString(*ModifiedTimeString, ModifiedTimeString.Len());
		}
	}

	// If this asset is currently dirty, also hash how many times it was dirtied in this session.
	// If it's ever saved, we'll reset this counter but update the last saved date
	if (int32* DirtyCounter = UE::USDClasses::Private::PackagePathNameToDirtyCounter.Find(PackageFullName))
	{
		HashToUpdate.Update(reinterpret_cast<const uint8*>(DirtyCounter), sizeof(*DirtyCounter));
	}

	return true;
#else
	return false;
#endif	  // WITH_EDITOR
}

UWorld* IUsdClassesModule::GetCurrentWorld(bool bEditorWorldsOnly)
{
	UWorld* EditorWorld = nullptr;
	UWorld* LowestPIEWorld = nullptr;
	int32 LowestPIEInstance = TNumericLimits<int32>::Max();

	for (const FWorldContext& Context : GEngine->GetWorldContexts())
	{
		UWorld* World = Context.World();
		if (!World)
		{
			continue;
		}

		if (!bEditorWorldsOnly && Context.WorldType == EWorldType::PIE)
		{
			if (!LowestPIEInstance || Context.PIEInstance < LowestPIEInstance)
			{
				LowestPIEWorld = World;
				LowestPIEInstance = Context.PIEInstance;
			}
		}
		else if (Context.WorldType == EWorldType::Editor)
		{
			EditorWorld = World;
		}
	}

	if (LowestPIEWorld)
	{
		return LowestPIEWorld;
	}

	return EditorWorld;
}

TSet<UObject*> IUsdClassesModule::GetAssetDependencies(UObject* Asset)
{
	TSet<UObject*> Result;

	if (UMaterial* Material = Cast<UMaterial>(Asset))
	{
		TArray<UTexture*> UsedTextures;
		Material->GetUsedTextures(UsedTextures);

		Result.Reserve(Result.Num() + UsedTextures.Num());
		for (UTexture* UsedTexture : UsedTextures)
		{
			Result.Add(UsedTexture);
		}
	}
	else if (UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(Asset))
	{
		Result.Reserve(Result.Num() + MaterialInstance->TextureParameterValues.Num());
		for (const FTextureParameterValue& TextureValue : MaterialInstance->TextureParameterValues)
		{
			Result.Add(TextureValue.ParameterValue);
		}

		// We'll have a dependency on our reference material too of course (this happens for Mdl
		// materials for example, that create new UMaterial assets every time, and also material instances).
		Result.Add(MaterialInstance->Parent.Get());
	}
	else if (USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Asset))
	{
		Result.Add(SkeletalMesh->GetSkeleton());
		Result.Add(SkeletalMesh->GetPhysicsAsset());

		Result.Reserve(Result.Num() + SkeletalMesh->GetMaterials().Num());
		for (const FSkeletalMaterial& SkeletalMaterial : SkeletalMesh->GetMaterials())
		{
			Result.Add(SkeletalMaterial.MaterialInterface);
		}
	}
	else if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(Asset))
	{
		Result.Reserve(Result.Num() + StaticMesh->GetStaticMaterials().Num());
		for (const FStaticMaterial& StaticMaterial : StaticMesh->GetStaticMaterials())
		{
			Result.Add(StaticMaterial.MaterialInterface);
		}
	}
	else if (UGeometryCache* GeometryCache = Cast<UGeometryCache>(Asset))
	{
		for (UMaterialInterface* UsedMaterial : GeometryCache->Materials)
		{
			Result.Add(UsedMaterial);
		}
	}
	else if (UAnimSequence* AnimSequence = Cast<UAnimSequence>(Asset))
	{
		Result.Add(AnimSequence->GetPreviewMesh());
		Result.Add(AnimSequence->GetSkeleton());
	}
	else if (UGroomBindingAsset* GroomBinding = Cast<UGroomBindingAsset>(Asset))
	{
		Result.Add(GroomBinding->GetGroom());
		Result.Add(GroomBinding->GetTargetSkeletalMesh());
		Result.Add(GroomBinding->GetSourceSkeletalMesh());
		Result.Add(GroomBinding->GetSourceGeometryCache());
		Result.Add(GroomBinding->GetTargetGeometryCache());
	}
	else if (UGroomAsset* GroomAsset = Cast<UGroomAsset>(Asset))
	{
		// Do nothing. The atual groom assets have no additional dependencies
	}
	else if (UGroomCache* GroomCache = Cast<UGroomCache>(Asset))
	{
		// Do nothing. The groom cache doesn't have any additional dependencies
	}
	else if (UTexture* Texture = Cast<UTexture>(Asset))
	{
		// Do nothing. Textures have no additional dependencies
	}
	else if (USkeleton* Skeleton = Cast<USkeleton>(Asset))
	{
		// Do nothing. Skeletons have no additional dependencies
	}
	else if (UPhysicsAsset* PhysicsAsset = Cast<UPhysicsAsset>(Asset))
	{
		// Do nothing. PhysicsAssets have no additional dependencies
	}
	else if (UAnimBlueprint* AnimBP = Cast<UAnimBlueprint>(Asset))
	{
		// Do nothing. AnimBlueprints have no additional dependencies
	}
	else if (USparseVolumeTexture* SparseVolumeTexture = Cast<USparseVolumeTexture>(Asset))
	{
		// Do nothing. SparseVolumeTextures have no additional dependencies
	}
	else if (USoundBase* Sound = Cast<USoundBase>(Asset))
	{
		// Do nothing. Sounds have no additional dependencies
	}
	else
	{
		UE_LOGF(LogUsd, Warning, "Unknown asset '%ls' encountered when collecting dependencies.", Asset ? *Asset->GetName() : TEXT("nullptr"));
	}

	// This way we don't have to nullptr check everything we add to the set
	Result.Remove(nullptr);
	return Result;
}

UUsdAssetCache3* IUsdClassesModule::GetAssetCacheForProject()
{
	// First check if we have any valid AssetCache on the project settings
	UUsdProjectSettings* ProjectSettings = GetMutableDefault<UUsdProjectSettings>();
	if (ProjectSettings)
	{
		// Check if the package exists on disk first to try and avoid some ugly warnings if we try calling TryLoad with a broken path
		const FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(ProjectSettings->DefaultAssetCache);
		if (AssetData.IsValid())
		{
			if (UUsdAssetCache3* DefaultCache = Cast<UUsdAssetCache3>(ProjectSettings->DefaultAssetCache.TryLoad()))
			{
				return DefaultCache;
			}
		}
	}

	// Don't record the creation of the asset cache itself into the transaction buffer (and the setting of it on the project settings)
	// so that it remains if we undo. In general I don't think we ever want to "undo the creation of assets" anyway, but here
	// it also helps prevent this issue: Get stage actor without asset cache -> Open stage (and get a new
	// asset cache that we create right here) -> Undo -> Open stage again.
	// That would give the stage actor a brand new asset cache without the property values (including the tracked assets)
	// the previous one had, which could lead us to create duplicates of every asset...
	TGuardValue<ITransaction*> SuppressTransaction{GUndo, nullptr};

	UUsdAssetCache3* DefaultCache = nullptr;
#if WITH_EDITOR
	if (GIsEditor)
	{
		// Nothing is set on the project settings yet, so let's create a brand new asset cache.
		// First let's find a unique package name
		FString DesiredPath = TEXT("/Game/UsdAssetCache");
		FString Suffix = TEXT("");
		FString UniquePackageName;
		FString UniqueAssetName;
		const FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
		AssetToolsModule.Get().CreateUniqueAssetName(DesiredPath, Suffix, UniquePackageName, UniqueAssetName);
		if (UniquePackageName.EndsWith(UniqueAssetName))
		{
			UniquePackageName = UniquePackageName.LeftChop(UniqueAssetName.Len() + 1);
		}

		// Create the new asset cache
		UFactory* Factory = nullptr;
		DefaultCache = Cast<UUsdAssetCache3>(
			AssetToolsModule.Get().CreateAsset(*UniqueAssetName, *UniquePackageName, UUsdAssetCache3::StaticClass(), Factory)
		);
	}
	else
#endif	  // WITH_EDITOR
	{
		DefaultCache = NewObject<UUsdAssetCache3>();
	}

	if (DefaultCache && ProjectSettings && GIsEditor)
	{
		const FText Text = LOCTEXT("NewAssetCacheToastText", "USD Asset Cache");

		const FText SubText = FText::Format(
			LOCTEXT(
				"NewAssetCacheToastSubText",
				"A new default UsdAssetCache asset was created for the project at path '{0}', but it can be changed in the project settings.\n\nThe UsdAssetCache is used by UsdStageActors in order to share and reuse assets generated from USD."
			),
			FText::FromString(DefaultCache->GetPathName())
		);

		UE_LOGF(LogUsd, Log, "%ls", *SubText.ToString().Replace(TEXT("\n\n"), TEXT(" ")));

		const UUsdProjectSettings* Settings = GetDefault<UUsdProjectSettings>();
		if (Settings && Settings->bShowCreateDefaultAssetCacheDialog)
		{
			static TWeakPtr<SNotificationItem> Notification;

			FNotificationInfo Toast(Text);
			Toast.SubText = SubText;
			Toast.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Warning"));
			Toast.CheckBoxText = LOCTEXT("DontAskAgain", "Don't prompt again");
			Toast.bUseLargeFont = false;
			Toast.bFireAndForget = false;
			Toast.FadeOutDuration = 0.0f;
			Toast.ExpireDuration = 0.0f;
			Toast.bUseThrobber = false;
			Toast.bUseSuccessFailIcons = false;
			Toast.ButtonDetails.Emplace(
				LOCTEXT("OverridenOpinionMessageOk", "Ok"),
				FText::GetEmpty(),
				FSimpleDelegate::CreateLambda(
					[]()
					{
						if (TSharedPtr<SNotificationItem> PinnedNotification = Notification.Pin())
						{
							PinnedNotification->SetCompletionState(SNotificationItem::CS_Success);
							PinnedNotification->ExpireAndFadeout();
						}
					}
				)
			);
			// This is flipped because the default checkbox message is "Don't prompt again"
			Toast.CheckBoxState = Settings->bShowCreateDefaultAssetCacheDialog ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
			Toast.CheckBoxStateChanged = FOnCheckStateChanged::CreateStatic(
				[](ECheckBoxState NewState)
				{
					if (UUsdProjectSettings* Settings = GetMutableDefault<UUsdProjectSettings>())
					{
						// This is flipped because the default checkbox message is "Don't prompt again"
						Settings->bShowCreateDefaultAssetCacheDialog = NewState == ECheckBoxState::Unchecked;
						Settings->SaveConfig();
					}
				}
			);

			// Only show one at a time
			if (!Notification.IsValid())
			{
				Notification = FSlateNotificationManager::Get().AddNotification(Toast);
			}

			if (TSharedPtr<SNotificationItem> PinnedNotification = Notification.Pin())
			{
				PinnedNotification->SetCompletionState(SNotificationItem::CS_Pending);
			}
		}

		ProjectSettings->DefaultAssetCache = DefaultCache;
		ProjectSettings->SaveConfig();
	}

	return DefaultCache;
}

class FUsdClassesModule : public IUsdClassesModule
{
public:
	virtual void StartupModule() override
	{
		PackageMarkedDirtyEventHandle = UPackage::PackageMarkedDirtyEvent.AddLambda(
			[](const UPackage* Package, bool bWasDirty)
			{
				if (Package)
				{
					UE::USDClasses::Private::PackagePathNameToDirtyCounter.FindOrAdd(Package->GetPathName())++;
				}
			}
		);

		PackageSavedWithContextEventHandle = UPackage::PackageSavedWithContextEvent.AddLambda(
			[](const FString& PackageFilename, UPackage* Package, FObjectPostSaveContext ObjectSaveContext)
			{
				if (Package)
				{
					UE::USDClasses::Private::PackagePathNameToDirtyCounter.Remove(Package->GetPathName());
				}
			}
		);
	}

	virtual void ShutdownModule() override
	{
		UPackage::PackageSavedWithContextEvent.Remove(PackageSavedWithContextEventHandle);
		UPackage::PackageMarkedDirtyEvent.Remove(PackageMarkedDirtyEventHandle);
	}

private:
	FDelegateHandle PackageMarkedDirtyEventHandle;
	FDelegateHandle PackageSavedWithContextEventHandle;
};

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FUsdClassesModule, USDClasses);
