// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDDrawModeComponent.h"

#include "SceneView.h"
#include "USDAssetUserData.h"
#include "USDMaterialUtils.h"
#include "USDProjectSettings.h"

#include "Components/BaseDynamicMeshSceneProxy.h"
#include "DynamicMeshBuilder.h"
#include "Engine/Engine.h"
#include "Engine/Texture2D.h"
#include "HitProxies.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialRenderProxy.h"
#include "PrimitiveSceneInfo.h"
#include "PrimitiveSceneProxy.h"
#include "PrimitiveViewRelevance.h"
#include "SceneInterface.h"
#include "PrimitiveDrawingUtils.h"
#include "UObject/Package.h"

#if WITH_EDITOR
#include "Editor/MaterialEditor/Public/MaterialEditingLibrary.h"
#endif	  // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(USDDrawModeComponent)

static bool GUseWholeExtentsToDetectSelection = true;
static FAutoConsoleVariableRef CVarUseWholeExtentsToDetectSelection(
	TEXT("USD.Bounds.UseWholeExtentsToDetectSelection"),
	GUseWholeExtentsToDetectSelection,
	TEXT("When true it means we can select the UUsdDrawModeComponents in 'bounds', 'cards' or 'origin' mode by clicking anywhere on the extent's "
		 "'volume' itself, and not only by clicking directly on the drawn lines")
);

static float GBoundsLineThickness = 2.0f;
static FAutoConsoleVariableRef CVarBoundsLineThickness(
	TEXT("USD.Bounds.BoundsLineThickness"),
	GBoundsLineThickness,
	TEXT("Line thickness to use when drawing 'bounds' or 'origin' alternate prim draw modes from UsdGeomModelAPI")
);

namespace UE::UsdDrawModeComponentImpl::Private
{
	// Simple proxy to draw just bounding box and coordinate axes lines
	class FUsdLinesSceneProxy : public FPrimitiveSceneProxy
	{
	public:
		FBox BoundsToDraw;
		FLinearColor BoundsColor;

		// Matrix to get a [0.5, 0.5] box centered on the origin (which is our mesh data)
		// to the position described by BoundsToDraw. Used to draw the hit proxy box
		FMatrix BoundsTransform;

		FMeshRenderBufferSet HitProxyBuffer;

		FMaterialRelevance MaterialRelevance;

	public:
		FUsdLinesSceneProxy(const UUsdDrawModeComponent* InComponent)
			: FPrimitiveSceneProxy(InComponent)
			, BoundsToDraw(FBox{InComponent->BoundsMin, InComponent->BoundsMax})
			, BoundsColor(InComponent->BoundsColor)
			, HitProxyBuffer(GetScene().GetFeatureLevel())
			, MaterialRelevance(UMaterial::GetDefaultMaterial(MD_Surface)->GetRelevance_Concurrent(GetScene().GetShaderPlatform()))
		{
			bWillEverBeLit = false;
			SetWireframeColor(FLinearColor(0.0f, 0.5f, 1.0f));

			const FVector Translation = BoundsToDraw.GetCenter();
			const FVector Scale = BoundsToDraw.GetSize();
			FTransform Trans{FQuat::Identity, Translation, Scale};
			BoundsTransform = Trans.ToMatrixWithScale();

			// Create a simple box that we can present during the hit testing pass to make sure that
			// our bounding boxes are easy to click
			{
				// Reference: PrimitiveDrawingUtils.cpp, DrawBox function

				// Calculate verts for a face pointing down Z
				FVector3f Positions[4] = {
					FVector3f(-0.5, -0.5, +0.5),
					FVector3f(-0.5, +0.5, +0.5),
					FVector3f(+0.5, +0.5, +0.5),
					FVector3f(+0.5, -0.5, +0.5)};

				FVector2f UVs[4] = {
					FVector2f(0, 0),
					FVector2f(0, 1),
					FVector2f(1, 1),
					FVector2f(1, 0),
				};

				// Then rotate this face 6 times
				FRotator3f FaceRotations[6];
				FaceRotations[0] = FRotator3f(0, 0, 0);
				FaceRotations[1] = FRotator3f(90.f, 0, 0);
				FaceRotations[2] = FRotator3f(-90.f, 0, 0);
				FaceRotations[3] = FRotator3f(0, 0, 90.f);
				FaceRotations[4] = FRotator3f(0, 0, -90.f);
				FaceRotations[5] = FRotator3f(180.f, 0, 0);

				const int NumTriangles = 2 * 6;
				const int NumVertices = 4 * 6;
				const int NumTexCoords = 1;
				HitProxyBuffer.PositionVertexBuffer.Init(NumVertices);
				HitProxyBuffer.StaticMeshVertexBuffer.Init(NumVertices, NumTexCoords);
				HitProxyBuffer.ColorVertexBuffer.Init(NumVertices);
				HitProxyBuffer.TriangleCount = NumTriangles;

				for (uint32 FaceIndex = 0; FaceIndex < 6; FaceIndex++)
				{
					FMatrix44f FaceTransform = FRotationMatrix44f(FaceRotations[FaceIndex]);

					uint32 NumVertsBeforeThisFace = FaceIndex * 4;

					for (uint32 VertexIndex = 0; VertexIndex < 4; VertexIndex++)
					{
						uint32 GlobalVertIndex = VertexIndex + NumVertsBeforeThisFace;

						HitProxyBuffer.PositionVertexBuffer.VertexPosition(GlobalVertIndex) = FaceTransform.TransformPosition(Positions[VertexIndex]);

						HitProxyBuffer.StaticMeshVertexBuffer.SetVertexTangents(
							GlobalVertIndex,
							FaceTransform.TransformVector(FVector3f(1, 0, 0)),
							FaceTransform.TransformVector(FVector3f(0, 1, 0)),
							FaceTransform.TransformVector(FVector3f(0, 0, 1))
						);

						const uint32 UVIndex = 0;
						HitProxyBuffer.StaticMeshVertexBuffer.SetVertexUV(GlobalVertIndex, UVIndex, UVs[VertexIndex]);

						HitProxyBuffer.ColorVertexBuffer.VertexColor(GlobalVertIndex) = FColor::White;
					}

					HitProxyBuffer.IndexBuffer.Indices.Append({NumVertsBeforeThisFace, NumVertsBeforeThisFace + 1, NumVertsBeforeThisFace + 2});
					HitProxyBuffer.IndexBuffer.Indices.Append({NumVertsBeforeThisFace, NumVertsBeforeThisFace + 2, NumVertsBeforeThisFace + 3});
				}

				HitProxyBuffer.Material = UMaterial::GetDefaultMaterial(MD_Surface);

				ENQUEUE_RENDER_COMMAND(FUsdCardsSceneProxyUploadBuffer)
				(
					[this](FRHICommandListImmediate& RHICmdList)
					{
						HitProxyBuffer.Upload();
					}
				);
			}
		}

		virtual void GetDynamicMeshElements(
			const TArray<const FSceneView*>& Views,
			const FSceneViewFamily& ViewFamily,
			uint32 VisibilityMap,
			FMeshElementCollector& Collector
		) const override
		{
			const FMatrix& Matrix = GetLocalToWorld();

			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				if (VisibilityMap & (1 << ViewIndex))
				{
					if (FPrimitiveDrawInterface* PDI = Collector.GetPDI(ViewIndex))
					{
						bool bIsWireframe = false;
						if (AllowDebugViewmodes())
						{
							bIsWireframe = ViewFamily.EngineShowFlags.Wireframe;

							// Render bounds for our bounds
							if (ViewFamily.EngineShowFlags.Bounds)
							{
								RenderBounds(PDI, ViewFamily.EngineShowFlags, GetBounds(), IsSelected());
							}
						}

						const uint8 DepthPriority = (uint8)GetDepthPriorityGroup(Views[ViewIndex]);

						// When hit testing we just draw a regular solid box so that we can select the component by
						// clicking anywhere within it
						const bool bIsHitTesting = ViewFamily.EngineShowFlags.HitProxies;
						if (bIsHitTesting && GUseWholeExtentsToDetectSelection)
						{
							bool bHasPrecomputedVolumetricLightmap;
							FMatrix PreviousLocalToWorld;
							int32 SingleCaptureIndex;
							bool bOutputVelocity;
							GetScene().GetPrimitiveUniformShaderParameters_RenderThread(
								GetPrimitiveSceneInfo(),
								bHasPrecomputedVolumetricLightmap,
								PreviousLocalToWorld,
								SingleCaptureIndex,
								bOutputVelocity
							);
							bOutputVelocity |= AlwaysHasVelocity();

							FDynamicPrimitiveUniformBuffer&
								DynamicPrimitiveUniformBuffer = Collector.AllocateOneFrameResource<FDynamicPrimitiveUniformBuffer>();

							DynamicPrimitiveUniformBuffer.Set(
								Collector.GetRHICommandList(),
								BoundsTransform * GetLocalToWorld(),
								PreviousLocalToWorld,
								GetBounds(),
								GetLocalBounds(),
								GetLocalBounds(),
								true,
								bHasPrecomputedVolumetricLightmap,
								bOutputVelocity,
								GetCustomPrimitiveData()
							);

							FMeshBatch& Mesh = Collector.AllocateMesh();
							Mesh.VertexFactory = &HitProxyBuffer.VertexFactory;
							Mesh.MaterialRenderProxy = HitProxyBuffer.Material ? HitProxyBuffer.Material->GetRenderProxy() : nullptr;
							Mesh.ReverseCulling = IsLocalToWorldDeterminantNegative();
							Mesh.Type = PT_TriangleList;
							Mesh.DepthPriorityGroup = DepthPriority;

							FMeshBatchElement& BatchElement = Mesh.Elements[0];
							BatchElement.IndexBuffer = &HitProxyBuffer.IndexBuffer;
							BatchElement.PrimitiveUniformBufferResource = &DynamicPrimitiveUniformBuffer.UniformBuffer;
							BatchElement.FirstIndex = 0;
							BatchElement.NumPrimitives = HitProxyBuffer.IndexBuffer.Indices.Num() / 3;
							BatchElement.MinVertexIndex = 0;
							BatchElement.MaxVertexIndex = HitProxyBuffer.PositionVertexBuffer.GetNumVertices() - 1;
							Collector.AddMesh(ViewIndex, Mesh);
						}
						else
						{
							DrawLines(
								PDI,
								Matrix,
								bIsWireframe ? GetWireframeColor() : BoundsColor,
								DepthPriority,
								bIsWireframe ? 0.0f : GBoundsLineThickness
							);
						}
					}
				}
			}
		}

		// Actually does the line drawing
		virtual void DrawLines(
			FPrimitiveDrawInterface* PDI,
			const FMatrix& LocalToWorld,
			const FLinearColor& LineColor,
			uint8 DepthPriority,
			float LineThickness
		) const = 0;

		virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
		{
			FPrimitiveViewRelevance ViewRelevance;
			ViewRelevance.bDrawRelevance = IsShown(View);
			ViewRelevance.bDynamicRelevance = true;
			ViewRelevance.bNormalTranslucency = true;
			return ViewRelevance;
		}

		virtual uint32 GetMemoryFootprint(void) const override
		{
			return (sizeof(*this) + GetAllocatedSize());
		}

		SIZE_T GetTypeHash() const override
		{
			static size_t UniquePointer;
			return reinterpret_cast<size_t>(&UniquePointer);
		}
	};

	// Version of the FUsdLinesSceneProxy proxy that draws a bounding box
	class FUsdDrawModeLinesSceneProxy : public FUsdLinesSceneProxy
	{
	public:
		using FUsdLinesSceneProxy::FUsdLinesSceneProxy;

		virtual void DrawLines(
			FPrimitiveDrawInterface* PDI,
			const FMatrix& LocalToWorldMatrix,
			const FLinearColor& LineColor,
			uint8 DepthPriority,
			float LineThickness
		) const override
		{
			DrawWireBox(PDI, LocalToWorldMatrix, BoundsToDraw, LineColor, DepthPriority, LineThickness);
		}
	};

	// Version of the FUsdLinesSceneProxy proxy that draws XYZ coordinate axes (always in the UE coordinate system)
	class FUsdOriginLinesSceneProxy : public FUsdLinesSceneProxy
	{
	public:
		using FUsdLinesSceneProxy::FUsdLinesSceneProxy;

		virtual void DrawLines(
			FPrimitiveDrawInterface* PDI,
			const FMatrix& LocalToWorldMatrix,
			const FLinearColor& LineColor,
			uint8 DepthPriority,
			float LineThickness
		) const override
		{
			const FTransform World{LocalToWorldMatrix};
			const FVector AxesLengths = BoundsToDraw.Max;

			// In theory we should draw this with the wireframe color in wireframe mode but maybe it's more useful to have RGB XYZ colors still?
			// Note: The DrawCoordinateSystem function does exist, but we don't use it here as it has a uniform scaling for all axes
			// and it may be useful to show the axes scaled as usual given the prim's transform (i.e. doing this if the prims is
			// squashed on the Z we will see a squashed Z line, while DrawCoordinateSystem would give the same line lengths for all
			// axes)
			PDI->DrawLine(
				World.GetLocation(),
				World.GetLocation() + World.TransformVector(FVector{AxesLengths.X, 0.0, 0.0}),
				FLinearColor::Red,
				DepthPriority,
				LineThickness
			);
			PDI->DrawLine(
				World.GetLocation(),
				World.GetLocation() + World.TransformVector(FVector{0.0, AxesLengths.Y, 0.0}),
				FLinearColor::Green,
				DepthPriority,
				LineThickness
			);
			PDI->DrawLine(
				World.GetLocation(),
				World.GetLocation() + World.TransformVector(FVector{0.0, 0.0, AxesLengths.Z}),
				FLinearColor::Blue,
				DepthPriority,
				LineThickness
			);
		}
	};

	// Proxy capable of drawing meshes with materials and textures.
	// Dynamic meshes via GetDynamicMeshElements can only have flat colors, so we need a slightly more complicated proxy.
	// Reference: FDynamicMeshSceneProxy,
	class FUsdCardsSceneProxy final : public FPrimitiveSceneProxy
	{
	public:

		// Matrix to get a [0.5, 0.5] box centered on the origin (which is our mesh data)
		// to the position described by BoundsToDraw
		FMatrix BoundsTransform;

		TArray<FMeshRenderBufferSet, TInlineAllocator<6>> Buffers;
		TArray<FMaterialRelevance, TInlineAllocator<6>> MaterialRelevances;

	public:
		FUsdCardsSceneProxy(const UUsdDrawModeComponent* InComponent, const TArray<UMaterialInstance*>& MaterialInstances)
			: FPrimitiveSceneProxy(InComponent)
		{
			bWillEverBeLit = true;
			SetWireframeColor(FLinearColor(0.0f, 0.5f, 1.0f));

			FBox BoundsToDraw = FBox{InComponent->BoundsMin, InComponent->BoundsMax};
			const FVector Translation = BoundsToDraw.GetCenter();
			const FVector Scale = BoundsToDraw.GetSize();
			FTransform Trans{FQuat::Identity, Translation, Scale};
			BoundsTransform = Trans.ToMatrixWithScale();

			float OffsetAlongAxis = InComponent->CardGeometry == EUsdModelCardGeometry::Cross ? 0.0f : 0.5f;

			// Weirdly enough given the drawMode rules when we have EUsdModelCardFace::None here we draw *all* faces, except we will just be using
			// DisplayColor materials on all of them
			const bool bDrawingAllWithDisplayColor = InComponent->GetAuthoredFaces() == EUsdModelCardFace::None;

			for (int32 FaceIndex = 0; FaceIndex < 6; ++FaceIndex)
			{
				EUsdModelCardFace Face = (EUsdModelCardFace)(1 << FaceIndex);
				EUsdModelCardFace OppositeFace = UsdUtils::GetOppositeFaceOnSameAxis(Face);

				// If either face on an axis is enabled, we need to draw both
				const bool bFaceIsAuthored = EnumHasAllFlags(InComponent->GetAuthoredFaces(), Face);
				const bool bOppositeFaceIsAuthored = EnumHasAllFlags(InComponent->GetAuthoredFaces(), OppositeFace);
				if (!bFaceIsAuthored && !bOppositeFaceIsAuthored && !bDrawingAllWithDisplayColor)
				{
					continue;
				}

				FMeshRenderBufferSet& Buffer = Buffers.Emplace_GetRef(GetScene().GetFeatureLevel());

				const int NumTriangles = 2;
				const int NumVertices = 4;

				FStaticMeshBuildVertex VertexBuildData[NumVertices];

				switch (Face)
				{
					case EUsdModelCardFace::XPos:
					{
						VertexBuildData[0] = { .Position = {OffsetAlongAxis, -0.5, -0.5}, .TangentX = {0, 1, 0}, .TangentY = {0, 0, 1}, .TangentZ = {1, 0, 0} };
						VertexBuildData[1] = { .Position = {OffsetAlongAxis, -0.5, +0.5}, .TangentX = {0, 1, 0}, .TangentY = {0, 0, 1}, .TangentZ = {1, 0, 0} };
						VertexBuildData[2] = { .Position = {OffsetAlongAxis, +0.5, +0.5}, .TangentX = {0, 1, 0}, .TangentY = {0, 0, 1}, .TangentZ = {1, 0, 0} };
						VertexBuildData[3] = { .Position = {OffsetAlongAxis, +0.5, -0.5}, .TangentX = {0, 1, 0}, .TangentY = {0, 0, 1}, .TangentZ = {1, 0, 0} };
						break;
					}
					case EUsdModelCardFace::YPos:
					{
						VertexBuildData[0] = { .Position = {+0.5, OffsetAlongAxis, -0.5}, .TangentX = {0, 0, 1}, .TangentY = {1, 0, 0}, .TangentZ = {0, 1, 0} };
						VertexBuildData[1] = { .Position = {+0.5, OffsetAlongAxis, +0.5}, .TangentX = {0, 0, 1}, .TangentY = {1, 0, 0}, .TangentZ = {0, 1, 0} };
						VertexBuildData[2] = { .Position = {-0.5, OffsetAlongAxis, +0.5}, .TangentX = {0, 0, 1}, .TangentY = {1, 0, 0}, .TangentZ = {0, 1, 0} };
						VertexBuildData[3] = { .Position = {-0.5, OffsetAlongAxis, -0.5}, .TangentX = {0, 0, 1}, .TangentY = {1, 0, 0}, .TangentZ = {0, 1, 0} };
						break;
					}
					case EUsdModelCardFace::ZPos:
					{
						VertexBuildData[0] = { .Position = {+0.5, +0.5, OffsetAlongAxis}, .TangentX = {1, 0, 0}, .TangentY = {0, 1, 0}, .TangentZ = {0, 0, 1} };
						VertexBuildData[1] = { .Position = {+0.5, -0.5, OffsetAlongAxis}, .TangentX = {1, 0, 0}, .TangentY = {0, 1, 0}, .TangentZ = {0, 0, 1} };
						VertexBuildData[2] = { .Position = {-0.5, -0.5, OffsetAlongAxis}, .TangentX = {1, 0, 0}, .TangentY = {0, 1, 0}, .TangentZ = {0, 0, 1} };
						VertexBuildData[3] = { .Position = {-0.5, +0.5, OffsetAlongAxis}, .TangentX = {1, 0, 0}, .TangentY = {0, 1, 0}, .TangentZ = {0, 0, 1} };
						break;
					}
					case EUsdModelCardFace::XNeg:
					{
						VertexBuildData[0] = { .Position = {-OffsetAlongAxis, +0.5, -0.5}, .TangentX = {0, 0, 1}, .TangentY = {0, 1, 1}, .TangentZ = {-1, 0, 0} };
						VertexBuildData[1] = { .Position = {-OffsetAlongAxis, +0.5, +0.5}, .TangentX = {0, 0, 1}, .TangentY = {0, 1, 1}, .TangentZ = {-1, 0, 0} };
						VertexBuildData[2] = { .Position = {-OffsetAlongAxis, -0.5, +0.5}, .TangentX = {0, 0, 1}, .TangentY = {0, 1, 1}, .TangentZ = {-1, 0, 0} };
						VertexBuildData[3] = { .Position = {-OffsetAlongAxis, -0.5, -0.5}, .TangentX = {0, 0, 1}, .TangentY = {0, 1, 1}, .TangentZ = {-1, 0, 0} };
						break;
					}
					case EUsdModelCardFace::YNeg:
					{
						VertexBuildData[0] = { .Position = {-0.5, -OffsetAlongAxis, -0.5}, .TangentX = {0, 0, 1}, .TangentY = {-1, 0, 0}, .TangentZ = {0, -1, 0} };
						VertexBuildData[1] = { .Position = {-0.5, -OffsetAlongAxis, +0.5}, .TangentX = {0, 0, 1}, .TangentY = {-1, 0, 0}, .TangentZ = {0, -1, 0} };
						VertexBuildData[2] = { .Position = {+0.5, -OffsetAlongAxis, +0.5}, .TangentX = {0, 0, 1}, .TangentY = {-1, 0, 0}, .TangentZ = {0, -1, 0} };
						VertexBuildData[3] = { .Position = {+0.5, -OffsetAlongAxis, -0.5}, .TangentX = {0, 0, 1}, .TangentY = {-1, 0, 0}, .TangentZ = {0, -1, 0} };
						break;
					}
					case EUsdModelCardFace::ZNeg:
					{
						VertexBuildData[0] = { .Position = {+0.5, -0.5, -OffsetAlongAxis}, .TangentX = {1, 0, 0}, .TangentY = {0, -1, 0}, .TangentZ = {0, 0, -1} };
						VertexBuildData[1] = { .Position = {+0.5, +0.5, -OffsetAlongAxis}, .TangentX = {1, 0, 0}, .TangentY = {0, -1, 0}, .TangentZ = {0, 0, -1} };
						VertexBuildData[2] = { .Position = {-0.5, +0.5, -OffsetAlongAxis}, .TangentX = {1, 0, 0}, .TangentY = {0, -1, 0}, .TangentZ = {0, 0, -1} };
						VertexBuildData[3] = { .Position = {-0.5, -0.5, -OffsetAlongAxis}, .TangentX = {1, 0, 0}, .TangentY = {0, -1, 0}, .TangentZ = {0, 0, -1} };
						break;
					}
					default:
					{
						ensure(false);
						break;
					}
				};

				// This face is not authored but we're going to be drawing something for it --> Flip UVs horizontally.
				// We don't know if our material is a texture material from the opposite face or a DisplayColor,
				// but if it's the former we're doing the right thing by flipping the UV, and if it's the latter it doesn't matter!
				const bool bFlipHorizontally = !bFaceIsAuthored;
				if (bFlipHorizontally)
				{
					VertexBuildData[0].UVs[0] = {0, 1};
					VertexBuildData[1].UVs[0] = {0, 0};
					VertexBuildData[2].UVs[0] = {1, 0};
					VertexBuildData[3].UVs[0] = {1, 1};
				}
				else
				{
					VertexBuildData[0].UVs[0] = {1, 1};
					VertexBuildData[1].UVs[0] = {1, 0};
					VertexBuildData[2].UVs[0] = {0, 0};
					VertexBuildData[3].UVs[0] = {0, 1};
				}

				const bool bSRGB = true;
				const FColor BoundsColor = InComponent->BoundsColor.ToFColor(bSRGB);

				VertexBuildData[0].Color = BoundsColor;
				VertexBuildData[1].Color = BoundsColor;
				VertexBuildData[2].Color = BoundsColor;
				VertexBuildData[3].Color = BoundsColor;

				// MakeConstMeshBuildVertexView adds 8 (MAX_STATIC_TEXCOORDS) UVs instead of just the 1 we want.
				const FConstMeshBuildVertexView VertexView
				{
					.Position = MakeConstStridedView(VertexBuildData, &FStaticMeshBuildVertex::Position),
					.TangentX = MakeConstStridedView(VertexBuildData, &FStaticMeshBuildVertex::TangentX),
					.TangentY = MakeConstStridedView(VertexBuildData, &FStaticMeshBuildVertex::TangentY),
					.TangentZ = MakeConstStridedView(VertexBuildData, &FStaticMeshBuildVertex::TangentZ),
					.UVs = { MakeConstStridedView(sizeof(FStaticMeshBuildVertex), &VertexBuildData[0].UVs[0], NumVertices) },
					.Color = MakeConstStridedView(VertexBuildData, &FStaticMeshBuildVertex::Color),
				};

				Buffer.StaticMeshVertexBuffer.Init(VertexView);
				Buffer.PositionVertexBuffer.Init(VertexView);
				Buffer.ColorVertexBuffer.Init(VertexView);

				Buffer.IndexBuffer.Indices = {0, 1, 2, 0, 2, 3};
				Buffer.TriangleCount = NumTriangles;

				Buffer.Material = MaterialInstances[FaceIndex];

				// The component should always have a material for us, even if just the default material
				if (!ensure(Buffer.Material))
				{
					return;
				}

				MaterialRelevances.Emplace(Buffer.Material->GetRelevance_Concurrent(GetScene().GetShaderPlatform()));

				ENQUEUE_RENDER_COMMAND(FUsdCardsSceneProxyUploadBuffer)
				(
					[&Buffer](FRHICommandListImmediate& RHICmdList)
					{
						Buffer.Upload();
					}
				);
			}
		}

		virtual void GetDynamicMeshElements(
			const TArray<const FSceneView*>& Views,
			const FSceneViewFamily& ViewFamily,
			uint32 VisibilityMap,
			FMeshElementCollector& Collector
		) const override
		{
			bool bHasPrecomputedVolumetricLightmap;
			FMatrix PreviousLocalToWorld;
			int32 SingleCaptureIndex;
			bool bOutputVelocity;
			GetScene().GetPrimitiveUniformShaderParameters_RenderThread(
				GetPrimitiveSceneInfo(),
				bHasPrecomputedVolumetricLightmap,
				PreviousLocalToWorld,
				SingleCaptureIndex,
				bOutputVelocity
			);
			bOutputVelocity |= AlwaysHasVelocity();

			FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer = Collector.AllocateOneFrameResource<FDynamicPrimitiveUniformBuffer>();

			DynamicPrimitiveUniformBuffer.Set(
				Collector.GetRHICommandList(),
				BoundsTransform * GetLocalToWorld(),
				PreviousLocalToWorld,
				GetBounds(),
				GetLocalBounds(),
				GetLocalBounds(),
				true,
				bHasPrecomputedVolumetricLightmap,
				bOutputVelocity,
				GetCustomPrimitiveData()
			);

			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				if (VisibilityMap & (1 << ViewIndex))
				{
					if (FPrimitiveDrawInterface* PDI = Collector.GetPDI(ViewIndex))
					{
						FMaterialRenderProxy* OverrideMaterialProxy = nullptr;

						if (AllowDebugViewmodes())
						{
							if (ViewFamily.EngineShowFlags.Wireframe)
							{
								OverrideMaterialProxy = new FColoredMaterialRenderProxy(
									GEngine->WireframeMaterial ? GEngine->WireframeMaterial->GetRenderProxy() : nullptr,
									GetWireframeColor()
								);
								Collector.RegisterOneFrameMaterialProxy(OverrideMaterialProxy);
							}

							if (ViewFamily.EngineShowFlags.Bounds)
							{
								RenderBounds(PDI, ViewFamily.EngineShowFlags, GetBounds(), IsSelected());
							}
						}

						for (const FMeshRenderBufferSet& Buffer : Buffers)
						{
							FMeshBatch& Mesh = Collector.AllocateMesh();
							Mesh.VertexFactory = &Buffer.VertexFactory;
							Mesh.MaterialRenderProxy = OverrideMaterialProxy ? OverrideMaterialProxy
													   : Buffer.Material	 ? Buffer.Material->GetRenderProxy()
																			 : nullptr;
							Mesh.ReverseCulling = IsLocalToWorldDeterminantNegative();
							Mesh.Type = PT_TriangleList;
							Mesh.DepthPriorityGroup = (uint8)GetDepthPriorityGroup(Views[ViewIndex]);

							FMeshBatchElement& BatchElement = Mesh.Elements[0];
							BatchElement.IndexBuffer = &Buffer.IndexBuffer;
							BatchElement.PrimitiveUniformBufferResource = &DynamicPrimitiveUniformBuffer.UniformBuffer;
							BatchElement.FirstIndex = 0;
							BatchElement.NumPrimitives = Buffer.IndexBuffer.Indices.Num() / 3;
							BatchElement.MinVertexIndex = 0;
							BatchElement.MaxVertexIndex = Buffer.PositionVertexBuffer.GetNumVertices() - 1;
							Collector.AddMesh(ViewIndex, Mesh);
						}
					}
				}
			}
		}

		virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
		{
			// Reference: DynamicMeshSceneProxy::GetViewRelevance

			FPrimitiveViewRelevance Result;
			Result.bDrawRelevance = IsShown(View);
			Result.bShadowRelevance = IsShadowCast(View);
			Result.bDynamicRelevance = true;
			Result.bRenderInMainPass = ShouldRenderInMainPass();
			Result.bUsesLightingChannels = GetLightingChannelMask() != GetDefaultLightingChannelMask();
			Result.bTranslucentSelfShadow = bCastVolumetricTranslucentShadow;
			Result.bRenderCustomDepth = ShouldRenderCustomDepth();
			Result.bVelocityRelevance = DrawsVelocity() && Result.bOpaque && Result.bRenderInMainPass;

			if (MaterialRelevances.IsValidIndex(0))
			{
				MaterialRelevances[0].SetPrimitiveViewRelevance(Result);
			}
			return Result;
		}

		virtual bool CanBeOccluded() const override
		{
			if (MaterialRelevances.IsValidIndex(0))
			{
				return !MaterialRelevances[0].bDisableDepthTest;
			}

			return true;
		}

		virtual uint32 GetMemoryFootprint(void) const override
		{
			return (sizeof(*this) + GetAllocatedSize());
		}

		SIZE_T GetTypeHash() const override
		{
			static size_t UniquePointer;
			return reinterpret_cast<size_t>(&UniquePointer);
		}
	};

	int32 FaceToFaceIndex(EUsdModelCardFace Face)
	{
		ensure(Face != EUsdModelCardFace::None);
		return FMath::CountTrailingZeros((int32)Face);
	}

	void SetTextureParameter(UMaterialInstance* Instance, UTexture2D* Texture)
	{
		FMaterialParameterInfo Info;
		Info.Name = TEXT("Texture");

		if (UMaterialInstanceDynamic* Dynamic = Cast<UMaterialInstanceDynamic>(Instance))
		{
			Dynamic->SetTextureParameterValueByInfo(Info, Texture);
		}
		else
		{
			// We should always create MIDs on-demand now, and never a MIC
			ensure(false);
		}
	}
};	  // namespace UE::UsdDrawModeComponentImpl::Private

EUsdModelCardFace UsdUtils::GetOppositeFaceOnSameAxis(EUsdModelCardFace Face)
{
	switch (Face)
	{
		case EUsdModelCardFace::XPos:
		{
			return EUsdModelCardFace::XNeg;
		}
		case EUsdModelCardFace::YPos:
		{
			return EUsdModelCardFace::YNeg;
		}
		case EUsdModelCardFace::ZPos:
		{
			return EUsdModelCardFace::ZNeg;
		}
		case EUsdModelCardFace::XNeg:
		{
			return EUsdModelCardFace::XPos;
		}
		case EUsdModelCardFace::YNeg:
		{
			return EUsdModelCardFace::YPos;
		}
		case EUsdModelCardFace::ZNeg:
		{
			return EUsdModelCardFace::ZPos;
		}
		default:
		{
			ensure(false);
			break;
		}
	}

	return EUsdModelCardFace::None;
}

void UUsdDrawModeComponent::SetBoundsMin(const FVector& NewMin)
{
	Modify();
	BoundsMin = NewMin;
	MarkRenderStateDirty();
}

void UUsdDrawModeComponent::SetBoundsMax(const FVector& NewMax)
{
	Modify();
	BoundsMax = NewMax;
	MarkRenderStateDirty();
}

void UUsdDrawModeComponent::SetDrawMode(EUsdDrawMode NewDrawMode)
{
	Modify();
	DrawMode = NewDrawMode;
	MarkRenderStateDirty();
}

void UUsdDrawModeComponent::SetBoundsColor(FLinearColor NewColor)
{
	Modify();
	BoundsColor = NewColor;
	MarkRenderStateDirty();
}

void UUsdDrawModeComponent::SetCardGeometry(EUsdModelCardGeometry NewGeometry)
{
	Modify();
	CardGeometry = NewGeometry;
	MarkRenderStateDirty();
}

void UUsdDrawModeComponent::SetCardTextureXPos(UTexture2D* NewTexture)
{
	Modify();
	// We have this logic so that clearing the texture on the details panel means
	// removing the opinion of that face texture in the USD stage
	// (when we convert the component back to USD we'll clear faces that aren't in AuthoredFaces).
	if (NewTexture)
	{
		AuthoredFaces |= (int32)EUsdModelCardFace::XPos;
	}
	else
	{
		AuthoredFaces &= (int32)~EUsdModelCardFace::XPos;
	}

	CardTextureXPos = NewTexture;
	MarkRenderStateDirty();
}

void UUsdDrawModeComponent::SetCardTextureYPos(UTexture2D* NewTexture)
{
	Modify();
	if (NewTexture)
	{
		AuthoredFaces |= (int32)EUsdModelCardFace::YPos;
	}
	else
	{
		AuthoredFaces &= (int32)~EUsdModelCardFace::YPos;
	}

	CardTextureYPos = NewTexture;
	MarkRenderStateDirty();
}

void UUsdDrawModeComponent::SetCardTextureZPos(UTexture2D* NewTexture)
{
	Modify();
	if (NewTexture)
	{
		AuthoredFaces |= (int32)EUsdModelCardFace::ZPos;
	}
	else
	{
		AuthoredFaces &= (int32)~EUsdModelCardFace::ZPos;
	}

	CardTextureZPos = NewTexture;
	MarkRenderStateDirty();
}

void UUsdDrawModeComponent::SetCardTextureXNeg(UTexture2D* NewTexture)
{
	Modify();
	if (NewTexture)
	{
		AuthoredFaces |= (int32)EUsdModelCardFace::XNeg;
	}
	else
	{
		AuthoredFaces &= (int32)~EUsdModelCardFace::XNeg;
	}

	CardTextureXNeg = NewTexture;
	MarkRenderStateDirty();
}

void UUsdDrawModeComponent::SetCardTextureYNeg(UTexture2D* NewTexture)
{
	Modify();
	if (NewTexture)
	{
		AuthoredFaces |= (int32)EUsdModelCardFace::YNeg;
	}
	else
	{
		AuthoredFaces &= (int32)~EUsdModelCardFace::YNeg;
	}

	CardTextureYNeg = NewTexture;
	MarkRenderStateDirty();
}

void UUsdDrawModeComponent::SetCardTextureZNeg(UTexture2D* NewTexture)
{
	Modify();
	if (NewTexture)
	{
		AuthoredFaces |= (int32)EUsdModelCardFace::ZNeg;
	}
	else
	{
		AuthoredFaces &= (int32)~EUsdModelCardFace::ZNeg;
	}

	CardTextureZNeg = NewTexture;
	MarkRenderStateDirty();
}

UTexture2D* UUsdDrawModeComponent::GetTextureForFace(EUsdModelCardFace Face) const
{
	switch (Face)
	{
		case EUsdModelCardFace::XPos:
		{
			return CardTextureXPos;
		}
		case EUsdModelCardFace::YPos:
		{
			return CardTextureYPos;
		}
		case EUsdModelCardFace::ZPos:
		{
			return CardTextureZPos;
		}
		case EUsdModelCardFace::XNeg:
		{
			return CardTextureXNeg;
		}
		case EUsdModelCardFace::YNeg:
		{
			return CardTextureYNeg;
		}
		case EUsdModelCardFace::ZNeg:
		{
			return CardTextureZNeg;
		}
		default:
		{
			break;
		}
	}

	return nullptr;
}

void UUsdDrawModeComponent::SetTextureForFace(EUsdModelCardFace Face, UTexture2D* Texture)
{
	switch (Face)
	{
		case EUsdModelCardFace::XPos:
		{
			SetCardTextureXPos(Texture);
			break;
		}
		case EUsdModelCardFace::YPos:
		{
			SetCardTextureYPos(Texture);
			break;
		}
		case EUsdModelCardFace::ZPos:
		{
			SetCardTextureZPos(Texture);
			break;
		}
		case EUsdModelCardFace::XNeg:
		{
			SetCardTextureXNeg(Texture);
			break;
		}
		case EUsdModelCardFace::YNeg:
		{
			SetCardTextureYNeg(Texture);
			break;
		}
		case EUsdModelCardFace::ZNeg:
		{
			SetCardTextureZNeg(Texture);
			break;
		}
		default:
		{
			break;
		}
	}
}

UUsdDrawModeComponent::UUsdDrawModeComponent()
	: BoundsMin{FVector{-0.5}}
	, BoundsMax{FVector{0.5}}
	, DrawMode{EUsdDrawMode::Bounds}
	, BoundsColor{FLinearColor::White}
	, CardGeometry{EUsdModelCardGeometry::Cross}
	, CardTextureXPos{nullptr}
	, CardTextureYPos{nullptr}
	, CardTextureZPos{nullptr}
	, CardTextureXNeg{nullptr}
	, CardTextureYNeg{nullptr}
	, CardTextureZNeg{nullptr}
	, MaterialInstances({nullptr, nullptr, nullptr, nullptr, nullptr, nullptr})
	, AuthoredFaces{(int32)EUsdModelCardFace::None}
{
	if (IsTemplate())
	{
		return;
	}

	RefreshMaterialInstances();
}

FPrimitiveSceneProxy* UUsdDrawModeComponent::CreateSceneProxy()
{
	using namespace UE::UsdDrawModeComponentImpl::Private;

	// We call this here because changing a property on the editor doesn't call our BlueprintSetter
	// functions, but nevertheless still triggers a PostEditChange event that invalidates our render
	// proxy anyway. This means that if we delay refreshing the material instances in here it ends up
	// called in all cases and we always get textures/materials that are up-to-date
	RefreshMaterialInstances();

	// The render thread owns and deletes these, so we can just reallocate every time.
	// CreateSceneProxy gets called when we call MarkRenderStateDirty()
	switch (DrawMode)
	{
		case EUsdDrawMode::Origin:
		{
			return new FUsdOriginLinesSceneProxy(this);
		}
		case EUsdDrawMode::Bounds:
		{
			return new FUsdDrawModeLinesSceneProxy(this);
		}
		case EUsdDrawMode::Cards:
		{
			return new FUsdCardsSceneProxy(this, MaterialInstances);
		}
		case EUsdDrawMode::Default:
		case EUsdDrawMode::Inherited:
		default:
		{
			// We shouldn't have an UUsdDrawModeComponent if we have these draw modes...
			ensure(false);
			break;
		}
	}

	return nullptr;
}

FBoxSphereBounds UUsdDrawModeComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	FBox LocalSpaceBox{BoundsMin, BoundsMax};
	LocalSpaceBox = LocalSpaceBox.TransformBy(LocalToWorld);
	return FBoxSphereBounds(LocalSpaceBox);
}

void UUsdDrawModeComponent::GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials) const
{
	OutMaterials.Reset(MaterialInstances.Num());
	for (UMaterialInstance* Instance : MaterialInstances)
	{
		if (Instance)
		{
			OutMaterials.Add(Instance);
		}
	}
}

#if WITH_EDITOR
void UUsdDrawModeComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// Redirect our property change events (e.g. editing the value directly on the Details panel) to
	// the blueprint setters, so that they can also e.g. trigger changes to AuthoredFaces.
	// In theory we only need this for the card face texture properties, but let's just do it for all
	// of them anyway for consistency

	const FName PropertyName = PropertyChangedEvent.GetMemberPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UUsdDrawModeComponent, BoundsMin))
	{
		SetBoundsMin(BoundsMin);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UUsdDrawModeComponent, BoundsMax))
	{
		SetBoundsMax(BoundsMax);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UUsdDrawModeComponent, DrawMode))
	{
		SetDrawMode(DrawMode);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UUsdDrawModeComponent, BoundsColor))
	{
		SetBoundsColor(BoundsColor);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UUsdDrawModeComponent, CardGeometry))
	{
		SetCardGeometry(CardGeometry);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UUsdDrawModeComponent, CardTextureXPos))
	{
		SetCardTextureXPos(CardTextureXPos);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UUsdDrawModeComponent, CardTextureYPos))
	{
		SetCardTextureYPos(CardTextureYPos);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UUsdDrawModeComponent, CardTextureZPos))
	{
		SetCardTextureZPos(CardTextureZPos);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UUsdDrawModeComponent, CardTextureXNeg))
	{
		SetCardTextureXNeg(CardTextureXNeg);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UUsdDrawModeComponent, CardTextureYNeg))
	{
		SetCardTextureYNeg(CardTextureYNeg);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UUsdDrawModeComponent, CardTextureZNeg))
	{
		SetCardTextureZNeg(CardTextureZNeg);
	}

	// Propagate this only after our changes, because that is what will cause AUsdStageActor::OnObjectPropertyChanged
	// to respond, and we want to write out to USD only after our blueprint setters have had the chance to tweak
	// AuthoredFaces
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

HHitProxy* UUsdDrawModeComponent::CreateMeshHitProxy(int32 SectionIndex, int32 MaterialIndex) const
{
	return nullptr;
}
#endif	  // WITH_EDITOR

EUsdModelCardFace UUsdDrawModeComponent::GetAuthoredFaces() const
{
	return (EUsdModelCardFace)AuthoredFaces;
}

void UUsdDrawModeComponent::SetAuthoredFaces(EUsdModelCardFace NewAuthoredFaces)
{
	Modify();
	AuthoredFaces = (int32)NewAuthoredFaces;
	MarkRenderStateDirty();
}

UMaterialInstance* UUsdDrawModeComponent::GetOrCreateTextureMaterial(EUsdModelCardFace Face)
{
	if (Face == EUsdModelCardFace::None)
	{
		return nullptr;
	}

	int32 FaceIndex = UE::UsdDrawModeComponentImpl::Private::FaceToFaceIndex(Face);
	if (!ensure(MaterialInstances.IsValidIndex(FaceIndex)))
	{
		return nullptr;
	}

	const UUsdProjectSettings* ProjectSettings = GetDefault<UUsdProjectSettings>();
	if (!ProjectSettings)
	{
		return nullptr;
	}

	TObjectPtr<UMaterialInstance>& Instance = MaterialInstances[FaceIndex];

	// Need to create a new instance of the texture material for this face
	// TODO: If the texture is really large UE may automatically make it into a VT texture, and our material with a
	// regular texture samplers won't really work. Should we have a "VT upgrade" mechanism? Is that even relevant for these
	// "card" textures? (who is using 4K+ texture cards?)
	if (!Instance || FSoftObjectPath{Instance->Parent} != ProjectSettings->ReferenceModelCardTextureMaterial)
	{
		if (UMaterialInterface* ParentMaterial = Cast<UMaterialInterface>(ProjectSettings->ReferenceModelCardTextureMaterial.TryLoad()))
		{
			UObject* Outer = this;
			if (UMaterialInstanceDynamic* NewMID = UMaterialInstanceDynamic::Create(ParentMaterial, Outer, NAME_None))
			{
				Instance = NewMID;
			}
		}
	}

	return Instance;
}

void UUsdDrawModeComponent::RefreshMaterialInstances()
{
	// We must follow the following rules (some of these from
	// https://openusd.org/release/api/class_usd_geom_model_a_p_i.html#UsdGeomModelAPI_drawMode):
	// - If no texture is set (i.e. if AuthoredFaces is None) then draw all faces with BoundsColor. Otherwise just draw the axes set with textures;
	// - If a texture is present for an axis, but the back face texture is missing, use the same texture flipped horizontally
	//   (the proxy will handle that);
	// - If a texture is set but fails to resolve (i.e. AuthoredFaces is set but the texture is nullptr) then draw that face with bounds color
	//   (we will do this by using a DisplayColor material instead);

	using namespace UE::UsdDrawModeComponentImpl::Private;

	const bool bDrawingAllWithDisplayColor = (EUsdModelCardFace)AuthoredFaces == EUsdModelCardFace::None;

	TSet<int32, DefaultKeyFuncs<int32>, TInlineSetAllocator<8>> IndicesToSetDrawModeColor;
	for (int32 FaceIndex = 0; FaceIndex < 6; ++FaceIndex)
	{
		if (bDrawingAllWithDisplayColor)
		{
			IndicesToSetDrawModeColor.Add(FaceIndex);
			continue;
		}

		EUsdModelCardFace Face = (EUsdModelCardFace)(1 << FaceIndex);
		UTexture2D* Texture = GetTextureForFace(Face);
		const bool bDrawFace = EnumHasAllFlags((EUsdModelCardFace)AuthoredFaces, Face);

		EUsdModelCardFace OppositeFace = UsdUtils::GetOppositeFaceOnSameAxis(Face);
		UTexture2D* OppositeTexture = GetTextureForFace(OppositeFace);
		const bool bDrawOppositeFace = EnumHasAllFlags((EUsdModelCardFace)AuthoredFaces, OppositeFace);

		// Face is authored and we have a texture --> Just draw that texture
		if (bDrawFace && Texture)
		{
			SetTextureParameter(GetOrCreateTextureMaterial(Face), Texture);
		}
		// Face is authored but we don't have a texture --> Failed to resolve our texture --> Draw DisplayColor
		else if (bDrawFace)
		{
			IndicesToSetDrawModeColor.Add(FaceIndex);
		}
		// Face is not authored, but the opposite face is and it has a texture --> Draw that texture (proxy will flip it horizontally)
		else if (!bDrawFace && bDrawOppositeFace && OppositeTexture)
		{
			SetTextureParameter(GetOrCreateTextureMaterial(Face), OppositeTexture);
		}
		// Face is not authored, but the opposite face is. It doesn't have a texture though. (+ other cases) --> Draw DisplayColor
		else
		{
			IndicesToSetDrawModeColor.Add(FaceIndex);
		}
	}

	UMaterialInstance* DisplayColorInstance = nullptr;
	if (IndicesToSetDrawModeColor.Num() > 0)
	{
		// Check if we already have any DisplayColor materials we can reuse. These don't
		// have any parameters so there's no reason at all to have instances for each.
		// Actually we don't even need any material instance at all really... although using the
		// plugin material directly on the component is probably not the best idea
		if (const UUsdProjectSettings* Settings = GetDefault<UUsdProjectSettings>())
		{
			for (UMaterialInstance* Instance : MaterialInstances)
			{
				if (!Instance)
				{
					continue;
				}

				// We only ever spawn opaque one-sided display colors for the cards
				if (FSoftObjectPath{Instance->Parent} == Settings->ReferenceDisplayColorMaterial)
				{
					DisplayColorInstance = Instance;
					break;
				}
			}
		}

		// Create a new display color instance
		if (!DisplayColorInstance)
		{
			UsdUnreal::MaterialUtils::FDisplayColorMaterial Desc;
			Desc.bHasOpacity = false;
			Desc.bIsDoubleSided = false;

			DisplayColorInstance = UsdUnreal::MaterialUtils::CreateDisplayColorMaterialInstanceDynamic(Desc);

			if (DisplayColorInstance)
			{
				// Leave PrimPath as empty as it likely will be reused by many prims
				UUsdAssetUserData* UserData = NewObject<UUsdAssetUserData>(DisplayColorInstance, TEXT("USDAssetUserData"));
				DisplayColorInstance->AddAssetUserData(UserData);

				DisplayColorInstance->SetFlags(RF_Transient);
			}
		}

		for (int32 Index : IndicesToSetDrawModeColor)
		{
			MaterialInstances[Index] = DisplayColorInstance;
		}
	}
}
