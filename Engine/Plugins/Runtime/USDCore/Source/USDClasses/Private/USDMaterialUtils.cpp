// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDMaterialUtils.h"

#include "USDProjectSettings.h"

#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "UObject/Package.h"

#if WITH_EDITOR
#include "MaterialEditingLibrary.h"
#endif	  // WITH_EDITOR

namespace UE::USDMaterialUtils::Private
{
	static const FString DisplayColorID = TEXT("!DisplayColor");
}

FString UsdUnreal::MaterialUtils::FDisplayColorMaterial::ToString() const
{
	return FString::Printf(TEXT("%s_%d_%d"), *UE::USDMaterialUtils::Private::DisplayColorID, bHasOpacity, bIsDoubleSided);
}

FString UsdUnreal::MaterialUtils::FDisplayColorMaterial::ToPrettyString() const
{
	return FString::Printf(TEXT("DisplayColor%s%s"), bHasOpacity ? TEXT("_Translucent") : TEXT(""), bIsDoubleSided ? TEXT("_TwoSided") : TEXT(""));
}

TOptional<UsdUnreal::MaterialUtils::FDisplayColorMaterial> UsdUnreal::MaterialUtils::FDisplayColorMaterial::FromString(
	const FString& DisplayColorString
)
{
	TArray<FString> Tokens;
	DisplayColorString.ParseIntoArray(Tokens, TEXT("_"));

	if (Tokens.Num() != 3 || Tokens[0] != UE::USDMaterialUtils::Private::DisplayColorID)
	{
		return {};
	}

	FDisplayColorMaterial Result;
	Result.bHasOpacity = static_cast<bool>(FCString::Atoi(*Tokens[1]));
	Result.bIsDoubleSided = static_cast<bool>(FCString::Atoi(*Tokens[2]));
	return Result;
}

const FSoftObjectPath* UsdUnreal::MaterialUtils::GetReferenceMaterialPath(const FDisplayColorMaterial& DisplayColorDescription)
{
	const UUsdProjectSettings* Settings = GetDefault<UUsdProjectSettings>();
	if (!Settings)
	{
		return nullptr;
	}

	if (DisplayColorDescription.bHasOpacity)
	{
		if (DisplayColorDescription.bIsDoubleSided)
		{
			return &Settings->ReferenceDisplayColorAndOpacityTwoSidedMaterial;
		}
		else
		{
			return &Settings->ReferenceDisplayColorAndOpacityMaterial;
		}
	}
	else
	{
		if (DisplayColorDescription.bIsDoubleSided)
		{
			return &Settings->ReferenceDisplayColorTwoSidedMaterial;
		}
		else
		{
			return &Settings->ReferenceDisplayColorMaterial;
		}
	}
}

UMaterialInstanceDynamic* UsdUnreal::MaterialUtils::CreateDisplayColorMaterialInstanceDynamic(const FDisplayColorMaterial& DisplayColorDescription)
{
	const FSoftObjectPath* ParentPathPtr = GetReferenceMaterialPath(DisplayColorDescription);
	if (!ParentPathPtr)
	{
		return nullptr;
	}

	if (UMaterialInterface* ParentMaterial = Cast<UMaterialInterface>(ParentPathPtr->TryLoad()))
	{
		FName AssetName = MakeUniqueObjectName(
			GetTransientPackage(),
			UMaterialInstanceConstant::StaticClass(),
			*DisplayColorDescription.ToPrettyString()
		);

		if (UMaterialInstanceDynamic* NewMaterial = UMaterialInstanceDynamic::Create(ParentMaterial, GetTransientPackage(), AssetName))
		{
			return NewMaterial;
		}
	}

	return nullptr;
}

UMaterialInstanceConstant* UsdUnreal::MaterialUtils::CreateDisplayColorMaterialInstanceConstant(const FDisplayColorMaterial& DisplayColorDescription)
{
#if WITH_EDITOR
	const FSoftObjectPath* ParentPathPtr = GetReferenceMaterialPath(DisplayColorDescription);
	if (!ParentPathPtr)
	{
		return nullptr;
	}

	if (UMaterialInterface* ParentMaterial = Cast<UMaterialInterface>(ParentPathPtr->TryLoad()))
	{
		FName AssetName = MakeUniqueObjectName(
			GetTransientPackage(),
			UMaterialInstanceConstant::StaticClass(),
			*DisplayColorDescription.ToPrettyString()
		);

		if (UMaterialInstanceConstant* MaterialInstance = NewObject<UMaterialInstanceConstant>(GetTransientPackage(), AssetName, RF_NoFlags))
		{
			UMaterialEditingLibrary::SetMaterialInstanceParent(MaterialInstance, ParentMaterial);
			return MaterialInstance;
		}
	}
#endif	  // WITH_EDITOR
	return nullptr;
}

FSoftObjectPath UsdUnreal::MaterialUtils::GetReferencePreviewSurfaceMaterial(EUsdReferenceMaterialProperties ReferenceMaterialProperties)
{
	const UUsdProjectSettings* Settings = GetDefault<UUsdProjectSettings>();
	if (!Settings)
	{
		return {};
	}

	const bool bIsTranslucent = EnumHasAnyFlags(ReferenceMaterialProperties, EUsdReferenceMaterialProperties::Translucent);
	const bool bIsVT = EnumHasAnyFlags(ReferenceMaterialProperties, EUsdReferenceMaterialProperties::VT);
	const bool bIsTwoSided = EnumHasAnyFlags(ReferenceMaterialProperties, EUsdReferenceMaterialProperties::TwoSided);

	const FSoftObjectPath* TargetMaterialPath = nullptr;
	if (bIsTranslucent)
	{
		if (bIsVT)
		{
			if (bIsTwoSided)
			{
				return Settings->ReferencePreviewSurfaceTranslucentTwoSidedVTMaterial;
			}
			else
			{
				return Settings->ReferencePreviewSurfaceTranslucentVTMaterial;
			}
		}
		else
		{
			if (bIsTwoSided)
			{
				return Settings->ReferencePreviewSurfaceTranslucentTwoSidedMaterial;
			}
			else
			{
				return Settings->ReferencePreviewSurfaceTranslucentMaterial;
			}
		}
	}
	else
	{
		if (bIsVT)
		{
			if (bIsTwoSided)
			{
				return Settings->ReferencePreviewSurfaceTwoSidedVTMaterial;
			}
			else
			{
				return Settings->ReferencePreviewSurfaceVTMaterial;
			}
		}
		else
		{
			if (bIsTwoSided)
			{
				return Settings->ReferencePreviewSurfaceTwoSidedMaterial;
			}
			else
			{
				return Settings->ReferencePreviewSurfaceMaterial;
			}
		}
	}
}

FSoftObjectPath UsdUnreal::MaterialUtils::GetVTVersionOfReferencePreviewSurfaceMaterial(const FSoftObjectPath& ReferenceMaterial)
{
	if (!ReferenceMaterial.IsValid())
	{
		return {};
	}

	const UUsdProjectSettings* Settings = GetDefault<UUsdProjectSettings>();
	if (!Settings)
	{
		return {};
	}

	if (ReferenceMaterial.ToString().Contains(TEXT("VT"), ESearchCase::CaseSensitive, ESearchDir::FromEnd))
	{
		return ReferenceMaterial;
	}
	else if (ReferenceMaterial == Settings->ReferencePreviewSurfaceMaterial)
	{
		return Cast<UMaterialInterface>(Settings->ReferencePreviewSurfaceVTMaterial.TryLoad());
	}
	else if (ReferenceMaterial == Settings->ReferencePreviewSurfaceTwoSidedMaterial)
	{
		return Cast<UMaterialInterface>(Settings->ReferencePreviewSurfaceTwoSidedVTMaterial.TryLoad());
	}
	else if (ReferenceMaterial == Settings->ReferencePreviewSurfaceTranslucentMaterial)
	{
		return Cast<UMaterialInterface>(Settings->ReferencePreviewSurfaceTranslucentVTMaterial.TryLoad());
	}
	else if (ReferenceMaterial == Settings->ReferencePreviewSurfaceTranslucentTwoSidedMaterial)
	{
		return Cast<UMaterialInterface>(Settings->ReferencePreviewSurfaceTranslucentTwoSidedVTMaterial.TryLoad());
	}

	// We should only ever call this function with a ReferenceMaterial that matches one of the above paths
	ensure(false);
	return {};
}

FSoftObjectPath UsdUnreal::MaterialUtils::GetTwoSidedVersionOfReferencePreviewSurfaceMaterial(const FSoftObjectPath& ReferenceMaterial)
{
	if (!ReferenceMaterial.IsValid())
	{
		return {};
	}

	const UUsdProjectSettings* Settings = GetDefault<UUsdProjectSettings>();
	if (!Settings)
	{
		return {};
	}

	if (ReferenceMaterial.ToString().Contains(TEXT("TwoSided"), ESearchCase::CaseSensitive, ESearchDir::FromEnd))
	{
		return ReferenceMaterial;
	}
	else if (ReferenceMaterial == Settings->ReferencePreviewSurfaceMaterial)
	{
		return Settings->ReferencePreviewSurfaceTwoSidedMaterial;
	}
	else if (ReferenceMaterial == Settings->ReferencePreviewSurfaceTranslucentMaterial)
	{
		return Settings->ReferencePreviewSurfaceTranslucentTwoSidedMaterial;
	}
	else if (ReferenceMaterial == Settings->ReferencePreviewSurfaceVTMaterial)
	{
		return Settings->ReferencePreviewSurfaceTwoSidedVTMaterial;
	}
	else if (ReferenceMaterial == Settings->ReferencePreviewSurfaceTranslucentVTMaterial)
	{
		return Settings->ReferencePreviewSurfaceTranslucentTwoSidedVTMaterial;
	}

	// We should only ever call this function with a ReferenceMaterial that matches one of the above paths
	ensure(false);
	return {};
}

bool UsdUnreal::MaterialUtils::IsReferencePreviewSurfaceMaterial(const FSoftObjectPath& Material)
{
	if (!Material.IsValid())
	{
		return false;
	}

	const UUsdProjectSettings* Settings = GetDefault<UUsdProjectSettings>();
	if (!Settings)
	{
		return false;
	}

	TSet<FSoftObjectPath> ReferenceMaterials = {
		Settings->ReferencePreviewSurfaceMaterial,
		Settings->ReferencePreviewSurfaceTranslucentMaterial,
		Settings->ReferencePreviewSurfaceTwoSidedMaterial,
		Settings->ReferencePreviewSurfaceTranslucentTwoSidedMaterial,
		Settings->ReferencePreviewSurfaceVTMaterial,
		Settings->ReferencePreviewSurfaceTranslucentVTMaterial,
		Settings->ReferencePreviewSurfaceTwoSidedVTMaterial,
		Settings->ReferencePreviewSurfaceTranslucentTwoSidedVTMaterial};

	return ReferenceMaterials.Contains(Material);
}

namespace UE::USDMaterialUtils::Private
{
	static TArray<FName> RegisteredRenderContexts;
}

void UsdUnreal::MaterialUtils::RegisterRenderContext(const FName& RenderContextName)
{
	UE::USDMaterialUtils::Private::RegisteredRenderContexts.AddUnique(RenderContextName);
	UE::USDMaterialUtils::Private::RegisteredRenderContexts.Sort(
		[](const FName& LHS, const FName& RHS)
		{
			return LHS.ToString() < RHS.ToString();
		}
	);
}

void UsdUnreal::MaterialUtils::UnregisterRenderContext(const FName& RenderContextName)
{
	UE::USDMaterialUtils::Private::RegisteredRenderContexts.Remove(RenderContextName);
}

const TArray<FName>& UsdUnreal::MaterialUtils::GetRegisteredRenderContexts()
{
	return UE::USDMaterialUtils::Private::RegisteredRenderContexts;
}
