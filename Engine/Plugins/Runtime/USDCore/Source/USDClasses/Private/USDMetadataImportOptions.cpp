// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDMetadataImportOptions.h"

#include "AnalyticsEventAttribute.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(USDMetadataImportOptions)

void UsdUtils::AddAnalyticsAttributes(const FUsdMetadataImportOptions& Options, TArray<FAnalyticsEventAttribute>& InOutAttributes)
{
	InOutAttributes.Emplace(TEXT("CollectMetadata"), Options.bCollectMetadata);
	InOutAttributes.Emplace(TEXT("CollectFromEntireSubtree"), Options.bCollectFromEntireSubtrees);
	InOutAttributes.Emplace(TEXT("CollectOnComponents"), Options.bCollectOnComponents);

	FString FilterString = TEXT("[");
	for (const FString& Filter : Options.BlockedPrefixFilters)
	{
		FilterString += Filter + TEXT(", ");
	}
	FilterString.RemoveFromEnd(TEXT(", "));
	FilterString += TEXT("]");
	InOutAttributes.Emplace(TEXT("BlockedPrefixFilters"), FilterString);

	InOutAttributes.Emplace(TEXT("bInvertFilters"), Options.bInvertFilters);
}

void UsdUtils::HashForImport(const FUsdMetadataImportOptions& Options, FSHA1& HashToUpdate)
{
	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.bCollectMetadata), sizeof(Options.bCollectMetadata));
	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.bCollectFromEntireSubtrees), sizeof(Options.bCollectFromEntireSubtrees));
	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.bCollectOnComponents), sizeof(Options.bCollectOnComponents));

	for (const FString& Filter : Options.BlockedPrefixFilters)
	{
		HashToUpdate.UpdateWithString(*Filter, Filter.Len());
	}

	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.bInvertFilters), sizeof(Options.bInvertFilters));
}
