// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDObjectUtils.h"

#include "USDAssetImportData.h"
#include "USDAssetUserData.h"
#include "USDLog.h"

#include "Animation/AnimBlueprint.h"
#include "Animation/AnimSequence.h"
#include "Animation/Skeleton.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture.h"
#include "GeometryCache.h"
#include "GroomAsset.h"
#include "GroomBindingAsset.h"
#include "GroomCache.h"
#include "LevelSequence.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInterface.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "SparseVolumeTexture/SparseVolumeTexture.h"

UAssetImportData* UsdUnreal::ObjectUtils::GetBaseAssetImportData(UObject* Asset)
{
	UAssetImportData* ImportData = nullptr;
#if WITH_EDITORONLY_DATA
	if (UStaticMesh* Mesh = Cast<UStaticMesh>(Asset))
	{
		ImportData = Mesh->GetAssetImportData();
	}
	else if (USkeleton* Skeleton = Cast<USkeleton>(Asset))
	{
		if (USkeletalMesh* SkelMesh = Skeleton->GetPreviewMesh())
		{
			ImportData = SkelMesh->GetAssetImportData();
		}
	}
	else if (UPhysicsAsset* PhysicsAsset = Cast<UPhysicsAsset>(Asset))
	{
		if (USkeletalMesh* SkelMesh = PhysicsAsset->GetPreviewMesh())
		{
			ImportData = SkelMesh->GetAssetImportData();
		}
	}
	else if (UAnimBlueprint* AnimBP = Cast<UAnimBlueprint>(Asset))
	{
		// We will always have a skeleton, but not necessarily we will have a preview mesh directly
		// on the UAnimBlueprint
		if (USkeleton* AnimBPSkeleton = AnimBP->TargetSkeleton.Get())
		{
			if (USkeletalMesh* SkelMesh = AnimBPSkeleton->GetPreviewMesh())
			{
				ImportData = SkelMesh->GetAssetImportData();
			}
		}
	}
	else if (USkeletalMesh* SkelMesh = Cast<USkeletalMesh>(Asset))
	{
		ImportData = SkelMesh->GetAssetImportData();
	}
	else if (UAnimSequence* SkelAnim = Cast<UAnimSequence>(Asset))
	{
		ImportData = SkelAnim->AssetImportData;
	}
	else if (UMaterialInterface* Material = Cast<UMaterialInterface>(Asset))
	{
		ImportData = Material->AssetImportData;
	}
	else if (UTexture* Texture = Cast<UTexture>(Asset))
	{
		ImportData = Texture->AssetImportData;
	}
	else if (UGeometryCache* GeometryCache = Cast<UGeometryCache>(Asset))
	{
		ImportData = GeometryCache->AssetImportData;
	}
	else if (UGroomAsset* Groom = Cast<UGroomAsset>(Asset))
	{
		ImportData = Groom->AssetImportData;
	}
	else if (UGroomCache* GroomCache = Cast<UGroomCache>(Asset))
	{
		ImportData = GroomCache->AssetImportData;
	}
	else if (UStreamableSparseVolumeTexture* SparseVolumeTexture = Cast<UStreamableSparseVolumeTexture>(Asset))
	{
		ImportData = SparseVolumeTexture->AssetImportData;
	}

#endif
	return ImportData;
}

UUsdAssetImportData* UsdUnreal::ObjectUtils::GetAssetImportData(UObject* Asset)
{
	return Cast<UUsdAssetImportData>(GetBaseAssetImportData(Asset));
}

void UsdUnreal::ObjectUtils::SetAssetImportData(UObject* Asset, UAssetImportData* ImportData)
{
	if (!Asset)
	{
		return;
	}

#if WITH_EDITOR
	if (UStaticMesh* Mesh = Cast<UStaticMesh>(Asset))
	{
		Mesh->SetAssetImportData(ImportData);
	}
	else if (USkeletalMesh* SkelMesh = Cast<USkeletalMesh>(Asset))
	{
		SkelMesh->SetAssetImportData(ImportData);
	}
	else if (UAnimSequence* SkelAnim = Cast<UAnimSequence>(Asset))
	{
		SkelAnim->AssetImportData = ImportData;
	}
	else if (UMaterialInterface* Material = Cast<UMaterialInterface>(Asset))
	{
		Material->AssetImportData = ImportData;
	}
	else if (UTexture* Texture = Cast<UTexture>(Asset))
	{
		Texture->AssetImportData = ImportData;
	}
	else if (UGeometryCache* GeometryCache = Cast<UGeometryCache>(Asset))
	{
		GeometryCache->AssetImportData = ImportData;
	}
	else if (UGroomAsset* Groom = Cast<UGroomAsset>(Asset))
	{
		Groom->AssetImportData = ImportData;
	}
	else if (UGroomCache* GroomCache = Cast<UGroomCache>(Asset))
	{
		GroomCache->AssetImportData = ImportData;
	}
	else if (UStreamableSparseVolumeTexture* SparseVolumeTexture = Cast<UStreamableSparseVolumeTexture>(Asset))
	{
		SparseVolumeTexture->AssetImportData = ImportData;
	}
#endif	  // WITH_EDITOR
}

UUsdAssetUserData* UsdUnreal::ObjectUtils::GetAssetUserData(const UObject* Object, TSubclassOf<UUsdAssetUserData> Class)
{
	if (!Object)
	{
		return nullptr;
	}

	if (!Class)
	{
		Class = UUsdAssetUserData::StaticClass();
	}

	const IInterface_AssetUserData* AssetUserDataInterface = Cast<const IInterface_AssetUserData>(Object);
	if (!AssetUserDataInterface)
	{
		UE_LOGF(
			LogUsd,
			Warning,
			"Tried getting AssetUserData from object '%ls', but the class '%ls' doesn't implement the AssetUserData interface!",
			*Object->GetPathName(),
			*Object->GetClass()->GetName()
		);
		return nullptr;
	}

	// Const cast because there is no const access of asset user data on the interface
	return Cast<UUsdAssetUserData>(const_cast<IInterface_AssetUserData*>(AssetUserDataInterface)->GetAssetUserDataOfClass(Class));
}

UUsdAssetUserData* UsdUnreal::ObjectUtils::GetOrCreateAssetUserData(UObject* Object, TSubclassOf<UUsdAssetUserData> Class)
{
	if (!Object)
	{
		return nullptr;
	}

	if (!Class)
	{
		Class = UUsdAssetUserData::StaticClass();
	}

	IInterface_AssetUserData* AssetUserDataInterface = Cast<IInterface_AssetUserData>(Object);
	if (!AssetUserDataInterface)
	{
		UE_LOGF(
			LogUsd,
			Warning,
			"Tried adding AssetUserData to object '%ls', but it doesn't implement the AssetUserData interface!",
			*Object->GetPathName()
		);
		return nullptr;
	}

	UUsdAssetUserData* AssetUserData = Cast<UUsdAssetUserData>(AssetUserDataInterface->GetAssetUserDataOfClass(Class));
	if (!AssetUserData)
	{
		// For now we're expecting objects to only have one instance of UUsdAssetUserData
		ensure(!AssetUserDataInterface->HasAssetUserDataOfClass(UUsdAssetUserData::StaticClass()));

		AssetUserData = NewObject<UUsdAssetUserData>(Object, Class, TEXT("UsdAssetUserData"), RF_Public);
		AssetUserDataInterface->AddAssetUserData(AssetUserData);
	}

	return AssetUserData;
}

bool UsdUnreal::ObjectUtils::SetAssetUserData(UObject* Object, UUsdAssetUserData* AssetUserData)
{
	if (!Object)
	{
		return false;
	}

	IInterface_AssetUserData* AssetUserDataInterface = Cast<IInterface_AssetUserData>(Object);
	if (!AssetUserDataInterface)
	{
		UE_LOGF(
			LogUsd,
			Warning,
			"Tried adding AssetUserData to object '%ls', but it doesn't implement the AssetUserData interface!",
			*Object->GetPathName()
		);
		return false;
	}

	while (AssetUserDataInterface->HasAssetUserDataOfClass(UUsdAssetUserData::StaticClass()))
	{
		UE_LOGF(LogUsd, Verbose, "Removing old AssetUserData from object '%ls' before adding a new one", *Object->GetPathName());
		AssetUserDataInterface->RemoveUserDataOfClass(UUsdAssetUserData::StaticClass());
	}

	AssetUserDataInterface->AddAssetUserData(AssetUserData);
	return true;
}

TSubclassOf<UUsdAssetUserData> UsdUnreal::ObjectUtils::GetAssetUserDataClassForObject(UClass* ObjectClass)
{
	if (!ObjectClass)
	{
		return nullptr;
	}

	if (ObjectClass->IsChildOf(UMaterialInterface::StaticClass()))
	{
		return UUsdMaterialAssetUserData::StaticClass();
	}
	else if (ObjectClass->IsChildOf(UStaticMesh::StaticClass()) || ObjectClass->IsChildOf(USkeletalMesh::StaticClass()))
	{
		return UUsdMeshAssetUserData::StaticClass();
	}
	else if (ObjectClass->IsChildOf(UGeometryCache::StaticClass()))
	{
		return UUsdGeometryCacheAssetUserData::StaticClass();
	}
	else if (ObjectClass->IsChildOf(UAnimSequence::StaticClass()))
	{
		return UUsdAnimSequenceAssetUserData::StaticClass();
	}
	else if (ObjectClass->IsChildOf(USparseVolumeTexture::StaticClass()))
	{
		return UUsdSparseVolumeTextureAssetUserData::StaticClass();
	}
	else if (ObjectClass->ImplementsInterface(UInterface_AssetUserData::StaticClass()))
	{
		// Only return UUsdAssetUserData in case the object can hold asset user data, otherwise we'd get a warning
		// if we try using our return value with e.g. UsdUnreal::ObjectUtils::SetAssetUserData
		return UUsdAssetUserData::StaticClass();
	}

	return nullptr;
}

FString UsdUnreal::ObjectUtils::SanitizeObjectName(const FString& InObjectName)
{
	FString SanitizedText = InObjectName;
	const TCHAR* InvalidChar = INVALID_OBJECTNAME_CHARACTERS;
	while (*InvalidChar)
	{
		SanitizedText.ReplaceCharInline(*InvalidChar, TCHAR('_'), ESearchCase::CaseSensitive);
		++InvalidChar;
	}

	return SanitizedText;
}

FString UsdUnreal::ObjectUtils::GetPrefixedAssetName(const FString& DesiredName, UClass* AssetClass)
{
	if (!AssetClass)
	{
		return DesiredName;
	}

	FString Prefix;
	FString Suffix = DesiredName;

	if (AssetClass->IsChildOf(UStaticMesh::StaticClass()))
	{
		Prefix = TEXT("SM_");
	}
	else if (AssetClass->IsChildOf(UGroomAsset::StaticClass()) || AssetClass->IsChildOf(UGroomCache::StaticClass()) || AssetClass->IsChildOf(UGroomBindingAsset::StaticClass()))
	{
		Prefix = TEXT("GR_");
	}
	else if (AssetClass->IsChildOf(USkeletalMesh::StaticClass()))
	{
		Prefix = TEXT("SK_");
	}
	else if (AssetClass->IsChildOf(USkeleton::StaticClass()))
	{
		Prefix = TEXT("SKEL_");
	}
	else if (AssetClass->IsChildOf(UPhysicsAsset::StaticClass()))
	{
		FString TempName = Suffix;

		// The asset is named after the SkelRoot prim. If we're importing back a scene that was originally exported,
		// we should clean up these prefixes or else we may end up with something like "PHYS_SK_PrimName"
		TempName.RemoveFromStart(TEXT("PHYS_"), ESearchCase::CaseSensitive);
		TempName.RemoveFromStart(TEXT("SK_"), ESearchCase::CaseSensitive);
		if (!TempName.IsEmpty())
		{
			Suffix = TempName;
		}

		Prefix = TEXT("PHYS_");
	}
	else if (AssetClass->IsChildOf(UAnimSequence::StaticClass()))
	{
		Prefix = TEXT("AS_");
	}
	else if (AssetClass->IsChildOf(UMaterialInterface::StaticClass()))
	{
		if (AssetClass->IsChildOf(UMaterialInstance::StaticClass()))
		{
			Prefix = TEXT("MI_");
		}
		else
		{
			Prefix = TEXT("M_");
		}
	}
	else if (AssetClass->IsChildOf(UTexture::StaticClass()))
	{
		Prefix = TEXT("T_");
	}
	else if (AssetClass->IsChildOf(ULevelSequence::StaticClass()))
	{
		Prefix = TEXT("LS_");
	}
	else if (AssetClass->IsChildOf(UAnimBlueprint::StaticClass()))
	{
		FString TempName = Suffix;

		// The asset is named after the SkelRoot prim. If we're importing back a scene that was originally exported,
		// we should clean up these prefixes or else we may end up with something like "ABP_SK_PrimName"
		TempName.RemoveFromStart(TEXT("ABP_"), ESearchCase::CaseSensitive);
		TempName.RemoveFromStart(TEXT("SK_"), ESearchCase::CaseSensitive);
		if (!TempName.IsEmpty())
		{
			Suffix = TempName;
		}

		Prefix = TEXT("ABP_");
	}
	else if (AssetClass->IsChildOf(USparseVolumeTexture::StaticClass()))
	{
		Prefix = TEXT("SVT_");
	}

	if (!Suffix.StartsWith(Prefix))
	{
		Suffix = Prefix + Suffix;
	}

	return Suffix;
}

bool UsdUnreal::ObjectUtils::RemoveNumberedSuffix(FString& Prefix)
{
	if (Prefix.IsNumeric())
	{
		return false;
	}

	bool bRemoved = false;

	FString LastChar = Prefix.Right(1);
	while ((LastChar.IsNumeric() || LastChar == TEXT("_")) && Prefix.Len() > 1)
	{
		Prefix.LeftChopInline(1, EAllowShrinking::No);
		LastChar = Prefix.Right(1);

		bRemoved = true;
	}
	Prefix.Shrink();

	return bRemoved;
}
