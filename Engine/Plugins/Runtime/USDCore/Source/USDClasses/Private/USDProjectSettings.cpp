// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDProjectSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(USDProjectSettings)

#if WITH_EDITOR
void UUsdProjectSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	UDeveloperSettings::PostEditChangeProperty(PropertyChangedEvent);

	// Allows settings to be applied even if the properties were set via utility blueprints/Python
	SaveConfig();
}
#endif	  // WITH_EDITOR
