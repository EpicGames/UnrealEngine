// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDStageOptions.h"

#include "AnalyticsEventAttribute.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(USDStageOptions)

void UsdUtils::AddAnalyticsAttributes(const FUsdStageOptions& Options, TArray<FAnalyticsEventAttribute>& InOutAttributes)
{
	InOutAttributes.Emplace(TEXT("MetersPerUnit"), LexToString(Options.MetersPerUnit));
	InOutAttributes.Emplace(TEXT("UpAxis"), Options.UpAxis == EUsdUpAxis::YAxis ? TEXT("Y") : TEXT("Z"));
}

void UsdUtils::HashForExport(const FUsdStageOptions& Options, FSHA1& HashToUpdate)
{
	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options), sizeof(Options));
}
