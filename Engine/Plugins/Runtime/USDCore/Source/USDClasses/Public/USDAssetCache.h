// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"

#include "USDAssetCache.generated.h"

#define UE_API USDCLASSES_API

UCLASS(meta = (ScriptName = "UsdAssetCache_Deprecated"))
class UE_DEPRECATED(all, "Use the UUsdAssetCache2 class instead, which is an actual asset you can create on the Content Browser") UUsdAssetCache : public UObject
{
	GENERATED_BODY()

public:
	UE_API UUsdAssetCache();

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	UE_API void CacheAsset(const FString& Hash, UObject* Asset, const FString& PrimPath = FString());
	UE_API void DiscardAsset(const FString& Hash);
	UE_API UObject* GetCachedAsset(const FString& Hash) const;

	TMap<FString, TObjectPtr<UObject>> GetCachedAssets() const
	{
		TMap<FString, TObjectPtr<UObject>> CachedAssets(TransientStorage);
		CachedAssets.Append(PersistentStorage);

		return CachedAssets;
	}

	UE_API void LinkAssetToPrim(const FString& PrimPath, UObject* Asset);
	UE_API void RemoveAssetPrimLink(const FString& PrimPath);
	UE_API UObject* GetAssetForPrim(const FString& PrimPath) const;

	/** Returns the first prim path associated with an asset (see TMap::FindKey) */
	UE_API FString GetPrimForAsset(UObject* Asset) const;

	/**
	 * Returns the hash used as a key for an asset, in case we own it.
	 * Returns the empty string otherwise.
	 */
	UE_API FString GetHashForAsset(UObject* Asset) const;

	TMap<FString, TWeakObjectPtr<UObject>> GetAssetPrimLinks() const
	{
		return PrimPathToAssets;
	};	  // Can't return a reference as it wouldn't be thread-safe

	bool IsAssetOwnedByCache(UObject* Asset) const
	{
		return OwnedAssets.Contains(Asset);
	}

	int32 GetNumAssets() const
	{
		return TransientStorage.Num() + PersistentStorage.Num();
	}

	UE_API void Reset();

	/**
	 * Every time an asset is retrieved/inserted we add it to ActiveAssets. With these functions that container can be reset/returned.
	 *
	 * When importing via the USDStageImporter we will move assets from this cache to the content folder.
	 * The problem is that some of those assets may not be currently used (e.g. inactive variant, purpose, etc.).
	 * The USDStageImporter will then use these functions before/after translating the scene to know which items from the
	 * cache that are actually used for the current scene.
	 */
	UE_API void MarkAssetsAsStale();

	/** Returns assets that aren't marked as stale */
	UE_API TSet<UObject*> GetActiveAssets() const;

	// We need to be serializable so that AUsdStageActor can duplicate us for PIE
	UE_API virtual void Serialize(FArchive& Ar) override;

private:
	// For now everything is NonPIEDuplicateTransient as this is mostly a subobject of AUsdStageActor. When the actor is duplicated
	// it will need to reload the stage anyway to rebuild its prim links to components and assets, so there is no point in duplicating
	// the properties here just yet. Obviously we want to duplicate these properties
	UPROPERTY(NonPIEDuplicateTransient, Transient, VisibleAnywhere, Category = "Assets")
	TMap<FString, TObjectPtr<UObject>> TransientStorage;

	UPROPERTY(NonPIEDuplicateTransient, VisibleAnywhere, Category = "Assets")
	TMap<FString, TObjectPtr<UObject>> PersistentStorage;

	UPROPERTY(EditAnywhere, Category = "Assets", AdvancedDisplay)
	bool bAllowPersistentStorage;

	// Points to the assets in primary storage, used to quickly check if we own an asset
	UPROPERTY(NonPIEDuplicateTransient)
	TSet<TWeakObjectPtr<UObject>> OwnedAssets;

	// Keeps associations from prim paths to assets that we own in primary storage
	UPROPERTY(NonPIEDuplicateTransient)
	TMap<FString, TWeakObjectPtr<UObject>> PrimPathToAssets;

	// Assets that were added/retrieved since the last call to MarkAssetsAsSlate();
	mutable TSet<UObject*> ActiveAssets;

	// Used to ensure thread safety
	mutable FCriticalSection CriticalSection;
};

#undef UE_API
