// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/LruCache.h"
#include "Containers/Map.h"
#include "Serialization/BulkData.h"
#include "UObject/ObjectKey.h"

#include "USDAssetCache2.generated.h"

#define UE_API USDCLASSES_API

UCLASS(MinimalAPI, BlueprintType, Blueprintable)
class UE_DEPRECATED(5.8, "Use the UUsdAssetCache3 class instead, which exposes the cached assets on the Content Browser") UUsdAssetCache2 : public UObject
{
	GENERATED_BODY()

	class FCachedAssetInfo;

public:
	/**
	 * The asset cache will always retain all currently used assets.
	 * In addition to that, this limit specifies how much size is allocated to storing assets that remain only for the
	 * current session and that aren't being used by any stage.
	 * Set this to 0 to disable storing unreferenced assets.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	double UnreferencedAssetStorageSizeMB = 500;

	/**
	 * This limit specifies how much size is allocated to storing all persistent assets (i.e. assets that will be
	 * saved to disk, even if unused by any stage).
	 * Set this to 0 to disable persistent asset storage.
	 * This has no effect on temporary asset caches (e.g. the ones automatically generated when opening a stage
	 * without an asset cache asset assigned)
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	double PersistentAssetStorageSizeMB = 500;

public:
	/** Adds an asset to the cache attached to a particular hash, and optionally registering a referencer */
	UFUNCTION(BlueprintCallable, Category = "Caching", meta = (CallInEditor = "true"))
	UE_API void CacheAsset(const FString& Hash, UObject* Asset, const UObject* Referencer = nullptr);

	/**
	 * Returns true if the asset with the given hash can be removed from the cache. It will return false in case the
	 * asset is still being used, either by another consumer asset or directly by some referencer.
	 */
	UFUNCTION(BlueprintCallable, Category = "Caching", meta = (CallInEditor = "true"))
	UE_API bool CanRemoveAsset(const FString& Hash);

	/**
	 * If an asset is associated with `Hash`, it will be returned and the asset cache will stop tracking this asset
	 * entirely. Returns nullptr otherwise. See CanRemoveAsset.
	 *
	 * WARNING: The asset will still be outer'd to the asset cache, however. The caller is in charge of properly
	 * placing the asset at a new Outer object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Caching", meta = (CallInEditor = "true"))
	UE_API UObject* RemoveAsset(const FString& Hash);

	/**
	 * Returns an asset associated with a particular `Hash`.
	 * If the asset is persistent, unloaded and the "USD.OnDemandCachedAssetLoading" cvar is true, this may cause the
	 * asset to be read from the asset cache's file on disk.
	 */
	UFUNCTION(BlueprintCallable, Category = "Caching", meta = (CallInEditor = "true"))
	UE_API UObject* GetCachedAsset(const FString& Hash);

	/**
	 * Marks the provided asset as being used at this point, optionally adding a specific referencer.
	 * This is useful because the asset cache will always prioritize retaining the most recently used assets
	 */
	UFUNCTION(BlueprintCallable, Category = "Caching", meta = (CallInEditor = "true"))
	UE_API bool TouchAsset(const UObject* Asset, const UObject* Referencer = nullptr);

	/**
	 * Adds a new UObject referencer to a particular asset, returning true if the operation succeeded.
	 *
	 * Assets will not be evicted or removed from the cache while the referencer is registered.
	 * Note that internally the cache keeps FObjectKey structs constructed from the referencers, instead of direct
	 * pointers to them.
	 */
	UFUNCTION(BlueprintCallable, Category = "Caching", meta = (CallInEditor = "true"))
	UE_API bool AddAssetReference(const UObject* Asset, const UObject* Referencer);

	/**
	 * Removes an UObject referencer from a particular asset, returning true if the operation succeeded.
	 * If no specific Referencer is provided, all referencers to Asset will be removed.
	 */
	UFUNCTION(BlueprintCallable, Category = "Caching", meta = (CallInEditor = "true"))
	UE_API bool RemoveAssetReference(const UObject* Asset, const UObject* Referencer = nullptr);

	/**
	 * Removes the particular referencer to all assets tracked by the cache, if it was a referencer to any of them.
	 * Returns true if at least one asset referencer count was altered by this.
	 */
	UFUNCTION(BlueprintCallable, Category = "Caching", meta = (CallInEditor = "true"))
	UE_API bool RemoveAllAssetReferences(const UObject* Referencer);

	/**
	 * Returns the hash associated with an asset, in case we own it. Returns the empty string otherwise.
	 * Note: This has O(1) time complexity.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	UE_API FString GetHashForAsset(const UObject* Asset) const;

	/**
	 * Returns true in case the asset at `AssetPath` is tracked by the cache in any way (persistent asset,
	 * unreferenced or referenced). An example AssetPath would be "/Game/MyTextures/RedBrick.RedBrick".
	 */
	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	UE_API bool IsAssetOwnedByCache(const FString& AssetPath) const;

	/**
	 * Returns how many assets are tracked by the asset cache in total (summing up persistent, referenced and
	 * unreferenced storage)
	 */
	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	UE_API int32 GetNumAssets() const;

	/**
	 * Returns all asset hashes tracked by the asset cache, for all storage types. This includes unloaded
	 * persistent assets
	 */
	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	UE_API TArray<FString> GetAllAssetHashes() const;

	/**
	 * Returns all assets that are currently loaded in the asset cache.
	 * This will not include persistent assets that haven't been loaded yet.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	UE_API TArray<UObject*> GetAllLoadedAssets() const;

	/**
	 * Returns all asset paths tracked by the asset cache, for all storage types. (e.g.
	 * "/Game/MyTextures/RedBrick.RedBrick"). This includes unloaded persistent assets
	 */
	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	UE_API TArray<FString> GetAllCachedAssetPaths() const;

	/** Discards all tracked assets across all storage types */
	UFUNCTION(BlueprintCallable, Category = "Settings", meta = (CallInEditor = "true"))
	UE_API void Reset();

	/**
	 * Updates which assets belong to each storage type. You must call this in case you perform direct operations on
	 * the asset cache, after those operations are fully complete.
	 */
	UFUNCTION(BlueprintCallable, Category = "Settings", meta = (CallInEditor = "true"))
	UE_API void RefreshStorage();

public:
	UE_API UUsdAssetCache2();

	/**
	 * Every time an asset is retrieved/inserted we add it to ActiveAssets.
	 * With these functions that container can be reset/returned.
	 */
	UE_API void MarkAssetsAsStale();

	/** Returns assets that aren't marked as stale */
	UE_API TSet<UObject*> GetActiveAssets() const;

	// Begin UObject interfacte
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostLoad() override;
	// End UObject interfacte

private:
	// Tries unloading unreferenced, persistent assets back to bulkdata on disk if we can
	UE_API bool TryUnloadAsset(FCachedAssetInfo& InOutInfo);
	UE_API bool IsAssetOwnedByCacheInternal(const FString& AssetPath) const;
	UE_API bool AddAssetReferenceInternal(const UObject* Asset, const UObject* Referencer);
	UE_API void TouchAssetInternal(const UObject* Asset, const UObject* Referencer = nullptr);

private:
	friend struct FUsdScopedAssetCacheReferencer;

	// Describes the different "types of storage" we allow on the asset cache.
	// Note: Order is very important here. The rule of thumb is that it should always be OK for an asset in a storage
	// of lower value to have dependencies on assets in storage of higher value (e.g. it's OK for a material in
	// referenced storage to depend on textures in persistent storage, but *not* the other way around!)
	enum class ECacheStorageType : uint8
	{
		None = 0,
		Unreferenced,
		Referenced,
		Persistent
	};
	friend FArchive& operator<<(FArchive& Ar, UUsdAssetCache2::ECacheStorageType& Type);

	// Describes all information we know about an asset in storage.
	// Have to put definition here because of how TLruCache is header-only
	class FCachedAssetInfo
	{
	public:
		// We also store the asset hash in here as it is handy and allows O(1) asset to hash mapping via the LRUCache
		FString Hash;

		// Where the cached asset is actually serialized into
		FByteBulkData BulkData;

		// Information we need separately in order to deserialize the UObject from bulkdata
		FSoftObjectPath AssetClassName;
		FString AssetName;
		EObjectFlags AssetFlags;

		// Information fetched on-demand during RefreshStorage, but that persists to disk.
		// Note that "on disk" here really means "on the byte buffer", although the idea is the same
		uint64 SizeOnDiskInBytes = 0;
		uint64 SizeOnMemoryInBytes = 0;
		TSet<FSoftObjectPath> Dependencies;
		TSet<FSoftObjectPath> Consumers;

		// Transient stuff
		ECacheStorageType CurrentStorageType = ECacheStorageType::None;	   // Only used internally during RefreshStorage()
		TSet<FObjectKey> Referencers;

	public:
		void Serialize(FArchive& Ar, UObject* Owner);
	};

	// Main hash to asset storage for all assets that we're currently using and shouldn't be GC'd
	UPROPERTY(Transient, VisibleAnywhere, Category = "Owned Assets")
	TMap<FString, TObjectPtr<UObject>> AssetStorage;

	// Storage of paths to additional assets that we know are persisted on disk (and tracked on LRUCache),
	// but that we haven't loaded from bulkdata in this session yet
	TMap<FString, FSoftObjectPath> PendingPersistentStorage;

	// Main storage of assets in most recent to least recently used.
	// This also doubles as an inverse map from object to hash, and as a quick way of checking if we own an asset
	TLruCache<FSoftObjectPath, FCachedAssetInfo> LRUCache;

	// Assets that were added/retrieved since the last call to MarkAssetsAsSlate();
	mutable TSet<UObject*> ActiveAssets;

	// Used to ensure thread safety
	mutable FRWLock RWLock;

	// When this is set to something, we will track that it is referencing any new asset that we cache.
	// See FUsdScopedAssetCacheReferencer just below
	const UObject* CurrentScopedReferencer = nullptr;
};	  // UCLASS(BlueprintType)

PRAGMA_DISABLE_DEPRECATION_WARNINGS
struct UE_DEPRECATED(5.8, "No longer used as it's not needed by the UsdAssetCache3") FUsdScopedAssetCacheReferencer
{
public:
	UE_API explicit FUsdScopedAssetCacheReferencer(UUsdAssetCache2* InAssetCache, const UObject* Referencer);
	UE_API ~FUsdScopedAssetCacheReferencer();

	FUsdScopedAssetCacheReferencer() = delete;
	FUsdScopedAssetCacheReferencer(const FUsdScopedAssetCacheReferencer&) = delete;
	FUsdScopedAssetCacheReferencer(FUsdScopedAssetCacheReferencer&&) = delete;
	FUsdScopedAssetCacheReferencer& operator=(const FUsdScopedAssetCacheReferencer&) = delete;
	FUsdScopedAssetCacheReferencer& operator=(FUsdScopedAssetCacheReferencer&&) = delete;

private:
	TWeakObjectPtr<UUsdAssetCache2> AssetCache;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef UE_API
