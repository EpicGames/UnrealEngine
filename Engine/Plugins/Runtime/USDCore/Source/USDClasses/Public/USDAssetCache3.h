// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "UObject/ObjectKey.h"

#include "USDAssetCache3.generated.h"

#define UE_API USDCLASSES_API

/**
 * This class is an asset that can be created via the Content Browser and assigned to AUsdStageActors.
 *
 * Its main purpose is to track generated assets based on the hash of the source prim data: Whenever the AUsdStageActor needs
 * to generate e.g. a MaterialInstance, it will first hash the Material prim, and check whether its UUsdAssetCache3 already
 * has an asset of that class for the resulting hash.
 *
 * The cache can then be shared by multiple AUsdStageActors to prevent recreating UObjects from identical, already translated prim data.
 *
 * A "Default Asset Cache" can be set on the project settings, and will be automatically used for any AUsdStageActor that hasn't
 * had an asset cache manually set beforehand.
 */
UCLASS(MinimalAPI, BlueprintType, Blueprintable, meta = (DisplayName = "USD Asset Cache", ScriptName = "UsdAssetCache"))
class UUsdAssetCache3 : public UObject
{
	GENERATED_BODY()

public:

	/**
	 * Returns the cached UObject of the provided Class for the provided Hash if one exists.
	 * Otherwise, finds a new package for it on the cache's AssetDirectory and creates the asset
	 * via a NewObject<Class> call, using with a sanitized version of the desired name and flags.
	 *
	 * WARNING: As this may try loading a package from disk or call NewObject, this can only be called from the game thread!
	 *
	 * @param Hash - The string key to check with
	 * @param Class - The class of the object that we want to retrieve or create from the asset cache
	 * @param DesiredName - The name we want the created object to have (the actual name may have additional suffixes)
	 * @param DesiredFlags - The flags we want the created object to have (the actual applied flags may differ depending on context)
	 * @param bOutCreatedAsset - Set to true if we created the asset that was return, but false if we returned an existing asset
	 * @param Referencer - The asset will not be deleted or untracked until this referencer is removed (via any of the RemoveAssetReferencer member functions)
	 * @return The asset that was returned or created
	 */
	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	UE_API UObject* GetOrCreateCachedAsset(
		const FString& Hash,
		UClass* Class,
		const FString& DesiredName,
		int32 DesiredFlags,
		bool& bOutCreatedAsset,
		const UObject* Referencer = nullptr
	);

	/**
	 * Templated version of GetOrCreateCachedAsset for convenience
	 *
	 * WARNING: As this may try loading a package from disk or call NewObject, this can only be called from the game thread!
	 *
	 * @param Hash - The string key to check with
	 * @param DesiredName - The name we want the created object to have (the actual name may have additional suffixes)
	 * @param DesiredFlags - The flags we want the created object to have (the actual applied flags may differ depending on context)
	 * @param bOutCreatedAsset - Set to true if we created the asset that was return, but false if we returned an existing asset
	 * @param Referencer - The asset will not be deleted or untracked until this referencer is removed (via any of the RemoveAssetReferencer member functions)
	 * @return The asset that was returned or created
	 */
	template<typename T>
	T* GetOrCreateCachedAsset(
		const FString& Hash,
		FString DesiredName,
		EObjectFlags DesiredFlags,
		bool* bOutCreatedAsset = nullptr,
		const UObject* Referencer = nullptr
	)
	{
		bool bDidCreateAsset = false;
		T* Result = Cast<T>(GetOrCreateCachedAsset(Hash, T::StaticClass(), DesiredName, DesiredFlags, bDidCreateAsset, Referencer));
		if (bOutCreatedAsset)
		{
			*bOutCreatedAsset = bDidCreateAsset;
		}
		return Result;
	}

	/**
	 * For most asset types GetOrCreateCachedAsset should suffice: It will internally call NewObject<Class>().
	 *
	 * Some asset types or workflows have different ways of instantiating the assets though, like textures which must go through the UTextureFactory,
	 * MIDs that must be constructed via UMaterialInstanceDynamic::Create, and others. For those cases you can call this function, and provide
	 * a lambda that actually creates the UObject* itself based on the provided package Outer and sanitized FName.
	 *
	 * WARNING: As this may try loading a package from disk or call NewObject, this can only be called from the game thread!
	 *
	 * @param Hash - The string key to check with
	 * @param Class - The class of the object that we want to retrieve or create from the asset cache
	 * @param DesiredName - The name we want the created object to have (the actual name may have additional suffixes)
	 * @param DesiredFlags - The flags we want the created object to have (the actual applied flags may differ depending on context)
	 * @param ObjectCreationFunc - Lambda that will be called to actually create the UObject. The lambda should internally use the PackageOuter parameter as its outer, and the provided SanitizeName and FlagsToUse.
	 * @param bOutCreatedAsset - Set to true if we created the asset that was return, but false if we returned an existing asset
	 * @param Referencer - The asset will not be deleted or untracked until this referencer is removed (via any of the RemoveAssetReferencer member functions)
	 * @return The asset that was returned or created
	 */
	UE_API UObject* GetOrCreateCustomCachedAsset(
		const FString& Hash,
		UClass* Class,
		const FString& DesiredName,
		EObjectFlags DesiredFlags,
		TFunctionRef<UObject*(UPackage* PackageOuter, FName SanitizedName, EObjectFlags FlagsToUse)> ObjectCreationFunc,
		bool* bOutCreatedAsset = nullptr,
		const UObject* Referencer = nullptr
	);

	/**
	 * Templated version of GetOrCreateCachedAsset for convenience
	 *
	 * WARNING: As this may try loading a package from disk or call NewObject, this can only be called from the game thread!
	 *
	 * @param Hash - The string key to check with
	 * @param DesiredName - The name we want the created object to have (the actual name may have additional suffixes)
	 * @param DesiredFlags - The flags we want the created object to have (the actual applied flags may differ depending on context)
	 * @param ObjectCreationFunc - Lambda that will be called to actually create the UObject. The lambda should internally use the PackageOuter parameter as its outer, and the provided SanitizeName and FlagsToUse.
	 * @param bOutCreatedAsset - Set to true if we created the asset that was return, but false if we returned an existing asset
	 * @param Referencer - The asset will not be deleted or untracked until this referencer is removed (via any of the RemoveAssetReferencer member functions)
	 * @return The asset that was returned or created
	 */
	template<typename T>
	T* GetOrCreateCustomCachedAsset(
		const FString& Hash,
		FString DesiredName,
		EObjectFlags DesiredFlags,
		TFunctionRef<UObject*(UPackage* PackageOuter, FName SanitizedName, EObjectFlags FlagsToUse)> ObjectCreationFunc,
		bool* bOutCreatedAsset = nullptr,
		const UObject* Referencer = nullptr
	)
	{
		return Cast<T>(	   //
			GetOrCreateCustomCachedAsset(Hash, T::StaticClass(), DesiredName, DesiredFlags, ObjectCreationFunc, bOutCreatedAsset, Referencer)
		);
	}

	/** Adds an existing asset to the cache attached to a particular hash, and optionally registering a referencer */
	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	UE_API void CacheAsset(const FString& Hash, const FSoftObjectPath& AssetPath, const UObject* Referencer = nullptr);

	/**
	 * Removes all info about the asset associated with Hash from this cache, if there is any.
	 * Note: This will not delete the asset however: Only tracked, *unreferenced* assets can be deleted by the asset cache,
	 * and only when manually created by it or if flagged with SetAssetDeletable
	 */
	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	UE_API FSoftObjectPath StopTrackingAsset(const FString& Hash);

	/**
	 * Returns the asset associated with a particular Hash, if any. Returns nullptr if there isn't any
	 * associated path to this Hash, or if the associated path doesn't resolve to an asset.
	 *
	 * WARNING: As this may try loading a package from disk, this can only be called from the game thread!
	 */
	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	UE_API UObject* GetCachedAsset(const FString& Hash) const;

	/**
	 * Templated version of GetCachedAsset for convenience
	 *
	 * WARNING: As this may try loading a package from disk, this can only be called from the game thread!
	 */
	template<typename T>
	T* GetCachedAsset(const FString& Hash)
	{
		return Cast<T>(GetCachedAsset(Hash));
	}

	/**
	 * Returns the internal FSoftObjectPath associated with Hash, without trying to load the asset.
	 * If there is no asset associated with Hash, will return an invalid (empty) FSoftObjectPath.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	UE_API FSoftObjectPath GetCachedAssetPath(const FString& Hash) const;

	/**
	 * Returns the hash associated with a particular asset, or the empty string if there isn't any.
	 * Note: The asset cache keeps internal reverse maps, so this should be O(1)
	 */
	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	UE_API FString GetHashForAsset(const FSoftObjectPath& AssetPath) const;

	/**
	 * Returns true if this asset is currently tracked by the asset cache's main hash to asset maps
	 */
	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	UE_API bool IsAssetTrackedByCache(const FSoftObjectPath& AssetPath) const;

	/**
	 * Returns the total number of cached asset paths, whether these resolve to assets or not
	 */
	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	UE_API int32 GetNumAssets() const;

	/**
	 * Returns a copy of the internal mapping between hashes and asset paths
	 */
	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	UE_API TMap<FString, FSoftObjectPath> GetAllTrackedAssets() const;

	/**
	 * The same as GetAllTrackedAssets, except that it will automatically try loading all the asset paths before
	 * returning, which should be convenient for Python or Blueprint callers.
	 *
	 * WARNING: As this may try loading a package from disk, this can only be called from the game thread!
	 */
	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	UE_API TMap<FString, UObject*> LoadAndGetAllTrackedAssets() const;

	/**
	 * Adds a new UObject referencer to a particular asset, returning true if the operation succeeded.
	 * Assets will not be deleted or untracked by the asset cache while the referencer is registered.
	 */
	UFUNCTION(BlueprintCallable, Category = "Caching", meta = (CallInEditor = "true"))
	UE_API bool AddAssetReferencer(const UObject* Asset, const UObject* Referencer);

	/**
	 * Removes an UObject referencer from a particular asset, returning true if anything was removed.
	 * Will do nothing in case Asset or Referencer are invalid.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	UE_API bool RemoveAssetReferencer(const UObject* Asset, const UObject* Referencer);

	/**
	 * Removes all UObject referencers from a particular asset, returning true if anything was removed.
	 * Will do nothing in case Asset is invalid.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	UE_API bool RemoveAllReferencersForAsset(const UObject* Asset);

	/**
	 * Removes a particular UObject referencer from all tracked assets, returning true if anything was removed.
	 * Will do nothing in case Referencer is invalid.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	UE_API bool RemoveAllReferencerAssets(const UObject* Referencer);

	/**
	 * Removes all UObject referencer from all tracked assets, returning true if anything was removed.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	UE_API bool RemoveAllAssetReferencers();

	/**
	 * Sets a particular asset as deletable or not.
	 * Assets not flagged as deletable will never be deleted by the asset cache when DeleteUnreferencedAssets is called.
	 * Assets the cache creates itself via GetOrCreateCachedAsset or GetOrCreateCustomCachedAsset are automatically
	 * set as deletable.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	UE_API void SetAssetDeletable(const UObject* Asset, bool bIsDeletable);

	/** Returns whether a particular asset is currently marked as deletable or not */
	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	UE_API bool IsAssetDeletable(const UObject* Asset) const;

	/**
	 * Deletes all assets that:
	 *   - Are currently tracked by the asset cache;
	 *   - Are set as deletable;
	 *   - Are not used by other UObjects (by external assets, components, undo buffer, Python scripting variables, etc.).
	 *   - Have no referencers;
	 *   - Have not been saved to disk;
	 *
	 * If bShowConfirmation is true, this will fallback to using engine code for deleting the assets, showing a
	 * confirmation dialog listing the assets that will be deleted. If false, it will silently try deleting the assets it can.
	 *
	 * WARNING: This will clear the undo buffer (i.e. transaction history) and run garbage collection after deleting.
	 */
	UFUNCTION(BlueprintCallable, Category = "Cleanup", meta = (CallInEditor = "true"))
	UE_API void DeleteUnreferencedAssets(const bool bShowConfirmation = false);

	/**
	 * This is the same as calling DeleteUnreferencedAssets and providing true for bShowConfirmation.
	 * It is just exposed in this manner so we automatically get a button for calling this function on details panels of the
	 * asset cache.
	 *
	 * WARNING: This will clear the undo buffer (i.e. transaction history) and run garbage collection after deleting.
	 */
	UFUNCTION(BlueprintCallable, Category = "Cleanup", meta = (CallInEditor = "true", DisplayName = "Delete Unreferenced Assets"))
	UE_API void DeleteUnreferencedAssetsWithConfirmation();

	/**
	 * Checks the current AssetDirectory for any new assets that were generated from USD, and automatically caches
	 * them if possible.
	 *
	 * Note: This will never overwrite any existing information on the asset cache (i.e. if the newly found asset
	 * is associated with a hash that is already in use, it will be ignored)
	 */
	UFUNCTION(BlueprintCallable, Category = "Storage", meta = (CallInEditor = "true"))
	UE_API void RescanAssetDirectory();

public:
	/**
	 * The UUsdAssetCache3 can track all the UObjects that are referencing assets, so that it knows when to discard an
	 * unreferenced asset.
	 *
	 * This struct lets you specify a single UObject that will be automatically used as the referencer for the given asset
	 * cache for the duration of the scope.
	 *
	 * Usage:
	 *     FUsdScopedReferencer ScopedReferencer{ StageActor->UsdAssetCache, StageActor };
	 *     StageActor->SetRootLayer(MyRootLayer);  // Adds assets to the cache
	 */
	struct FUsdScopedReferencer
	{
	public:
		UE_API explicit FUsdScopedReferencer(UUsdAssetCache3* InAssetCache, const UObject* Referencer);
		UE_API ~FUsdScopedReferencer();

		FUsdScopedReferencer() = delete;
		FUsdScopedReferencer(const FUsdScopedReferencer&) = delete;
		FUsdScopedReferencer(FUsdScopedReferencer&&) = delete;
		FUsdScopedReferencer& operator=(const FUsdScopedReferencer&) = delete;
		FUsdScopedReferencer& operator=(FUsdScopedReferencer&&) = delete;

	private:
		TWeakObjectPtr<UUsdAssetCache3> AssetCache;
		const UObject* OldReferencer = nullptr;
	};

public:
	UE_API UUsdAssetCache3();

	UE_API virtual void PostLoad() override;
	UE_API virtual void BeginDestroy() override;
	UE_API virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif	  // WITH_EDITOR

	UE_API void OnRegistryAssetRenamed(const FAssetData& NewAssetData, const FString& OldName);

	UE_API void RequestDelayedAssetAutoCleanup();
	UE_API void TouchAsset(const FString& Hash, const UObject* Referencer = nullptr);
	UE_API void TouchAssetPath(const FSoftObjectPath& AssetPath, const UObject* Referencer = nullptr);
	UE_API void MarkAssetsAsStale();
	UE_API TSet<FSoftObjectPath> GetActiveAssets() const;
	UE_API const UObject* SetCurrentScopedReferencer(const UObject* NewReferencer);

public:
	/**
	 * Content directory where the asset cache will place newly created assets.
	 *
	 * Changing this directory to a new location will automatically try to cache any existing assets on that location, if they
	 * were generated from USD.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Storage", meta = (ContentDir, NoResetToDefault))
	FDirectoryPath AssetDirectory;

	/**
	 * When true, it means the asset cache will only ever return assets that are currently inside of the AssetDirectory folder.
	 * Move the assets out of the folder or change the folder and the asset cache will act as if these assets don't exist, potentially even losing
	 * track of them.
	 *
	 * When false, it means the asset cache will fully track and use its provided assets wherever they are in the content browser.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Storage")
	bool bOnlyHandleAssetsWithinAssetDirectory = false;

	/**
	 * This is the main internal property that maps hashes to asset paths.
	 *
	 * Add entries to this property (or modify existing entries) and they will be returned by the asset cache whenever that hash is queried.
	 *
	 * WARNING: Asset modifications are not currently tracked! Change a static mesh's vertex color from red to green, and it will show the
	 * green cube when opening a stage with this asset cache, even if you open stages where the prim contains red as its vertex color.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Storage")
	TMap<FString, FSoftObjectPath> HashToAssetPaths;

	/**
	 * If this is true, every time a UsdStageActor using this asset cache closes a stage or swaps asset caches it will attempt to call
	 * DeleteUnreferencedAssets, potentially dropping *any* unreferenced asset, due to this operation or previous ones.
	 *
	 * Enable this if you want your AssetDirectory folder to be automatically cleaned up as stages close, and don't plan on keeping
	 * other external references to those assets.
	 *
	 * Note: Some asset types may have complicated setups, and may end up with references from other properties, actors and components for
	 * some time (e.g. due to a component in a transient package or undo/redo buffer). This means any automatically cleanup may not
	 * manage to clean up *all* untracked assets. Subsequent cleanups should eventually collect all assets, however.
	 *
	 * WARNING: This will clear the undo buffer (i.e. transaction history) and run garbage collection after any cleanup operation!
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Cleanup")
	bool bCleanUpUnreferencedAssets = false;

protected:
	UE_API void AddReferenceInternal(const FString& Hash, const UObject* Referencer);
	UE_API FSoftObjectPath StopTrackingAssetInternal(const FString& Hash);
	UE_API bool RemoveAllAssetReferencersInternal(const FString& Hash);
	UE_API void TouchAssetInternal(const FSoftObjectPath& AssetPath, const UObject* Referencer = nullptr);
	UE_API void TryCachingAssetFromAssetUserData(const FAssetData& NewAssetData);
	UE_API bool IsTransientCache();
	UE_API void ForceValidAssetDirectoryInternal(bool bEmitWarning = true);

private:
	// Reverse map, to speed up queries like GetHashForAsset and IsAssetTrackedByCache. Should always match HashToAssetPaths
	UPROPERTY()
	TMap<FSoftObjectPath, FString> AssetPathToHashes;

	// If we're a transient asset cache, our assets will be placed on the transient package and there wouldn't necessarily be
	// anything holding a strong reference to them. This member is used for that. This is useful during direct import for example,
	// where a rogue GC call could otherwise cause our transient assets to be collected, if it happened at a bad time
	UPROPERTY(Transient)
	TMap<FString, TObjectPtr<UObject>> TransientObjectStorage;

	TMap<FString, TArray<FObjectKey>> HashToReferencer;
	TMap<FObjectKey, TArray<FString>> ReferencerToHash;
	TSet<FObjectKey> DeletableAssetKeys;

	// When this is set to something, we will automatically flag it as a referencer of any cached/touched asset
	const UObject* CurrentScopedReferencer = nullptr;

	bool bPendingCleanup = false;
	mutable FRWLock RWLock;

	// This member is mostly used by the UsdStageImporter: Assets are added to it whenever they are cached or fetched,
	// and reset by calling MarkAssetsAsStale(). The idea is that by resetting before the import, and then calling
	// GetActiveAssets() after the stage has been parsed, the importer can easily see which of the assets tracked
	// by the asset cache are actually used by the stage it is about to import, and then publish exactly those.
	mutable TSet<FSoftObjectPath> ActiveAssets;
};

#undef UE_API
