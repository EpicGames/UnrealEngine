// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorFramework/AssetImportData.h"

#include "USDAssetUserData.h"

#include "USDAssetImportData.generated.h"

UCLASS(MinimalAPI)
class UUsdAssetImportData : public UAssetImportData
{
	GENERATED_BODY()

public:
	UE_DEPRECATED(all, "Use USD AssetUserData instead of AssetImportData for USD-specific info")
	UPROPERTY()
	FString PrimPath_DEPRECATED;

	// Likely a UUSDStageImportOptions, but we don't declare it here
	// to prevent an unnecessary module dependency on USDStageImporter
	UPROPERTY()
	TObjectPtr<class UObject> ImportOptions;
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UCLASS()
class UE_DEPRECATED(all, "Use USD AssetUserData instead of AssetImportData for USD-specific info") USDCLASSES_API UUsdAnimSequenceAssetImportData
	: public UUsdAssetImportData
{
	GENERATED_BODY()

public:
	UPROPERTY()
	float LayerStartOffsetSeconds = 0.0f;
};

UCLASS()
class UE_DEPRECATED(all, "Use USD AssetUserData instead of AssetImportData for USD-specific info") USDCLASSES_API UUsdMeshAssetImportData
	: public UUsdAssetImportData
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TMap<int32, FUsdPrimPathList> MaterialSlotToPrimPaths;
};

PRAGMA_ENABLE_DEPRECATION_WARNINGS
