// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/AssetUserData.h"

#include "USDMetadata.h"

#include "USDAssetUserData.generated.h"

UCLASS(MinimalAPI, BlueprintType)
class UUsdAssetUserData : public UAssetUserData
{
	GENERATED_BODY()

public:
	// Paths to prims that generated the asset that owns this AssetUserData
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD")
	TArray<FString> PrimPaths;

	// Holds metadata collected for this asset, from all relevant Source prims.
	// The asset that owns this user data may be shared via the asset cache, and reused for
	// even entirely different stages. This map lets us keep track of which stage owns which
	// bits of metadata
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD")
	TMap<FString, FUsdCombinedPrimMetadata> StageIdentifierToMetadata;

	// Hash of the prim data that was used to generate this asset
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "USD")
	FString OriginalHash;
};

UCLASS(MinimalAPI, BlueprintType)
class UUsdAnimSequenceAssetUserData : public UUsdAssetUserData
{
	GENERATED_BODY()

public:
	// Offset into the stage in seconds to when this AnimSequence should start
	// playing to match the skeletal animation in that stage. This is required
	// because UAnimSequences just range from the first skeletal timeSample to the
	// last, and the first sample is not necessarily the startTimeCode for the
	// stage. Note that this is wrt. the startTimeCode of the *stage*, and not the
	// individual layer. This is used for animating USkeletalMeshComponents via
	// the AUsdStageActor and building the LevelSequences, and it is in seconds 
	// since it will need to drive the UAnimSequence with a time value in the 
	// [0, LengthSeconds] range. This should be applied *after* any offset/scale 
	// conversions on the time coordinate.
	UPROPERTY()
	float StageStartOffsetSeconds = 0.0f;

	// Total scale difference (as in, "MapToRoot") between the baked AnimSequence
	// playback speed and the stage playback speed.
	// E.g. If we have root.usda referencing ref1.usda with scale 2.0, and ref1.usda has
	// sublayer animseq.usda with scale 3.0 (where the SkelAnimation prim is authored),
	// then this property will hold 6.0
	UPROPERTY()
	float StageScaleSeconds = 1.0f;

	UE_DEPRECATED(5.7, "Use StageStartOffsetSeconds instead.")
	UPROPERTY()
	float LayerStartOffsetSeconds = 0.0f;
};

/** Simple wrapper because we're not allowed to have TMap properties with TArray<FString> as values */
USTRUCT(BlueprintType)
struct FUsdPrimPathList
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD")
	TArray<FString> PrimPaths;
};

/** AssetImportData assigned to Unreal materials that are generated when parsing USD Material prims */
UCLASS(MinimalAPI, BlueprintType)
class UUsdMaterialAssetUserData : public UUsdAssetUserData
{
	GENERATED_BODY()

public:
	/**
	 * In the context of our reference materials that just read a single texture for each material parameter, this
	 * describes the primvar that the USD material is sampling for each texture.
	 * e.g. {'BaseColor': 'st0', 'Metallic': 'st1'}
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD")
	TMap<FString, FString> ParameterToPrimvar;

	/**
	 * In the context of our reference materials that just read a single texture for each material parameter, this
	 * describes the Unreal UV index that will be used for sampling by each USD primvar. The idea is to use this in
	 * combination with ParameterToPrimvar, in order to describe which UV index the material has currently assigned to
	 * each parameter. When assigning the material to meshes later, we'll compare this member with the
	 * UUsdMeshAssetImportData's own PrimvarToUVIndex to see if they are compatible or not, and if not spawn a new
	 * material instance that is.
	 * e.g. {'firstPrimvar': 0, 'st': 1, 'st1': 2}
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD")
	TMap<FString, int32> PrimvarToUVIndex;
};

/** We assign these to UStaticMeshes or USkeletalMeshes generated from USD */
UCLASS(MinimalAPI, BlueprintType)
class UUsdMeshAssetUserData : public UUsdAssetUserData
{
	GENERATED_BODY()

public:
	/**
	 * Maps from a material slot index of an UStaticMesh or USkeletalMesh to a list of source prims that contain this
	 * assignment. It can contain multiple prims in case we combine material slots and/or collapse prims
	 * (e.g. {0: ['/Root/mesh', '/Root/othermesh/geomsubset0', '/Root/othermesh/geomsubset1'] }).
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD")
	TMap<int32, FUsdPrimPathList> MaterialSlotToPrimPaths;

	/** Describes which primvars should be assigned to each UV index. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD")
	TMap<FString, int32> PrimvarToUVIndex;
};

/** We assign these to UGeometryCaches generated from USD */
UCLASS(MinimalAPI, BlueprintType)
class UUsdGeometryCacheAssetUserData : public UUsdMeshAssetUserData
{
	GENERATED_BODY()

public:
	// Check analogous comment on UUsdAnimSequenceAssetUserData
	UPROPERTY()
	float StageStartOffsetSeconds = 0.0f;

	UE_DEPRECATED(5.7, "Use StageStartOffsetSeconds instead.")
	UPROPERTY()
	float LayerStartOffsetSeconds = 0.0f;
};

UCLASS(BlueprintType)
class UE_DEPRECATED(all, "This class is no longer used, as AUsdStageActors now use Sequencer Universal Object Locators for persistent bindings")
	USDCLASSES_API UUsdLevelSequenceAssetUserData : public UAssetUserData
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD")
	FGuid LastCheckedSignature;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD")
	TSet<FGuid> HandledBindingGuids;
};

/**
 * We use this mainly to help in mapping between stage timeCode and the FrameIndex for animated SVTs
 */
UCLASS(MinimalAPI, BlueprintType)
class UUsdSparseVolumeTextureAssetUserData : public UUsdAssetUserData
{
	GENERATED_BODY()

public:
	/** Paths to all the OpenVDBAsset prims that led to the generation of this SVT asset */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD")
	TArray<FString> SourceOpenVDBAssetPrimPaths;

	/** TimeCodes of all the filePath attribute time samples as seen on the OpenVDBAsset prim in its own layer, in order */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD")
	TArray<double> TimeSamplePathTimeCodes;

	/**
	 * Corresponding indices of the frame of the SVT that should be played at a particular timeCode.
	 * Example: TimeSamplePathTimeCodes is [10, 20] and TimeSamplePathIndices is [2, 7] --> At timeCode 10 the frame index 2
	 * of the SVT should be played, i.e. the .vdb file that is the third entry within TimeSamplePaths
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD")
	TArray<int32> TimeSamplePathIndices;

	/**
	 * File paths that originated each of the SVT frames, in order.
	 * The SVT should have as many frames as there are entries in this array.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD")
	TArray<FString> TimeSamplePaths;
};
