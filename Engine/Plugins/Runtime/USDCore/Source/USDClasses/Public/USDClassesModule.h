// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDMaterialUtils.h"	 // For backwards compatibility for a few releases since we moved the EUsdReferenceMaterialProperties enum

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

class UMaterialInstanceConstant;
class UMaterialInstanceDynamic;
class UUsdAssetCache3;
struct FAnalyticsEventAttribute;

class IUsdClassesModule : public IModuleInterface
{
public:
	/** Updates all plugInfo.json to point their LibraryPaths to TargetDllFolder */
	USDCLASSES_API static void UpdatePlugInfoFiles(const FString& PluginDirectory, const FString& TargetDllFolder);

	/**
	 * Sends analytics about a USD operation
	 * @param InAttributes - Additional analytics events attributes to send, along with new ones collected within this function
	 * @param EventName - Name of the analytics event (e.g. "Export.StaticMesh", so that the full event name is "Engine.Usage.USD.Export.StaticMesh")
	 * @param bAutomated - If the operation was automated (e.g. came from a Python script)
	 * @param ElapsedSeconds - How long the operation took in seconds
	 * @param NumberOfFrames - Number of time codes in the exported/imported/opened stage
	 * @param Extension - Extension of the main USD file opened/emitted/imported (e.g. "usda" or "usd")
	 */
	USDCLASSES_API static void SendAnalytics(
		TArray<FAnalyticsEventAttribute>&& InAttributes,
		const FString& EventName,
		bool bAutomated,
		double ElapsedSeconds,
		double NumberOfFrames,
		const FString& Extension
	);
	USDCLASSES_API static void SendAnalytics(TArray<FAnalyticsEventAttribute>&& InAttributes, const FString& EventName);

	USDCLASSES_API static FString GetClassNameForAnalytics(const UObject* Object);

	USDCLASSES_API static void AddAssetCountAttributes(const TSet<UObject*>& Assets, TArray<FAnalyticsEventAttribute>& InOutAttributes);

	USDCLASSES_API static void BlockAnalyticsEvents();
	USDCLASSES_API static void ResumeAnalyticsEvents();
	USDCLASSES_API static const TMap<FString, TArray<FAnalyticsEventAttribute>>& GetAccumulatedAnalytics();

	/**
	 * Updates HashToUpdate with the Object's package's persistent guid, the corresponding file save
	 * date and time, and the number of times the package has been dirtied since last being saved.
	 * This can be used to track the version of exported assets and levels, to prevent unnecessary re-exports.
	 */
	USDCLASSES_API static bool HashObjectPackage(const UObject* Object, FSHA1& HashToUpdate);

	/** Returns a world that could be suitably described as "the current world". (e.g. when in PIE, the PIE world) */
	USDCLASSES_API static UWorld* GetCurrentWorld(bool bEditorWorldsOnly = false);

	/**
	 * Returns the set of assets that this object depends on (e.g. when given a material, will return its textures.
	 * When given a mesh, will return materials, etc.)
	 */
	USDCLASSES_API static TSet<UObject*> GetAssetDependencies(UObject* Asset);

	// Returns the default asset cache for the project or create a new one at the project root
	USDCLASSES_API static UUsdAssetCache3* GetAssetCacheForProject();
};
