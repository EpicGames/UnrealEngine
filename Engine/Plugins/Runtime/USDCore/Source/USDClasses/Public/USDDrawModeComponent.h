// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/PrimitiveComponent.h"

#include "USDDrawModeComponent.generated.h"

#define UE_API USDCLASSES_API

class UMaterialInstance;
enum class EUsdUpAxis : uint8;

UENUM()
enum class EUsdDrawMode : int32
{
	Origin,
	Bounds,
	Cards,
	// For now we just hide these, so that the user can't pick them on the component details panel.
	// This because we don't yet have a good way of having component changes trigger resyncs: OnObjectPropertyChanged
	// is fired immediately for each property change, and resyncing right away may cause some trouble in case it
	// deletes components (which it will in this case) and the previous callstack of the OnObjectPropertyChanged still
	// wants to interact with the component, or fire OnObjectPropertyChanged for other properties, etc.
	Default UMETA(Hidden),
	Inherited UMETA(Hidden)
};

// When we're on EUsdDrawMode::Cards, describes how to draw our cards
UENUM()
enum class EUsdModelCardGeometry : int32
{
	Cross,
	Box,
	FromTexture UMETA(Hidden)	 // We don't really support this one, and will fallback to "Box"
};

UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EUsdModelCardFace : int32
{
	None = 0 UMETA(Hidden),
	XPos = 1,
	YPos = 2,
	ZPos = 4,
	XNeg = 8,
	YNeg = 16,
	ZNeg = 32
};
ENUM_CLASS_FLAGS(EUsdModelCardFace);

namespace UsdUtils
{
	USDCLASSES_API EUsdModelCardFace GetOppositeFaceOnSameAxis(EUsdModelCardFace Face);
}

/**
 * Component type that is used to draw bounds, cards and origin axes when translating a prim that has the UsdGeomModelAPI schema
 * applied.
 */
UCLASS(MinimalAPI, ClassGroup = (USD), meta = (BlueprintSpawnableComponent))
class UUsdDrawModeComponent : public UPrimitiveComponent
{
	GENERATED_BODY()

public:
	// Separate vectors instead of an FBox so we can get Sequencer tracks for these as is
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetBoundsMin, Category = "USD")
	FVector BoundsMin;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetBoundsMax, Category = "USD")
	FVector BoundsMax;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetDrawMode, Category = "USD")
	EUsdDrawMode DrawMode;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetBoundsColor, Category = "USD")
	FLinearColor BoundsColor;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetCardGeometry, Category = "USD|Cards")
	EUsdModelCardGeometry CardGeometry;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetCardTextureXPos, Category = "USD|Cards")
	TObjectPtr<UTexture2D> CardTextureXPos;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetCardTextureYPos, Category = "USD|Cards")
	TObjectPtr<UTexture2D> CardTextureYPos;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetCardTextureZPos, Category = "USD|Cards")
	TObjectPtr<UTexture2D> CardTextureZPos;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetCardTextureXNeg, Category = "USD|Cards")
	TObjectPtr<UTexture2D> CardTextureXNeg;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetCardTextureYNeg, Category = "USD|Cards")
	TObjectPtr<UTexture2D> CardTextureYNeg;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetCardTextureZNeg, Category = "USD|Cards")
	TObjectPtr<UTexture2D> CardTextureZNeg;

private:
	UPROPERTY()
	TArray<TObjectPtr<UMaterialInstance>> MaterialInstances;

	// EUsdModelCardFace flags to enable the card faces to draw.
	// We need these because the required USD behavior specifies a difference between not having a texture because
	// one wasn't authored (at which point we flip the opposite face), and not having a texture because it failed
	// to resolve (at which point we draw vertex color). In both of those cases we'll end up with nullptr on the
	// CardTexture property, but this will help us tell which case we're talking about.
	// Note that this must be an int32 instead of the actual EUsdModelCardFace type because otherwise during any
	// transaction the UPROPERTY Enum code will freak out if we have more than one flag set, presumably consider
	// the combination of flags an invalid value for the enum, and then just set it to zero instead (see UE-200646).
	UPROPERTY()
	int32 AuthoredFaces;

public:
	UFUNCTION(BlueprintSetter, Category = "USD")
	UE_API void SetBoundsMin(const FVector& NewMin);

	UFUNCTION(BlueprintSetter, Category = "USD")
	UE_API void SetBoundsMax(const FVector& NewMax);

	UFUNCTION(BlueprintSetter, Category = "USD")
	UE_API void SetDrawMode(EUsdDrawMode NewDrawMode);

	UFUNCTION(BlueprintSetter, Category = "USD")
	UE_API void SetBoundsColor(FLinearColor NewColor);

	UFUNCTION(BlueprintSetter, Category = "USD")
	UE_API void SetCardGeometry(EUsdModelCardGeometry NewGeometry);

	UFUNCTION(BlueprintSetter, Category = "USD|Cards")
	UE_API void SetCardTextureXPos(UTexture2D* NewTexture);

	UFUNCTION(BlueprintSetter, Category = "USD|Cards")
	UE_API void SetCardTextureYPos(UTexture2D* NewTexture);

	UFUNCTION(BlueprintSetter, Category = "USD|Cards")
	UE_API void SetCardTextureZPos(UTexture2D* NewTexture);

	UFUNCTION(BlueprintSetter, Category = "USD|Cards")
	UE_API void SetCardTextureXNeg(UTexture2D* NewTexture);

	UFUNCTION(BlueprintSetter, Category = "USD|Cards")
	UE_API void SetCardTextureYNeg(UTexture2D* NewTexture);

	UFUNCTION(BlueprintSetter, Category = "USD|Cards")
	UE_API void SetCardTextureZNeg(UTexture2D* NewTexture);

	UFUNCTION(BlueprintPure, Category = "USD|Cards")
	UE_API UTexture2D* GetTextureForFace(EUsdModelCardFace Face) const;

	UFUNCTION(BlueprintCallable, Category = "USD|Cards")
	UE_API void SetTextureForFace(EUsdModelCardFace Face, UTexture2D* Texture);

public:
	UE_API UUsdDrawModeComponent();
	virtual ~UUsdDrawModeComponent() override= default;

	// Begin UPrimitiveComponent interface
	UE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	UE_API virtual void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials = false) const override;
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual HHitProxy* CreateMeshHitProxy(int32 SectionIndex, int32 MaterialIndex) const override;
#endif
	// End UPrimitiveComponent interface

	UE_API EUsdModelCardFace GetAuthoredFaces() const;
	UE_API void SetAuthoredFaces(EUsdModelCardFace NewAuthoredFaces);

private:
	UE_API UMaterialInstance* GetOrCreateTextureMaterial(EUsdModelCardFace Face);
	UE_API void RefreshMaterialInstances();
};

#undef UE_API
