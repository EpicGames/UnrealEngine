// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDDuplicateType.generated.h"

/**
 * Describes the different types of "prim duplication" operations we support with UsdUtils::DuplicatePrims
 */
UENUM(BlueprintType)
enum class EUsdDuplicateType : uint8
{
	/** Generate a flattened duplicate of the composed prim onto the current edit target */
	FlattenComposedPrim,
	/** Duplicate the prim's specs on the current edit target only */
	SingleLayerSpecs,
	/** Duplicate each of the prim's specs across the entire stage */
	AllLocalLayerSpecs,
};
