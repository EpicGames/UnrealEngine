// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"

USDCLASSES_API DECLARE_LOG_CATEGORY_EXTERN(LogUsd, Log, All);
