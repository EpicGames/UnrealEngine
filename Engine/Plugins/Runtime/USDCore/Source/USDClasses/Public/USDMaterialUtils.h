// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API USDCLASSES_API

class UMaterialInstanceDynamic;
class UMaterialInstanceConstant;

enum class EUsdReferenceMaterialProperties : uint8
{
	None = 0,
	Translucent = 1,
	VT = 2,
	TwoSided = 4
};
ENUM_CLASS_FLAGS(EUsdReferenceMaterialProperties)

namespace UsdUnreal::MaterialUtils
{
	/** Describes the type of vertex color/DisplayColor material that we would need in order to render a prim's displayColor data as intended */
	struct FDisplayColorMaterial
	{
		bool bHasOpacity = false;
		bool bIsDoubleSided = false;

		UE_API FString ToString() const;
		UE_API FString ToPrettyString() const;
		static UE_API TOptional<FDisplayColorMaterial> FromString(const FString& DisplayColorString);
	};

	USDCLASSES_API const FSoftObjectPath* GetReferenceMaterialPath(const FDisplayColorMaterial& DisplayColorDescription);

	USDCLASSES_API UMaterialInstanceDynamic* CreateDisplayColorMaterialInstanceDynamic(const FDisplayColorMaterial& DisplayColorDescription);
	USDCLASSES_API UMaterialInstanceConstant* CreateDisplayColorMaterialInstanceConstant(const FDisplayColorMaterial& DisplayColorDescription);

	// Returns one of the alternatives of the UsdPreviewSurface reference material depending on the material overrides
	// provided, and nullptr otherwise
	USDCLASSES_API FSoftObjectPath GetReferencePreviewSurfaceMaterial(EUsdReferenceMaterialProperties ReferenceMaterialProperties);

	// Returns the VT version of the provided UsdPreviewSurface ReferenceMaterial. Returns the provided ReferenceMaterial back if
	// it is already a VT-capable reference material, and returns nullptr if ReferenceMaterial isn't one of our reference material
	// alternatives.
	// Example: Receives UsdPreviewSurfaceTwoSided -> Returns UsdPreviewSurfaceTwoSidedVT
	// Example: Receives UsdPreviewSurfaceTwoSidedVT -> Returns UsdPreviewSurfaceTwoSidedVT
	// Example: Receives SomeOtherReferenceMaterial -> Returns nullptr
	USDCLASSES_API FSoftObjectPath GetVTVersionOfReferencePreviewSurfaceMaterial(const FSoftObjectPath& ReferenceMaterial);

	// Returns the two-sided version of the provided UsdPreviewSurface ReferenceMaterial. Returns the provided ReferenceMaterial
	// back if it is already a two-sided-capable reference material, and returns nullptr if ReferenceMaterial isn't one of our reference
	// material alternatives.
	// Example: Receives UsdPreviewSurfaceTranslucent -> Returns UsdPreviewSurfaceTwoSidedTranslucent
	// Example: Receives UsdPreviewSurfaceTwoSidedTranslucent -> Returns UsdPreviewSurfaceTwoSidedTranslucent
	// Example: Receives SomeOtherReferenceMaterial -> Returns nullptr
	USDCLASSES_API FSoftObjectPath GetTwoSidedVersionOfReferencePreviewSurfaceMaterial(const FSoftObjectPath& ReferenceMaterial);

	// Returns whether the Material is one of the UsdPreviewSurface reference materials (which can be reassigned by the
	// user on a per project basis)
	USDCLASSES_API bool IsReferencePreviewSurfaceMaterial(const FSoftObjectPath& ReferenceMaterial);

	USDCLASSES_API void RegisterRenderContext(const FName& RenderContextName);
	USDCLASSES_API void UnregisterRenderContext(const FName& RenderContextName);
	USDCLASSES_API const TArray<FName>& GetRegisteredRenderContexts();
};	  // namespace UsdUnreal::MaterialUtils

#undef UE_API
