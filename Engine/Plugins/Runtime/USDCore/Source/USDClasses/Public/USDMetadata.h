// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDMetadata.generated.h"

// Describes a single metadata value collected from USD
USTRUCT(BlueprintType)
struct FUsdMetadataValue
{
	GENERATED_BODY()

	// USD typename. Anything from the "Value type token" column on the Basic data types tables from
	// https://openusd.org/release/api/_usd__page__datatypes.html is allowed, including
	// the array types (e.g. "uchar", "timecode", "matrix3d[]" and "half[]").
	// Exceptions are the "opaque" typeName that we don't support, and the "SdfListOp<Token>" typeName,
	// that we *do* support (it's the typeName for list-editable attributes like "apiSchemas")
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD")
	FString TypeName;

	// A stringified value that should match the type in TypeName (e.g. "[(1.0, 1.0, 0.5), (1.0, 1.0, 0.5)]" if
	// TypeName is "double3[]").
	// You can use the functions on UsdConversionLibrary (USDConversionLibrary.h) and UsdUtils namespace
	// (USDValueConversion.h) to help stringify/unstringify these types according to USD rules, from C++,
	// Blueprint and Python
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD")
	FString StringifiedValue;
};

// Describes all the metadata values collected from a particular USD prim
USTRUCT(BlueprintType)
struct FUsdPrimMetadata
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD")
	TMap<FString, FUsdMetadataValue> Metadata;
};

// Tracks metadata collected from multiple prim paths.
// This is useful because often multiple prims will be read to generate an asset, like when collapsing or
// collecting skinned Mesh prims for a SkeletalMesh.
// This may also be used if multiple prims within the same stage end up generating the same asset,
// which is shared via the asset cache: Both prims will store their metadata here.
USTRUCT(BlueprintType)
struct FUsdCombinedPrimMetadata
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD")
	TMap<FString, FUsdPrimMetadata> PrimPathToMetadata;
};
