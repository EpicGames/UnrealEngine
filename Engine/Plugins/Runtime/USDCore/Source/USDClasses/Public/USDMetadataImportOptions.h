// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDMetadataImportOptions.generated.h"

struct FAnalyticsEventAttribute;

USTRUCT(BlueprintType)
struct FUsdMetadataImportOptions
{
	GENERATED_BODY()

	// Whether to collect USD prim metadata into AssetUserData objects at all
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Metadata", config)
	bool bCollectMetadata = true;

	// Whether to collect USD metadata from not only a particular prim, but its entire subtree,
	// when apropriate. This is useful when used together with collapsing settings, for example.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Metadata", config, meta = (EditCondition = bCollectMetadata))
	bool bCollectFromEntireSubtrees = true;

	// We will always add the collected metadata to AssetUserData objects assigned to all generated assets
	// that support them. When this option is enabled, however, we will also separately collect USD prim
	// metadata and add that to AssetUserData objects added to every spawned *component*.
	// This can be useful for tracking metadata on prims that don't usually generate assets, like simple
	// Xforms, cameras, lights, etc., or for collecting metadata for prims with alternative draw modes
	// enabled, like bounds or cards: We won't generate the usual assets for those, but the metadata could
	// still be collected on the components with this option.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Metadata", config, meta = (EditCondition = bCollectMetadata))
	bool bCollectOnComponents = false;

	// When collecting metadata, we will ignore all entries that start with these prefixes.
	// Note that you can use the ":" separator for nested dictionaries, so for example using
	// "customData:ign" to ignore anything within the "ignoredValues" dictionary inside the
	// "customData" dictionary
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Metadata", config, meta = (EditCondition = bCollectMetadata))
	TArray<FString> BlockedPrefixFilters;

	// When this is false (default), the "BlockedPrefixFilters" property describe prefixes to block, and
	// metadata entries that match any of those prefixes are ignored and not collected.
	// When this is true, the "BlockedPrefixFilters" property is inverted, and describes prefixes to *allow*.
	// In that case, entries are only allowed and collected if they match at least one of the provided prefixes.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Metadata", config, meta = (EditCondition = bCollectMetadata))
	bool bInvertFilters = false;
};

namespace UsdUtils
{
	USDCLASSES_API void AddAnalyticsAttributes(const FUsdMetadataImportOptions& Options, TArray<FAnalyticsEventAttribute>& InOutAttributes);
	USDCLASSES_API void HashForImport(const FUsdMetadataImportOptions& Options, FSHA1& HashToUpdate);
}
