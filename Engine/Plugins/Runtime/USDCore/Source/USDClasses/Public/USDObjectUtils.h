// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SubclassOf.h"

class UUsdAssetCache3;
class UAssetImportData;
class UUsdAssetImportData;
class UUsdAssetUserData;

namespace UsdUnreal::ObjectUtils
{
	/**
	 * Utilities to allow getting and setting our AssetImportData to an asset from a base UObject*.
	 * Note that not all asset types support AssetImportData, and in some cases when retrieving it for e.g. a Skeleton,
	 * we'll actually check it's preview mesh instead (since Skeletons don't have AssetImportData). The setter won't do
	 * anything if you try setting asset import data on e.g. a Skeleton, on the other hand.
	 */
	USDCLASSES_API UAssetImportData* GetBaseAssetImportData(UObject* Asset);
	USDCLASSES_API UUsdAssetImportData* GetAssetImportData(UObject* Asset);
	USDCLASSES_API void SetAssetImportData(UObject* Asset, UAssetImportData* ImportData);

	/**
	 * Returns the object's UsdAssetUserData of a particular subclass if it has one
	 */
	USDCLASSES_API UUsdAssetUserData* GetAssetUserData(const UObject* Object, TSubclassOf<UUsdAssetUserData> Class = {});

	template<typename T>
	inline T* GetAssetUserData(UObject* Object)
	{
		return Cast<T>(GetAssetUserData(Object, T::StaticClass()));
	}

	/**
	 * Makes sure Object has an instance of UUsdAssetUserData of the provided subclass (defaulting to just UUsdAssetUserData itself) and returns it
	 */
	USDCLASSES_API UUsdAssetUserData* GetOrCreateAssetUserData(UObject* Object, TSubclassOf<UUsdAssetUserData> Class = {});

	template<typename T>
	inline T* GetOrCreateAssetUserData(UObject* Object)
	{
		return Cast<T>(GetOrCreateAssetUserData(Object, T::StaticClass()));
	}

	/**
	 * Removes all other UUsdAssetUserData instances from Object if they exist, then sets AssetUserData as Object's single UUsdAssetUserData
	 */
	USDCLASSES_API bool SetAssetUserData(UObject* Object, UUsdAssetUserData* AssetUserData);

	/**
	 * Returns the ideal class of AssetUserData to use for a particular asset type
	 * (e.g. UStaticMesh::StaticClass() --> UUsdMeshAssetUserData::StaticClass())
	 */
	USDCLASSES_API TSubclassOf<UUsdAssetUserData> GetAssetUserDataClassForObject(UClass* ObjectClass);

	// Adapted from ObjectTools as it is within an Editor-only module
	USDCLASSES_API FString SanitizeObjectName(const FString& InObjectName);

	USDCLASSES_API FString GetPrefixedAssetName(const FString& DesiredName, UClass* AssetClass);

	/**
	 * Removes any numbered suffix, followed by any number of underscores (e.g. Asset_2, Asset__232_31 or Asset94 all become 'Asset'), making
	 * sure the string is kept at least one character long. Returns true if it removed anything.
	 */
	USDCLASSES_API bool RemoveNumberedSuffix(FString& Prefix);

	/**
	 * Appends numbered suffixes to Name until the result is not contained in UsedNames, and returns it.
	 * Does not add the result to UsedNames before returning (as it is const).
	 * @param Name - Received string to make unique (e.g. "MyName")
	 * @param UsedNames - Strings that cannot be used for the result. Should have a .Contains() function that receives FString
	 * @return Modified Name so that it doesn't match anything in UsedNames (e.g. "MyName" again, or "MyName_0" or "MyName_423")
	 */
	template<typename StringContainer>
	inline FString GetUniqueName(FString Name, const StringContainer& UsedNames)
	{
		if (!UsedNames.Contains(Name))
		{
			return Name;
		}

		const bool bRemoved = RemoveNumberedSuffix(Name);

		// It's possible that removing the suffix made it into a unique name already
		if (bRemoved && !UsedNames.Contains(Name))
		{
			return Name;
		}

		int32 Suffix = 0;
		FString Result;
		do
		{
			Result = FString::Printf(TEXT("%s_%d"), *Name, Suffix++);
		} while (UsedNames.Contains(Result));

		return Result;
	}
};	  // namespace UsdUnreal::ObjectUtils
