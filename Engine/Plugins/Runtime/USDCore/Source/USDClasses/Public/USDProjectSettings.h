// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"
#include "Engine/EngineTypes.h"

#include "USDProjectSettings.generated.h"

UENUM(BlueprintType)
enum class EUsdSaveDialogBehavior : uint8
{
	NeverSave,
	AlwaysSave,
	ShowPrompt
};

UENUM(BlueprintType)
enum class EUsdEditInInstanceBehavior : uint8
{
	Ignore,
	RemoveInstanceable,
	ShowPrompt
};

UENUM(BlueprintType)
enum class EReferencerTypeHandling : uint8
{
	Ignore,
	MatchReferencedType,
	ClearReferencerType,
	ShowPrompt,
};

// We store these here so that we can return our project settings to these default values programmatically if we need to
namespace UsdDefaultReferenceMaterialPaths
{
	const inline FSoftObjectPath ReferencePreviewSurfaceMaterial{TEXT("/USDCore/Materials/UsdPreviewSurface.UsdPreviewSurface")};
	const inline FSoftObjectPath ReferencePreviewSurfaceTranslucentMaterial{TEXT("/USDCore/Materials/UsdPreviewSurfaceTranslucent.UsdPreviewSurfaceTranslucent")};
	const inline FSoftObjectPath ReferencePreviewSurfaceTwoSidedMaterial{TEXT("/USDCore/Materials/UsdPreviewSurfaceTwoSided.UsdPreviewSurfaceTwoSided")};
	const inline FSoftObjectPath ReferencePreviewSurfaceTranslucentTwoSidedMaterial{TEXT("/USDCore/Materials/UsdPreviewSurfaceTranslucentTwoSided.UsdPreviewSurfaceTranslucentTwoSided")};
	const inline FSoftObjectPath ReferencePreviewSurfaceVTMaterial{TEXT("/USDCore/Materials/UsdPreviewSurfaceVT.UsdPreviewSurfaceVT")};
	const inline FSoftObjectPath ReferencePreviewSurfaceTranslucentVTMaterial{TEXT("/USDCore/Materials/UsdPreviewSurfaceTranslucentVT.UsdPreviewSurfaceTranslucentVT")};
	const inline FSoftObjectPath ReferencePreviewSurfaceTwoSidedVTMaterial{TEXT("/USDCore/Materials/UsdPreviewSurfaceTwoSidedVT.UsdPreviewSurfaceTwoSidedVT")};
	const inline FSoftObjectPath ReferencePreviewSurfaceTranslucentTwoSidedVTMaterial{TEXT("/USDCore/Materials/UsdPreviewSurfaceTranslucentTwoSidedVT.UsdPreviewSurfaceTranslucentTwoSidedVT")};

	const inline FSoftObjectPath ReferenceSubstratePreviewSurfaceMaterial{TEXT("/InterchangeAssets/Substrate/UsdPreviewSurface.UsdPreviewSurface")};
	const inline FSoftObjectPath ReferenceSubstratePreviewSurfaceVTMaterial{TEXT("/InterchangeAssets/Substrate/UsdPreviewSurfaceVT.UsdPreviewSurfaceVT")};

	const inline FSoftObjectPath ReferenceDisplayColorMaterial{TEXT("/USDCore/Materials/DisplayColor.DisplayColor")};
	const inline FSoftObjectPath ReferenceDisplayColorAndOpacityMaterial{TEXT("/USDCore/Materials/DisplayColorAndOpacity.DisplayColorAndOpacity")};
	const inline FSoftObjectPath ReferenceDisplayColorTwoSidedMaterial{TEXT("/USDCore/Materials/DisplayColorTwoSided.DisplayColorTwoSided")};
	const inline FSoftObjectPath ReferenceDisplayColorAndOpacityTwoSidedMaterial{TEXT("/USDCore/Materials/DisplayColorAndOpacityTwoSided.DisplayColorAndOpacityTwoSided")};
}

// USDCore and defaultconfig here so this ends up at DefaultUSDCore.ini in the editor, and is sent to the
// packaged game as well

UCLASS(BlueprintType, config = USDCore, defaultconfig, meta = (DisplayName = USDCore), MinimalAPI)
class UUsdProjectSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	// Additional paths to check for USD plugins.
	//
	// If you want the USD plugins to be included in a packaged game, you must use a relative
	// path to a location within your project directory, and you must also add that same path
	// to the "Additional Non-Asset Directories To Copy" Project Packaging setting.
	//
	// For example, this relative path could be used to locate USD plugins in a directory at
	// the root of your project:
	//     ../USD_Plugins
	//
	// The packaging process cannot use an absolute path and will raise an error if given one
	// when it tries to concatenate the game content directory path with an absolute path.
	UPROPERTY(config, EditAnywhere, Category = USD, meta = (RelativeToGameContentDir))
	TArray<FDirectoryPath> AdditionalPluginDirectories;

	// The directories that will be used as the default search path by USD's default resolver
	// during asset resolution.
	//
	// Each directory in the search path should be an absolute path. If it is not, it will be
	// anchored to the current working directory.
	//
	// Note that the default search path must be set before the first invocation of USD's
	// resolver system, so changing this setting will require a restart of the engine in order
	// for the new setting to take effect.
	UPROPERTY(config, EditAnywhere, Category = USD, meta = (ConfigRestartRequired = true))
	TArray<FDirectoryPath> DefaultResolverSearchPath;

	// Material purposes to show on drop-downs in addition to the standard "preview" and "full"
	UPROPERTY(config, EditAnywhere, Category = USD)
	TArray<FName> AdditionalMaterialPurposes;

	// apiSchema names to show on the "Add Schema" right-click option, in addition to the standard common schemas
	UPROPERTY(config, EditAnywhere, Category = USD)
	TArray<FString> AdditionalCustomSchemaNames;

	// Whether to show on the output log messages, warnings and errors reported directly by the USD SDK
	UPROPERTY(config, EditAnywhere, Category = USD)
	bool bLogUsdSdkErrors = true;

	// Similar messages are merged into one. Disable this option to see the individual messages (may slow performance and use more memory).
	UPROPERTY(config, EditAnywhere, Category = USD)
	bool bOptimizeUsdLog = true;

	/**
	 * USD Asset Cache to use for USD Stage Actors that don't have any asset cache specified.
	 * Leave this empty to have each stage actor generate it's on transient cache instead.
	 */
	UPROPERTY(config, EditAnywhere, Category = USD, meta = (AllowedClasses = "/Script/USDClasses.UsdAssetCache3"))
	FSoftObjectPath DefaultAssetCache;

	UPROPERTY(config, EditAnywhere, Category = "USD|Dialogs")
	bool bShowCreateDefaultAssetCacheDialog = true;

	UPROPERTY(config, EditAnywhere, Category = "USD|Dialogs")
	bool bShowConfirmationWhenClearingLayers = true;

	UPROPERTY(config, EditAnywhere, Category = "USD|Dialogs")
	bool bShowConfirmationWhenMutingDirtyLayers = true;

	UPROPERTY(config, EditAnywhere, Category = "USD|Dialogs")
	bool bShowConfirmationWhenReloadingDirtyLayers = true;

	// Whether to show the warning dialog when authoring opinions that could have no effect on the composed stage
	UPROPERTY(config, EditAnywhere, Category = "USD|Dialogs")
	bool bShowOverriddenOpinionsWarning = true;

	// Whether to show the warning dialog when authoring opinions inside an instance or instance proxy
	UPROPERTY(config, EditAnywhere, Category = "USD|Dialogs")
	EUsdEditInInstanceBehavior EditInInstanceableBehavior = EUsdEditInInstanceBehavior::ShowPrompt;

	// How to behave when authoring a reference or payload to a prim whose type name differs from the one of the referencer prim
	UPROPERTY(config, EditAnywhere, Category = "USD|Dialogs")
	EReferencerTypeHandling ReferencerTypeHandling = EReferencerTypeHandling::ClearReferencerType;

	// Whether to show a warning whenever the "Duplicate All Local Layer Specs" option is picked, and the duplicated
	// prim has some specs outside the local layer stack that will not be duplicated.
	UPROPERTY(config, EditAnywhere, Category = "USD|Dialogs")
	bool bShowWarningOnIncompleteDuplication = true;

	// Whether to show the warning dialog when authoring a transforms directly to a camera component
	UPROPERTY(config, EditAnywhere, Category = "USD|Dialogs")
	bool bShowTransformOnCameraComponentWarning = true;

	// Whether to show the warning dialog when authoring a transform track directly to a camera component
	UPROPERTY(config, EditAnywhere, Category = "USD|Dialogs")
	bool bShowTransformTrackOnCameraComponentWarning = true;

	// Whether to show the warning dialog when snapping a subsequence section to the playback range
	UPROPERTY(config, EditAnywhere, Category = "USD|Dialogs")
	bool bShowSubsectionSnappingWarning = true;

	// Whether to show the warning dialog when authoring new visibility tracks from Unreal
	UPROPERTY(config, EditAnywhere, Category = "USD|Dialogs")
	bool bShowInheritedVisibilityWarning = true;

	// Whether to display the pop up dialog asking what to do about dirty USD layers when saving the UE level
	UPROPERTY(config, EditAnywhere, Category = "USD|Dialogs")
	EUsdSaveDialogBehavior ShowSaveLayersDialogWhenSaving = EUsdSaveDialogBehavior::ShowPrompt;

	// Whether to display the pop up dialog asking what to do about dirty USD layers when closing USD stages
	UPROPERTY(config, EditAnywhere, Category = "USD|Dialogs")
	EUsdSaveDialogBehavior ShowSaveLayersDialogWhenClosing = EUsdSaveDialogBehavior::ShowPrompt;

	// Note that the below properties being FSoftObjectPath ensure that these assets are cooked into packaged games

	UPROPERTY(config, EditAnywhere, Category = USD, meta = (AllowedClasses = "/Script/Engine.SoundAttenuation"))
	FSoftObjectPath DefaultSoundAttenuation = FSoftObjectPath{TEXT("/USDCore/USDDefaultAttenuation.USDDefaultAttenuation")};

	/**
	 * Material to use when handling .vdb files as Sparse Volume Textures. An instance of this material will be
	 * added to the AHeterogeneousVolume, and will use the parsed SparseVolumeTexture as a texture parameter.
	 * Note that alternatively Volume prims can have material bindings to Unreal materials, and the importer
	 * will prioritize trying to use those as the volumetric materials for the Sparse Volume Textures instead.
	 */
	UPROPERTY(config, EditAnywhere, Category = "USD|Reference Materials", meta = (AllowedClasses = "/Script/Engine.MaterialInterface"))
	FSoftObjectPath ReferenceDefaultSVTMaterial = FSoftObjectPath{TEXT("/Engine/EngineMaterials/SparseVolumeMaterial.SparseVolumeMaterial")};

	/**
	 * What material to use for UUsdDrawModeComponents with "Cards" draw mode and provided textures (corresponding to
	 * UsdGeomModelAPI with the "cards" drawMode).
	 * Each face of the card geometry will use a separate texture material instance, and the UTexture2D will be set
	 * as a material parameter named "Texture".
	 * You can swap this with your own material, but make sure the replacement material has a "Texture" parameter
	 */
	UPROPERTY(config, EditAnywhere, Category = "USD|Reference Materials", meta = (AllowedClasses = "/Script/Engine.MaterialInterface"))
	FSoftObjectPath ReferenceModelCardTextureMaterial = FSoftObjectPath{TEXT("/USDCore/Materials/CardTextureMaterial.CardTextureMaterial")};

	/**
	 * What material to use as reference material when creating material instances from USD materials.
	 * You can swap these with your own materials, but make sure that the replacement materials have parameters with
	 * the same names and types as the ones provided by the default material, otherwise the instances will not have
	 * the parameters filled with values extracted from the USD material when parsing.
	 */
	UPROPERTY(config, EditAnywhere, Category = "USD|Reference Materials|UsdPreviewSurface", meta = (AllowedClasses = "/Script/Engine.MaterialInterface"))
	FSoftObjectPath ReferencePreviewSurfaceMaterial = UsdDefaultReferenceMaterialPaths::ReferencePreviewSurfaceMaterial;

	/**
	 * What material to use as reference material when creating material instances from USD materials.
	 * You can swap these with your own materials, but make sure that the replacement materials have parameters with
	 * the same names and types as the ones provided by the default material, otherwise the instances will not have
	 * the parameters filled with values extracted from the USD material when parsing.
	 */
	UPROPERTY(config, EditAnywhere, Category = "USD|Reference Materials|UsdPreviewSurface", meta = (AllowedClasses = "/Script/Engine.MaterialInterface"))
	FSoftObjectPath ReferencePreviewSurfaceTranslucentMaterial = UsdDefaultReferenceMaterialPaths::ReferencePreviewSurfaceTranslucentMaterial;

	/**
	 * What material to use as reference material when creating material instances from USD materials.
	 * You can swap these with your own materials, but make sure that the replacement materials have parameters with
	 * the same names and types as the ones provided by the default material, otherwise the instances will not have
	 * the parameters filled with values extracted from the USD material when parsing.
	 */
	UPROPERTY(config, EditAnywhere, Category = "USD|Reference Materials|UsdPreviewSurface", meta = (AllowedClasses = "/Script/Engine.MaterialInterface"))
	FSoftObjectPath ReferencePreviewSurfaceTwoSidedMaterial = UsdDefaultReferenceMaterialPaths::ReferencePreviewSurfaceTwoSidedMaterial;

	/**
	 * What material to use as reference material when creating material instances from USD materials.
	 * You can swap these with your own materials, but make sure that the replacement materials have parameters with
	 * the same names and types as the ones provided by the default material, otherwise the instances will not have
	 * the parameters filled with values extracted from the USD material when parsing.
	 */
	UPROPERTY(config, EditAnywhere, Category = "USD|Reference Materials|UsdPreviewSurface", meta = (AllowedClasses = "/Script/Engine.MaterialInterface"))
	FSoftObjectPath ReferencePreviewSurfaceTranslucentTwoSidedMaterial = UsdDefaultReferenceMaterialPaths::ReferencePreviewSurfaceTranslucentTwoSidedMaterial;

	/**
	 * What material to use as reference material when creating material instances from USD materials.
	 * You can swap these with your own materials, but make sure that the replacement materials have parameters with
	 * the same names and types as the ones provided by the default material, otherwise the instances will not have
	 * the parameters filled with values extracted from the USD material when parsing.
	 */
	UPROPERTY(config, EditAnywhere, Category = "USD|Reference Materials|UsdPreviewSurface with Virtual Textures", meta = (AllowedClasses = "/Script/Engine.MaterialInterface"))
	FSoftObjectPath ReferencePreviewSurfaceVTMaterial = UsdDefaultReferenceMaterialPaths::ReferencePreviewSurfaceVTMaterial;

	/**
	 * What material to use as reference material when creating material instances from USD materials.
	 * You can swap these with your own materials, but make sure that the replacement materials have parameters with
	 * the same names and types as the ones provided by the default material, otherwise the instances will not have
	 * the parameters filled with values extracted from the USD material when parsing.
	 */
	UPROPERTY(config, EditAnywhere, Category = "USD|Reference Materials|UsdPreviewSurface with Virtual Textures", meta = (AllowedClasses = "/Script/Engine.MaterialInterface"))
	FSoftObjectPath ReferencePreviewSurfaceTranslucentVTMaterial = UsdDefaultReferenceMaterialPaths::ReferencePreviewSurfaceTranslucentVTMaterial;

	/**
	 * What material to use as reference material when creating material instances from USD materials.
	 * You can swap these with your own materials, but make sure that the replacement materials have parameters with
	 * the same names and types as the ones provided by the default material, otherwise the instances will not have
	 * the parameters filled with values extracted from the USD material when parsing.
	 */
	UPROPERTY(config, EditAnywhere, Category = "USD|Reference Materials|UsdPreviewSurface with Virtual Textures", meta = (AllowedClasses = "/Script/Engine.MaterialInterface"))
	FSoftObjectPath ReferencePreviewSurfaceTwoSidedVTMaterial = UsdDefaultReferenceMaterialPaths::ReferencePreviewSurfaceTwoSidedVTMaterial;

	/**
	 * What material to use as reference material when creating material instances from USD materials.
	 * You can swap these with your own materials, but make sure that the replacement materials have parameters with
	 * the same names and types as the ones provided by the default material, otherwise the instances will not have
	 * the parameters filled with values extracted from the USD material when parsing.
	 */
	UPROPERTY(config, EditAnywhere, Category = "USD|Reference Materials|UsdPreviewSurface with Virtual Textures", meta = (AllowedClasses = "/Script/Engine.MaterialInterface"))
	FSoftObjectPath ReferencePreviewSurfaceTranslucentTwoSidedVTMaterial = UsdDefaultReferenceMaterialPaths::ReferencePreviewSurfaceTranslucentTwoSidedVTMaterial;

	/**
	 * What material to use as reference material when creating material instances from USD materials.
	 * Exclusively used by USD imports via the Interchange framework, when Substrate is enabled for the project.
	 * You can swap these with your own materials, but make sure that the replacement materials have parameters with
	 * the same names and types as the ones provided by the default material, otherwise the instances will not have
	 * the parameters filled with values extracted from the USD material when parsing.
	 */
	UPROPERTY(config, EditAnywhere, Category = "USD|Reference Materials|UsdPreviewSurface", meta = (AllowedClasses = "/Script/Engine.MaterialInterface"))
	FSoftObjectPath ReferenceSubstratePreviewSurfaceMaterial = UsdDefaultReferenceMaterialPaths::ReferenceSubstratePreviewSurfaceMaterial;

	/**
	 * What material to use as reference material when creating material instances from USD materials.
	 * Exclusively used by USD imports via the Interchange framework, when Substrate is enabled for the project.
	 * You can swap these with your own materials, but make sure that the replacement materials have parameters with
	 * the same names and types as the ones provided by the default material, otherwise the instances will not have
	 * the parameters filled with values extracted from the USD material when parsing.
	 */
	UPROPERTY(config, EditAnywhere, Category = "USD|Reference Materials|UsdPreviewSurface", meta = (AllowedClasses = "/Script/Engine.MaterialInterface"))
	FSoftObjectPath ReferenceSubstratePreviewSurfaceVTMaterial = UsdDefaultReferenceMaterialPaths::ReferenceSubstratePreviewSurfaceVTMaterial;

	/**
	 * What material to use as reference material when creating material instances from USD materials.
	 * You can swap these with your own materials, but make sure that the replacement materials have parameters with
	 * the same names and types as the ones provided by the default material, otherwise the instances will not have
	 * the parameters filled with values extracted from the USD material when parsing.
	 */
	UPROPERTY(config, EditAnywhere, Category = "USD|Reference Materials|DisplayColor", meta = (AllowedClasses = "/Script/Engine.MaterialInterface"))
	FSoftObjectPath ReferenceDisplayColorMaterial = UsdDefaultReferenceMaterialPaths::ReferenceDisplayColorMaterial;

	/**
	 * What material to use as reference material when creating material instances from USD materials.
	 * You can swap these with your own materials, but make sure that the replacement materials have parameters with
	 * the same names and types as the ones provided by the default material, otherwise the instances will not have
	 * the parameters filled with values extracted from the USD material when parsing.
	 */
	UPROPERTY(config, EditAnywhere, Category = "USD|Reference Materials|DisplayColor", meta = (AllowedClasses = "/Script/Engine.MaterialInterface"))
	FSoftObjectPath ReferenceDisplayColorAndOpacityMaterial = UsdDefaultReferenceMaterialPaths::ReferenceDisplayColorAndOpacityMaterial;

	/**
	 * What material to use as reference material when creating material instances from USD materials.
	 * You can swap these with your own materials, but make sure that the replacement materials have parameters with
	 * the same names and types as the ones provided by the default material, otherwise the instances will not have
	 * the parameters filled with values extracted from the USD material when parsing.
	 */
	UPROPERTY(config, EditAnywhere, Category = "USD|Reference Materials|DisplayColor", meta = (AllowedClasses = "/Script/Engine.MaterialInterface"))
	FSoftObjectPath ReferenceDisplayColorTwoSidedMaterial = UsdDefaultReferenceMaterialPaths::ReferenceDisplayColorTwoSidedMaterial;

	/**
	 * What material to use as reference material when creating material instances from USD materials.
	 * You can swap these with your own materials, but make sure that the replacement materials have parameters with
	 * the same names and types as the ones provided by the default material, otherwise the instances will not have
	 * the parameters filled with values extracted from the USD material when parsing.
	 */
	UPROPERTY(config, EditAnywhere, Category = "USD|Reference Materials|DisplayColor", meta = (AllowedClasses = "/Script/Engine.MaterialInterface"))
	FSoftObjectPath ReferenceDisplayColorAndOpacityTwoSidedMaterial = UsdDefaultReferenceMaterialPaths::ReferenceDisplayColorAndOpacityTwoSidedMaterial;

public:
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
};
