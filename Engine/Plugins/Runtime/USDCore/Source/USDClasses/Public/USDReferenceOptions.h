// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDReferenceOptions.generated.h"

/**
 * Options to display when adding a reference or a payload for a prim
 */
UCLASS(MinimalAPI, Blueprintable)
class UUsdReferenceOptions : public UObject
{
	GENERATED_BODY()

public:
	// When enabled, the reference/payload will target a prim on this stage
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Options")
	bool bInternalReference = false;

	// File to use as the reference
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Options", meta = (EditCondition = "!bInternalReference", EditConditionHides))
	FFilePath TargetFile;

	// Use the default prim of the target stage as the referenced/payload prim
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Options", meta = (EditCondition = "!bInternalReference", EditConditionHides))
	bool bUseDefaultPrim = true;

	// Use a specific prim of the target stage as the referenced/payload prim
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Options", meta = (EditCondition = "!bUseDefaultPrim || bInternalReference"))
	FString TargetPrimPath;

	// Offset to apply to the referenced/payload prim's time sampled attributes
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Options", AdvancedDisplay)
	float TimeCodeOffset = 0.0f;

	// TimeCode scaling factor to apply to the referenced/payload prim's time sampled attributes
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Options", AdvancedDisplay)
	float TimeCodeScale = 1.0f;
};
