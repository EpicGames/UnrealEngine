// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/SetUtilities.h"
#include "Containers/UnrealString.h"

namespace UsdUtils
{
	/** Case sensitive hashing function for TMap */
	template<typename ValueType>
	struct FCaseSensitiveStringMapFuncs : BaseKeyFuncs<ValueType, FString, /*bInAllowDuplicateKeys*/ false>
	{
		static FORCEINLINE const FString& GetSetKey(const TPair<FString, ValueType>& Element)
		{
			return Element.Key;
		}

		static FORCEINLINE bool Matches(const FString& A, const FString& B)
		{
			return A.Equals(B, ESearchCase::CaseSensitive);
		}

		static FORCEINLINE uint32 GetKeyHash(const FString& Key)
		{
			return FCrc::StrCrc32<TCHAR>(*Key);
		}
	};

	/** Case sensitive hashing function for TSet */
	struct FCaseSensitiveStringSetFuncs : BaseKeyFuncs<FString, FString>
	{
		static FORCEINLINE const FString& GetSetKey(const FString& Element)
		{
			return Element;
		}
		static FORCEINLINE bool Matches(const FString& A, const FString& B)
		{
			return A.Equals(B, ESearchCase::CaseSensitive);
		}
		static FORCEINLINE uint32 GetKeyHash(const FString& Key)
		{
			return FCrc::StrCrc32<TCHAR>(*Key);
		}
	};

	template<typename ValueType>
	using TCaseSensitiveStringMap = TMap<FString, ValueType, FDefaultSetAllocator, FCaseSensitiveStringMapFuncs<ValueType>>;

	using FCaseSensitiveStringSet = TSet<FString, FCaseSensitiveStringSetFuncs>;
};	  // namespace UsdUnreal::ObjectUtils
