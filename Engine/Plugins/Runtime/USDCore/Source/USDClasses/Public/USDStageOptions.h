// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDStageOptions.generated.h"

struct FAnalyticsEventAttribute;

UENUM(BlueprintType)
enum class EUsdUpAxis : uint8
{
	YAxis,
	ZAxis,
};

USTRUCT(BlueprintType)
struct FUsdStageOptions
{
	GENERATED_BODY()

	/** MetersPerUnit to use for the stage. Defaults to 0.01 (i.e. 1cm per unit, which equals UE units) */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Stage options")
	double MetersPerUnit = 0.01;

	/** UpAxis to use for the stage. Defaults to ZAxis, which equals the UE convention */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Stage options")
	EUsdUpAxis UpAxis = EUsdUpAxis::ZAxis;
};

namespace UsdUtils
{
	USDCLASSES_API void AddAnalyticsAttributes(const FUsdStageOptions& Options, TArray<FAnalyticsEventAttribute>& InOutAttributes);

	USDCLASSES_API void HashForExport(const FUsdStageOptions& Options, FSHA1& HashToUpdate);
}
