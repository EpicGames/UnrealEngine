// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDUnrealAssetInfo.generated.h"

/**
 * Metadata added to a prim to indicate it was exported from a particular Unreal asset
 */
USTRUCT(BlueprintType)
struct FUsdUnrealAssetInfo
{
	GENERATED_BODY()

public:
	// Name of the Unreal asset
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Info")
	FString Name;

	// Filepath to the layer where the asset was exported to
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Info")
	FString Identifier;

	// Identifier string for the current asset version. Whenever the asset is updated inside Unreal, this will change
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Info")
	FString Version;

	// Path to the exported asset (e.g. "/Game/MyMaterials/Red.Red")
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Info")
	FString UnrealContentPath;

	// Class name of the exported asset (e.g. "StaticMesh")
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Info")
	FString UnrealAssetType;

	// DateTime string of the moment of export (e.g. "2022.06.17-14.19.54")
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Info")
	FString UnrealExportTime;

	// Engine version string describing the engine that exported this asset (e.g. "5.1.0-21000000+UE5")
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Info")
	FString UnrealEngineVersion;
};
