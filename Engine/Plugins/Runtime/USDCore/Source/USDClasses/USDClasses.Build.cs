// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class USDClasses : ModuleRules
	{
		public USDClasses(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Analytics",
					"Core",
					"CoreUObject",
					"Engine",
					"Slate",
					"SlateCore"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"DeveloperSettings",
					"GeometryCache",
					"GeometryFramework", // For BaseDynamicMeshSceneProxy.h
					"HairStrandsCore",
					"Json", // To read/write plugInfo.json files from UnrealUSDWrapper.cpp
					"LevelSequence",
					"MeshDescription",
					"RenderCore", // For UUsdDrawModeComponent implementation
					"RHI",
				}
			);

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"UnrealEd", // For ResetTransaction within UE::USDAssetCache3::Private::SafeDeleteObjects
						"MaterialEditor", // For SetMaterialInstanceParent, when managing the DisplayColor materials
					}
				);
			}
		}
	}
}
