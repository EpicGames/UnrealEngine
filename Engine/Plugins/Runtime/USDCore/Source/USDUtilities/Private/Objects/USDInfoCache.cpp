// Copyright Epic Games, Inc. All Rights Reserved.

#include "Objects/USDInfoCache.h"

#include "Objects/USDSchemaTranslator.h"
#include "USDErrorUtils.h"
#include "USDGeomMeshConversion.h"
#include "USDMemory.h"
#include "USDTypesConversion.h"

#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdStage.h"

#include "Async/ParallelFor.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/pcp/layerStack.h"
#include "pxr/usd/sdf/path.h"
#include "pxr/usd/usd/collectionAPI.h"
#include "pxr/usd/usd/prim.h"
#include "pxr/usd/usd/primCompositionQuery.h"
#include "pxr/usd/usd/primRange.h"
#include "pxr/usd/usdGeom/mesh.h"
#include "pxr/usd/usdGeom/pointInstancer.h"
#include "pxr/usd/usdGeom/scope.h"
#include "pxr/usd/usdGeom/subset.h"
#include "pxr/usd/usdGeom/xform.h"
#include "pxr/usd/usdShade/materialBindingAPI.h"
#include "pxr/usd/usdSkel/bindingAPI.h"
#include "pxr/usd/usdSkel/root.h"
#include "USDIncludesEnd.h"
#endif	  // USE_USD_SDK

#define LOCTEXT_NAMESPACE "UsdInfoCache"

static int32 GMaxNumVerticesCollapsedMesh = 5000000;
static FAutoConsoleVariableRef CVarMaxNumVerticesCollapsedMesh(
	TEXT("USD.MaxNumVerticesCollapsedMesh"),
	GMaxNumVerticesCollapsedMesh,
	TEXT("Maximum number of vertices that a combined Mesh can have for us to collapse it into a single StaticMesh")
);

// Can toggle on/off to compare performance with StaticMesh instead of GeometryCache
static bool GUseGeometryCacheUSD = true;
static FAutoConsoleVariableRef CVarUsdUseGeometryCache(
	TEXT("USD.GeometryCache.Enable"),
	GUseGeometryCacheUSD,
	TEXT("Use GeometryCache instead of static meshes for loading animated meshes")
);

static int32 GGeometryCacheMaxDepth = 15;
static FAutoConsoleVariableRef CVarGeometryCacheMaxDepth(
	TEXT("USD.GeometryCache.MaxDepth"),
	GGeometryCacheMaxDepth,
	TEXT("Maximum distance between an animated mesh prim to its collapsed geometry cache root")
);

static int32 GNumPerPrimLocks = 32;
static FAutoConsoleVariableRef CVarNumPerPrimLocks(
	TEXT("USD.NumPerPrimLocks"),
	GNumPerPrimLocks,
	TEXT(
		"Maximum number of locks that are distributed between all the prim info structs that the USDInfoCache keeps internally. More locks can imply better performance for the info cache build, but the total number of locks available on the system is finite"
	)
);

namespace UE::UsdInfoCache::Private
{
	// Flags to hint at the state of a prim for the purpose of geometry cache
	enum class EGeometryCachePrimState : uint8
	{
		None = 0x00,
		Uncollapsible = 0x01,		   // prim cannot be collapsed as part of a geometry cache
		Mesh = 0x02,				   // prim is a mesh, animated or not
		Xform = 0x04,				   // prim is a xform, animated or not
		Collapsible = Mesh | Xform,	   // only meshes and xforms can be collapsed into a geometry cache
		ValidRoot = 0x08			   // prim can collapse itself and its children into a geometry cache
	};
	ENUM_CLASS_FLAGS(EGeometryCachePrimState)

	struct FUsdPrimInfo
	{
		UE::FSdfPath PrimPath;

		uint64 ParentInfoIndex = static_cast<uint64>(INDEX_NONE);
		TArray<uint64> ChildIndices;

		int32 PrimLockIndex = INDEX_NONE;

		// If this is true, it means this prim can and wants to collapse its entire subtree.
		// If false, it either doesn't collapse its subtree, or we haven't visited it yet (same result)
		bool bCollapsesChildren = false;

		// Whether this prim can be collapsed or not, according to its schema translator
		// - Optional is not set: Prim wasn't visited yet, we don't know
		// - Optional has value: Whether the prim can be collapsed or not
		TOptional<bool> bXformSubtreeCanBeCollapsed;

		// This is used as a "visited" marker for RecursivePropagateVertexAndMaterialSlotCounts
		TOptional<uint64> ExpectedVertexCountForSubtree;
		TArray<UsdUtils::FUsdPrimMaterialSlot> SubtreeMaterialSlots;
		bool bSlotsWereMerged = false;

		int32 GeometryCacheDepth = -1;
		EGeometryCachePrimState GeometryCacheState = EGeometryCachePrimState::None;

		// Maps from prims, to all the prims that require also reading this prim to be translated into an asset.
		// Mainly used to update these assets whenever the depencency prim is updated.
		TSet<UE::FSdfPath> MaterialUserMainPrims;
		TSet<UE::FSdfPath> MainPrims;
		TSet<UE::FSdfPath> AuxPrims;
		bool bInstanceAuxPrimsRegistered = false;

		void ResetCollectedInfo()
		{
			bCollapsesChildren = false;

			bXformSubtreeCanBeCollapsed.Reset();

			ExpectedVertexCountForSubtree.Reset();
			SubtreeMaterialSlots.Reset();
			bSlotsWereMerged = false;

			GeometryCacheDepth = -1;
			GeometryCacheState = EGeometryCachePrimState::None;

			bInstanceAuxPrimsRegistered = false;
		}
	};
}	 // namespace UE::UsdInfoCache::Private

FArchive& operator<<(FArchive& Ar, UE::UsdInfoCache::Private::FUsdPrimInfo& Info)
{
	Ar << Info.PrimPath;

	Ar << Info.ParentInfoIndex;
	Ar << Info.ChildIndices;

	Ar << Info.PrimLockIndex;

	Ar << Info.bCollapsesChildren;

	Ar << Info.ExpectedVertexCountForSubtree;
	Ar << Info.SubtreeMaterialSlots;
	Ar << Info.bSlotsWereMerged;

	Ar << Info.GeometryCacheDepth;
	Ar << Info.GeometryCacheState;

	Ar << Info.MaterialUserMainPrims;
	Ar << Info.MainPrims;
	Ar << Info.AuxPrims;
	Ar << Info.bInstanceAuxPrimsRegistered;

	return Ar;
}

struct FUsdInfoCacheImpl
{
	FUsdInfoCacheImpl()
		: AllowedExtensionsForGeometryCacheSource(UnrealUSDWrapper::GetNativeFileFormats())
	{
		AllowedExtensionsForGeometryCacheSource.Add(TEXT("abc"));

		FScopedUnrealAllocs Allocs;
		PrimLocks = new FRWLock[GNumPerPrimLocks]();
	}

	FUsdInfoCacheImpl& operator=(const FUsdInfoCacheImpl& Other)
	{
		FReadScopeLock OtherScopedInfoMapLock(Other.InfoMapLock);
		FWriteScopeLock ThisScopedInfoMapLock(InfoMapLock);
		InfoMap = Other.InfoMap;
		PrimInfoArray = Other.PrimInfoArray;
		StaleInfoIndices = Other.StaleInfoIndices;

		AllowedExtensionsForGeometryCacheSource = Other.AllowedExtensionsForGeometryCacheSource;
		return *this;
	}

	~FUsdInfoCacheImpl()
	{
		FScopedUnrealAllocs Allocs;
		delete[] PrimLocks;
	}

	// Information we must have about all prims on the stage
	TArray<UE::UsdInfoCache::Private::FUsdPrimInfo> PrimInfoArray;
	TArray<uint64> StaleInfoIndices;
	TMap<UE::FSdfPath, uint64> InfoMap;
	mutable FRWLock InfoMapLock;

	// Temporarily used during the info cache build, as we need to do another pass on point instancers afterwards
	TArray<FString> TempPointInstancerPaths;
	mutable FRWLock TempPointInstancerPathsLock;

	TSet<UE::FSdfPath> TempUpdatedPrototypes;
	mutable FRWLock TempUpdatedPrototypesLock;

	TMap<UE::FSdfPath, TSet<UE::FSdfPath>> TempStaleMaterialUserMainPrims;
	mutable FRWLock TempStaleMaterialUserMainPrimsLock;

	TSet<uint64> TempPrimsToDisconnectAuxLinks;
	mutable FRWLock TempPrimsToDisconnectAuxLinksLock;

	// This is used to keep track of which prototypes are already being translated within this "translation session",
	// so that the schema translators can early out if they're trying to translate multiple instances of the same
	// prototype
	TSet<UE::FSdfPath> TranslatedPrototypes;
	mutable FRWLock TranslatedPrototypesLock;

	// Geometry cache can come from a reference or payload of these file types
	TArray<FString> AllowedExtensionsForGeometryCacheSource;

private:
	// Individual locks distributed across the FUsdPrimInfo
	FRWLock* PrimLocks = nullptr;

public:
	void ClearTransientInfo()
	{
		{
			FWriteScopeLock Lock(TempPointInstancerPathsLock);
			TempPointInstancerPaths.Empty();
		}
		{
			FWriteScopeLock Lock(TempUpdatedPrototypesLock);
			TempUpdatedPrototypes.Empty();
		}
		{
			FWriteScopeLock Lock(TempStaleMaterialUserMainPrimsLock);
			TempStaleMaterialUserMainPrims.Empty();
		}
		{
			FWriteScopeLock Lock(TempPrimsToDisconnectAuxLinksLock);
			TempPrimsToDisconnectAuxLinks.Empty();
		}
	}

	// WARNING: Assumes that the info map is locked for reading
	UE::UsdInfoCache::Private::FUsdPrimInfo* GetPrimInfo(const UE::FSdfPath& PrimPath) const
	{
		if (const uint64* Index = InfoMap.Find(PrimPath))
		{
			return const_cast<UE::UsdInfoCache::Private::FUsdPrimInfo*>(&PrimInfoArray[*Index]);
		}

		return nullptr;
	}

	[[nodiscard]] FReadScopeLock LockForReading(const UE::UsdInfoCache::Private::FUsdPrimInfo& Info) const
	{
		return FReadScopeLock{PrimLocks[Info.PrimLockIndex]};
	}

	[[nodiscard]] FWriteScopeLock LockForWriting(const UE::UsdInfoCache::Private::FUsdPrimInfo& Info) const
	{
		return FWriteScopeLock{PrimLocks[Info.PrimLockIndex]};
	}

	UE::UsdInfoCache::Private::FUsdPrimInfo& CreateNewInfo(const UE::FSdfPath& PrimPath, uint64& OutNewIndex)
	{
		using namespace UE::UsdInfoCache::Private;

		if (StaleInfoIndices.Num() > 0)
		{
			OutNewIndex = StaleInfoIndices.Pop();

			FUsdPrimInfo& Result = PrimInfoArray[OutNewIndex];
			Result = FUsdPrimInfo{};	// Get rid of old data since we're reusing an entry
			Result.PrimPath = PrimPath;
			return Result;
		}
		else
		{
			OutNewIndex = PrimInfoArray.Num();

			FUsdPrimInfo& Result = PrimInfoArray.Emplace_GetRef();
			Result.PrimPath = PrimPath;
			return Result;
		}
	}

	// This invalidates all data collected for a particular prim, but retains an info entry for
	// that prim, as well as parent/child indices
	//
	// WARNING: This assumes the InfoMap is locked for at least reading.
	// WARNING: Not thread safe (only called during the partial cleanup, which is single-threaded)
	void ResetPrimInfoEntry(uint64 Index)
	{
		using namespace UE::UsdInfoCache::Private;

		FUsdPrimInfo& Info = PrimInfoArray[Index];

		// We're already reset, early out
		if (!Info.ExpectedVertexCountForSubtree.IsSet())
		{
			return;
		}

		Info.ResetCollectedInfo();

		// Remember to wipe our main/aux links later after we've done traversing them
		TempPrimsToDisconnectAuxLinks.Add(Index);

		// Propagate to main prims
		for (const UE::FSdfPath& MainPath : Info.MainPrims)
		{
			if (uint64* MainPrimIndex = InfoMap.Find(MainPath))
			{
				ResetPrimInfoEntry(*MainPrimIndex);
			}
		}

		// Propagate to ancestors
		if (Info.ParentInfoIndex != INDEX_NONE)
		{
			ResetPrimInfoEntry(Info.ParentInfoIndex);
		}
	}

	// Fully discards all data and all parent/child indices collected for a prim at InfoIndex.
	// In practice this will just mark those entries/indices as stale so that we can reuse them later.
	//
	// WARNING: This assumes the InfoMap is locked for writing.
	// WARNING: Not thread safe (only called during the partial cleanup, which is single-threaded)
	void RemovePrimInfoSubtree(uint64 InfoIndex)
	{
		using namespace UE::UsdInfoCache::Private;

		// We do this so that we recursively step through all main links as well.
		// The main use case for this is point instancers with prototypes outside their own hierarchy. If we
		// edit those prototypes, they may end up with a different vertex count or material slot count, so
		// we need to update those counts for the point instancer's info struct (and so its ancestors) too.
		// It may seem slow to do this recursively as we traverse into children, but these Main links usually
		// point to ancestors anyway, and ResetPrimInfoEntry is quick to early out when we step into a visited
		// info struct
		ResetPrimInfoEntry(InfoIndex);

		FUsdPrimInfo& Info = PrimInfoArray[InfoIndex];

		for (uint64 ChildIndex : Info.ChildIndices)
		{
			FUsdPrimInfo& ChildInfo = PrimInfoArray[ChildIndex];
			RemovePrimInfoSubtree(ChildIndex);
		}

		InfoMap.Remove(Info.PrimPath);
		StaleInfoIndices.Push(InfoIndex);
	};

	// We have a separate function for this (not directly called from ResetPrimInfoEntry or other functions)
	// because we want to unregister these aux links for a subtree only after we have have traversed the subtree
	// resetting everything we need. Otherwise we may end up removing these aux links before we've had a chance
	// to traverse them
	//
	// WARNING: This assumes the InfoMap is locked for at least reading.
	// WARNING: Not thread safe (only called during the partial cleanup, which is single-threaded)
	void DisconnectResetPrimAuxLinks()
	{
		using namespace UE::UsdInfoCache::Private;

		for (uint64 Index : TempPrimsToDisconnectAuxLinks)
		{
			FUsdPrimInfo& Info = PrimInfoArray[Index];

			// Disconnect our Aux links
			for (const UE::FSdfPath& AuxPath : Info.AuxPrims)
			{
				if (FUsdPrimInfo* AuxPrimInfo = GetPrimInfo(AuxPath))
				{
					AuxPrimInfo->MainPrims.Remove(Info.PrimPath);
				}
			}
			Info.AuxPrims.Reset();

			// Disconnect our Main links
			for (const UE::FSdfPath& MainPath : Info.MainPrims)
			{
				if (FUsdPrimInfo* MainPrimInfo = GetPrimInfo(MainPath))
				{
					MainPrimInfo->AuxPrims.Remove(Info.PrimPath);
				}
			}
			Info.MainPrims.Reset();

			// Disconnect our MaterialUserMain links
			for (const UE::FSdfPath& MainPath : Info.MaterialUserMainPrims)
			{
				if (FUsdPrimInfo* MainPrimInfo = GetPrimInfo(MainPath))
				{
					MainPrimInfo->AuxPrims.Remove(Info.PrimPath);
				}
			}
			if (Info.MaterialUserMainPrims.Num() > 0)
			{
				// Stash these before we reset them
				//
				// This because we can't recompute these easily if all we're resyncing is the material: The material users are just
				// random prims in the stage. If we're resyncing just the material and the users are unmodified, then they really
				// have the exact same material binding anyway, so we can use this data to quickly restore material users to these prim
				// info structs, if they happen to match the same prim path of the resynced material prim
				TempStaleMaterialUserMainPrims.FindOrAdd(Info.PrimPath).Append(Info.MaterialUserMainPrims);
				Info.MaterialUserMainPrims.Reset();
			}
		}
	}

	void RegisterAuxiliaryPrims(const UE::FSdfPath& MainPrimPath, const TSet<UE::FSdfPath>& AuxPrimPaths)
	{
		using namespace UE::UsdInfoCache::Private;

		TRACE_CPUPROFILER_EVENT_SCOPE(RegisterAuxiliaryPrims);

		if (AuxPrimPaths.Num() == 0)
		{
			return;
		}

		FReadScopeLock ScopeLock{InfoMapLock};

		if (FUsdPrimInfo* MainPrim = GetPrimInfo(MainPrimPath))
		{
			FWriteScopeLock Lock = LockForWriting(*MainPrim);
			MainPrim->AuxPrims.Append(AuxPrimPaths);
		}

		for (const UE::FSdfPath& AuxPrimPath : AuxPrimPaths)
		{
			if (FUsdPrimInfo* AuxPrim = GetPrimInfo(AuxPrimPath))
			{
				FWriteScopeLock Lock = LockForWriting(*AuxPrim);
				AuxPrim->MainPrims.Add(MainPrimPath);
			}
		}
	}

	void RegisterMaterialUserPrims(const UE::FSdfPath& MaterialPath, const TSet<UE::FSdfPath>& UserPaths)
	{
		using namespace UE::UsdInfoCache::Private;

		TRACE_CPUPROFILER_EVENT_SCOPE(RegisterMaterialUserPrims);

		if (UserPaths.Num() == 0)
		{
			return;
		}

		FReadScopeLock ScopeLock{InfoMapLock};

		if (FUsdPrimInfo* Material = GetPrimInfo(MaterialPath))
		{
			FWriteScopeLock Lock = LockForWriting(*Material);
			Material->MaterialUserMainPrims.Append(UserPaths);
		}

		for (const UE::FSdfPath& UserPath : UserPaths)
		{
			if (FUsdPrimInfo* User = GetPrimInfo(UserPath))
			{
				FWriteScopeLock Lock = LockForWriting(*User);
				User->AuxPrims.Add(MaterialPath);
			}
		}
	}

	void TryRestoreMaterialUserLinks(UE::UsdInfoCache::Private::FUsdPrimInfo& MaterialInfo)
	{
		using namespace UE::UsdInfoCache::Private;

		FReadScopeLock ScopeLock{TempStaleMaterialUserMainPrimsLock};

		const TSet<UE::FSdfPath>* OldMaterialUsers = nullptr;
		UE::FSdfPath MaterialPrimPath;
		{
			FWriteScopeLock PrimLock = LockForWriting(MaterialInfo);

			OldMaterialUsers = TempStaleMaterialUserMainPrims.Find(MaterialInfo.PrimPath);
			if (!OldMaterialUsers || OldMaterialUsers->Num() == 0)
			{
				// We don't have any old users registered for this material prim path, just return
				return;
			}
			MaterialPrimPath = MaterialInfo.PrimPath;

			MaterialInfo.MaterialUserMainPrims.Append(*OldMaterialUsers);
		}

		for (const UE::FSdfPath& OldUserPath : *OldMaterialUsers)
		{
			if (FUsdPrimInfo* OldUser = GetPrimInfo(OldUserPath))
			{
				FWriteScopeLock Lock = LockForWriting(*OldUser);
				OldUser->AuxPrims.Add(MaterialPrimPath);
			}
		}
	}

#if USE_USD_SDK
	bool IsPotentialGeometryCacheRootInner(UE::UsdInfoCache::Private::FUsdPrimInfo& Info, const pxr::UsdPrim& Prim)
	{
		using namespace UE::UsdInfoCache::Private;

		FWriteScopeLock PrimLock = LockForWriting(Info);

		// When importing we fill all those in during the info cache initial build. If this is None still, it means
		// we're in the default geometry cache workflow for opening the stage, where geometry caches are generated
		// directly for single animated Mesh prims (so no collapsing of whole subtrees into geometry caches). We can
		// then find out if our prim is animated on-demand
		if (Info.GeometryCacheState == EGeometryCachePrimState::None)
		{
			Info.GeometryCacheState = UsdUtils::IsAnimatedMesh(Prim) ? EGeometryCachePrimState::ValidRoot : EGeometryCachePrimState::Uncollapsible;
		}
		return Info.GeometryCacheState == EGeometryCachePrimState::ValidRoot;
	}
#endif	  // USE_USD_SDK
};

FUsdInfoCache::FUsdInfoCache()
{
	// Make sure the UnrealUSDWrapper module is loaded first since FUsdInfoCacheImpl will call UnrealUSDWrapper::GetNativeFileFormats
	const FName ModuleName = TEXT("UnrealUSDWrapper");
	FModuleManager& Manager = FModuleManager::Get();
	if (!Manager.IsModuleLoaded(ModuleName))
	{
		Manager.LoadModule(ModuleName);
	}

	Impl = MakeUnique<FUsdInfoCacheImpl>();
}

FUsdInfoCache::~FUsdInfoCache()
{
}

void FUsdInfoCache::CopyImpl(const FUsdInfoCache& Other)
{
	*Impl = *Other.Impl;
}

bool FUsdInfoCache::Serialize(FArchive& Ar)
{
	if (FUsdInfoCacheImpl* ImplPtr = Impl.Get())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FUsdInfoCache::Serialize);
		{
			FWriteScopeLock ScopeLock(ImplPtr->InfoMapLock);
			Ar << ImplPtr->InfoMap;
			Ar << ImplPtr->PrimInfoArray;
			Ar << ImplPtr->StaleInfoIndices;
		}
	}

	return true;
}

bool FUsdInfoCache::ContainsInfoAboutPrim(const UE::FSdfPath& Path) const
{
	if (FUsdInfoCacheImpl* ImplPtr = Impl.Get())
	{
		FReadScopeLock ScopeLock(ImplPtr->InfoMapLock);
		return ImplPtr->InfoMap.Contains(Path);
	}

	return false;
}

TArray<UE::FSdfPath> FUsdInfoCache::GetChildren(const UE::FSdfPath& ParentPath) const
{
	using namespace UE::UsdInfoCache::Private;

	TArray<UE::FSdfPath> Result;

	if (FUsdInfoCacheImpl* ImplPtr = Impl.Get())
	{
		FReadScopeLock ScopeLock(ImplPtr->InfoMapLock);
		if (const FUsdPrimInfo* FoundInfo = ImplPtr->GetPrimInfo(ParentPath))
		{
			FReadScopeLock PrimLock = ImplPtr->LockForReading(*FoundInfo);

			Result.Reserve(FoundInfo->ChildIndices.Num());
			for (uint64 ChildIndex : FoundInfo->ChildIndices)
			{
				const FUsdPrimInfo& ChildInfo = ImplPtr->PrimInfoArray[ChildIndex];
				FReadScopeLock ChildLock = ImplPtr->LockForReading(ChildInfo);
				Result.Add(ChildInfo.PrimPath);
			}
		}
	}

	return Result;
}

bool FUsdInfoCache::IsPathCollapsed(const UE::FSdfPath& Path, ECollapsingType CollapsingType) const
{
	using namespace UE::UsdInfoCache::Private;

	if (FUsdInfoCacheImpl* ImplPtr = Impl.Get())
	{
		FReadScopeLock ScopeLock(ImplPtr->InfoMapLock);

		if (uint64* PrimIndex = ImplPtr->InfoMap.Find(Path))
		{
			uint64 IterIndex = INDEX_NONE;
			{
				// We're only collapsed if a parent collapses us
				const FUsdPrimInfo& Info = ImplPtr->PrimInfoArray[*PrimIndex];
				FReadScopeLock PrimLock = ImplPtr->LockForReading(Info);
				IterIndex = Info.ParentInfoIndex;
			}

			while (IterIndex != INDEX_NONE)
			{
				const FUsdPrimInfo& Info = ImplPtr->PrimInfoArray[IterIndex];
				FReadScopeLock PrimLock = ImplPtr->LockForReading(Info);
				if (Info.bCollapsesChildren)
				{
					return true;
				}

				IterIndex = Info.ParentInfoIndex;
			}

			return false;
		}

		// This should never happen: We should have cached the entire tree
		ensureMsgf(false, TEXT("Prim path '%s' has not been cached!"), *Path.GetString());
	}

	return false;
}

bool FUsdInfoCache::DoesPathCollapseChildren(const UE::FSdfPath& Path, ECollapsingType CollapsingType) const
{
	using namespace UE::UsdInfoCache::Private;

	if (FUsdInfoCacheImpl* ImplPtr = Impl.Get())
	{
		FReadScopeLock ScopeLock(ImplPtr->InfoMapLock);

		if (uint64* PrimIndex = ImplPtr->InfoMap.Find(Path))
		{
			uint64 IterIndex = INDEX_NONE;
			{
				const FUsdPrimInfo& Info = ImplPtr->PrimInfoArray[*PrimIndex];

				FReadScopeLock PrimLock = ImplPtr->LockForReading(Info);
				if (!Info.bCollapsesChildren)
				{
					// If this prim doesn't even want to collapse its children, we're done
					return false;
				}

				IterIndex = Info.ParentInfoIndex;
			}

			// Even if this prim wants to collapse its children though, it could be that it's collapsed
			// by a parent instead (collapsing is always done top-down)
			while (IterIndex != INDEX_NONE)
			{
				const FUsdPrimInfo& AncestorInfo = ImplPtr->PrimInfoArray[IterIndex];

				FReadScopeLock PrimLock = ImplPtr->LockForReading(AncestorInfo);
				if (AncestorInfo.bCollapsesChildren)
				{
					return false;
				}

				IterIndex = AncestorInfo.ParentInfoIndex;
			}

			return true;
		}

		// This should never happen: We should have cached the entire tree
		ensureMsgf(false, TEXT("Prim path '%s' has not been cached!"), *Path.GetString());
	}

	return false;
}

namespace UE::UsdInfoCache::Private
{
#if USE_USD_SDK
	bool RecursiveQueryCanBeCollapsed(
		const pxr::UsdPrim& UsdPrim,
		FUsdSchemaTranslationContext& Context,
		FUsdInfoCacheImpl& Impl,
		FUsdSchemaTranslatorRegistry& Registry
	)
	{
		using namespace UE::UsdInfoCache::Private;

		TRACE_CPUPROFILER_EVENT_SCOPE(UE::USDInfoCache::Private::RecursiveQueryCanBeCollapsed);

		UE::FSdfPath UsdPrimPath = UE::FSdfPath{UsdPrim.GetPrimPath()};

		FReadScopeLock ScopeLock{Impl.InfoMapLock};

		// If we already have a value for our prim then we can just return it right now. We only fill these bCanBeCollapsed values
		// through here, so if we know e.g. that UsdPrim can be collapsed, we know its entire subtree can too.
		FUsdPrimInfo* MainPrimInfo = Impl.GetPrimInfo(UsdPrimPath);
		if (MainPrimInfo)
		{
			FReadScopeLock PrimLock = Impl.LockForReading(*MainPrimInfo);
			if (MainPrimInfo->bXformSubtreeCanBeCollapsed.IsSet())
			{
				return MainPrimInfo->bXformSubtreeCanBeCollapsed.GetValue();
			}
		}

		// If we're here, we don't know whether UsdPrim CanBeCollapsed or not.
		// Since these calls are usually cheap, let's just query it for ourselves right now
		bool bCanBeCollapsed = true;
		if (TSharedPtr<FUsdSchemaTranslator> SchemaTranslator = Registry.CreateTranslatorForSchema(Context.AsShared(), UE::FUsdTyped(UsdPrim)))
		{
			bCanBeCollapsed = SchemaTranslator->CanBeCollapsed(ECollapsingType::Assets);
		}

		// If we can be collapsed ourselves we're not still done, because this is about the subtree. If any of our
		// children can't be collapsed, we actually can't either
		if (bCanBeCollapsed)
		{
			TArray<pxr::UsdPrim> Children;
			for (pxr::UsdPrim Child : UsdPrim.GetFilteredChildren(pxr::UsdTraverseInstanceProxies(pxr::UsdPrimAllPrimsPredicate)))
			{
				// We don't care about non-GeomImagable prims
				// (materials, etc., stuff we don't have schema translators for will be skipped and default to bCanBeCollapsed=true)
				if (Child.IsA<pxr::UsdGeomImageable>())
				{
					Children.Emplace(Child);
				}
			}

			TArray<bool> ChildrenCanBeCollapsed;
			ChildrenCanBeCollapsed.SetNumZeroed(Children.Num());

			const int32 MinBatchSize = 1;
			ParallelFor(
				TEXT("RecursiveQueryCanBeCollapsed"),
				Children.Num(),
				MinBatchSize,
				[&](int32 Index)
				{
					ChildrenCanBeCollapsed[Index] = RecursiveQueryCanBeCollapsed(Children[Index], Context, Impl, Registry);
				}
			);

			for (bool bChildCanBeCollapsed : ChildrenCanBeCollapsed)
			{
				if (!bChildCanBeCollapsed)
				{
					bCanBeCollapsed = false;
					break;
				}
			}
		}

		// Record what we found about our main prim
		if (MainPrimInfo)
		{
			FWriteScopeLock PrimLock = Impl.LockForWriting(*MainPrimInfo);
			MainPrimInfo->bXformSubtreeCanBeCollapsed = bCanBeCollapsed;
		}

		// Before we return though, what we can do here is that if we know that we can't be collapsed ourselves,
		// then none of our ancestors can either! So let's quickly paint upwards to make future queries faster
		if (!bCanBeCollapsed && MainPrimInfo)
		{
			uint64 IterIndex = INDEX_NONE;
			{
				IterIndex = MainPrimInfo->ParentInfoIndex;
			}

			while (IterIndex != INDEX_NONE)
			{
				FUsdPrimInfo& AncestorInfo = Impl.PrimInfoArray[IterIndex];
				FWriteScopeLock PrimLock = Impl.LockForWriting(*MainPrimInfo);

				// We found something that was already filled out: Let's stop traversing here
				if (AncestorInfo.bXformSubtreeCanBeCollapsed.IsSet())
				{
					// If we can't collapse ourselves then like we mentioned above none of our ancestors should
					// be able to collapse either
					ensure(AncestorInfo.bXformSubtreeCanBeCollapsed.GetValue() == false);
					break;
				}
				else
				{
					AncestorInfo.bXformSubtreeCanBeCollapsed = false;
				}

				IterIndex = AncestorInfo.ParentInfoIndex;
			}
		}

		return bCanBeCollapsed;
	}
#endif	  // USE_USD_SDK
}	 // namespace UE::UsdInfoCache::Private

UE::FSdfPath FUsdInfoCache::UnwindToNonCollapsedPath(const UE::FSdfPath& Path, ECollapsingType CollapsingType) const
{
	using namespace UE::UsdInfoCache::Private;

	TRACE_CPUPROFILER_EVENT_SCOPE(FUsdInfoCache::UnwindToNonCollapsedPath);

	if (FUsdInfoCacheImpl* ImplPtr = Impl.Get())
	{
		FReadScopeLock ScopeLock(ImplPtr->InfoMapLock);

		if (uint64* PrimIndex = ImplPtr->InfoMap.Find(Path))
		{
			TFunction<TOptional<UE::FSdfPath>(uint64)> GetCollapseRootFromParent;
			GetCollapseRootFromParent = [ImplPtr, &GetCollapseRootFromParent](uint64 Index) -> TOptional<UE::FSdfPath>
			{
				if (Index == INDEX_NONE)
				{
					return {};
				}

				const FUsdPrimInfo& Info = ImplPtr->PrimInfoArray[Index];

				uint64 ParentIndex = INDEX_NONE;
				{
					FReadScopeLock PrimLock = ImplPtr->LockForReading(Info);
					ParentIndex = Info.ParentInfoIndex;
				}

				TOptional<UE::FSdfPath> CollapseRootPath = GetCollapseRootFromParent(ParentIndex);
				if (CollapseRootPath.IsSet())
				{
					// Our parent says it is collapsed with this collapse root: That's going to be
					// the collapse root for our children too
					return CollapseRootPath;
				}

				FReadScopeLock PrimLock = ImplPtr->LockForReading(Info);
				if (Info.bCollapsesChildren)
				{
					// We are the collapse root, let's return this to our children
					return Info.PrimPath;
				}

				// Nothing collapses so far
				return {};
			};

			TOptional<UE::FSdfPath> CollapseRoot = GetCollapseRootFromParent(*PrimIndex);
			if (CollapseRoot.IsSet())
			{
				return CollapseRoot.GetValue();
			}

			// We're not being collapsed by anything, so we're already the "non collapsed path"
			const FUsdPrimInfo& Info = ImplPtr->PrimInfoArray[*PrimIndex];
			FReadScopeLock PrimLock = ImplPtr->LockForReading(Info);
			return Info.PrimPath;
		}

		// This should never happen: We should have cached the entire tree
		ensureMsgf(false, TEXT("Prim path '%s' has not been cached!"), *Path.GetString());
	}

	return Path;
}

TSet<UE::FSdfPath> FUsdInfoCache::GetMainPrims(const UE::FSdfPath& AuxPrimPath) const
{
	if (FUsdInfoCacheImpl* ImplPtr = Impl.Get())
	{
		FReadScopeLock ScopeLock(ImplPtr->InfoMapLock);

		if (const UE::UsdInfoCache::Private::FUsdPrimInfo* FoundInfo = ImplPtr->GetPrimInfo(AuxPrimPath))
		{
			FReadScopeLock PrimLock = Impl->LockForReading(*FoundInfo);

			TSet<UE::FSdfPath> Result = FoundInfo->MainPrims;
			Result.Add(AuxPrimPath);
			return Result;
		}
	}

	return {AuxPrimPath};
}

TSet<UE::FSdfPath> FUsdInfoCache::GetAuxiliaryPrims(const UE::FSdfPath& MainPrimPath) const
{
	if (FUsdInfoCacheImpl* ImplPtr = Impl.Get())
	{
		FReadScopeLock ScopeLock(ImplPtr->InfoMapLock);

		if (const UE::UsdInfoCache::Private::FUsdPrimInfo* FoundInfo = ImplPtr->GetPrimInfo(MainPrimPath))
		{
			FReadScopeLock PrimLock = Impl->LockForReading(*FoundInfo);

			TSet<UE::FSdfPath> Result = FoundInfo->AuxPrims;
			Result.Add(MainPrimPath);
			return Result;
		}
	}

	return {MainPrimPath};
}

TSet<UE::FSdfPath> FUsdInfoCache::GetMaterialUsers(const UE::FSdfPath& Path) const
{
	if (FUsdInfoCacheImpl* ImplPtr = Impl.Get())
	{
		FReadScopeLock ScopeLock(ImplPtr->InfoMapLock);

		if (const UE::UsdInfoCache::Private::FUsdPrimInfo* FoundInfo = ImplPtr->GetPrimInfo(Path))
		{
			FReadScopeLock PrimLock = Impl->LockForReading(*FoundInfo);
			return FoundInfo->MaterialUserMainPrims;
		}
	}

	return {};
}

bool FUsdInfoCache::IsMaterialUsed(const UE::FSdfPath& Path) const
{
	if (FUsdInfoCacheImpl* ImplPtr = Impl.Get())
	{
		FReadScopeLock ScopeLock(ImplPtr->InfoMapLock);

		if (const UE::UsdInfoCache::Private::FUsdPrimInfo* FoundInfo = ImplPtr->GetPrimInfo(Path))
		{
			FReadScopeLock PrimLock = Impl->LockForReading(*FoundInfo);
			return FoundInfo->MaterialUserMainPrims.Num() > 0;
		}
	}

	return {};
}

namespace UE::USDInfoCache::Private
{
#if USE_USD_SDK
	void GetPrimVertexCountAndSlots(
		const pxr::UsdPrim& UsdPrim,
		const FUsdSchemaTranslationContext& Context,
		const FUsdInfoCacheImpl& Impl,
		uint64& OutVertexCount,
		TArray<UsdUtils::FUsdPrimMaterialSlot>& OutMaterialSlots
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GetPrimVertexCountAndSlots);

		FScopedUsdAllocs Allocs;

		if (UsdPrim.IsA<pxr::UsdGeomGprim>() || UsdPrim.IsA<pxr::UsdGeomSubset>())
		{
			OutVertexCount = UsdUtils::GetGprimVertexCount(pxr::UsdGeomGprim{UsdPrim}, Context.Time);

			pxr::TfToken RenderContextToken = pxr::UsdShadeTokens->universalRenderContext;
			if (!Context.RenderContext.IsNone())
			{
				RenderContextToken = UnrealToUsd::ConvertToken(*Context.RenderContext.ToString()).Get();
			}

			pxr::TfToken MaterialPurposeToken = pxr::UsdShadeTokens->allPurpose;
			if (!Context.MaterialPurpose.IsNone())
			{
				MaterialPurposeToken = UnrealToUsd::ConvertToken(*Context.MaterialPurpose.ToString()).Get();
			}

			const bool bProvideMaterialIndices = false;
			UsdUtils::FUsdPrimMaterialAssignmentInfo LocalInfo = UsdUtils::GetPrimMaterialAssignments(
				UsdPrim,
				Context.Time,
				bProvideMaterialIndices,
				RenderContextToken,
				MaterialPurposeToken
			);

			OutMaterialSlots.Append(MoveTemp(LocalInfo.Slots));
		}
		else if (pxr::UsdGeomPointInstancer PointInstancer{UsdPrim})
		{
			const pxr::UsdRelationship& Prototypes = PointInstancer.GetPrototypesRel();

			pxr::SdfPathVector PrototypePaths;
			if (Prototypes.GetTargets(&PrototypePaths))
			{
				TArray<uint64> PrototypeVertexCounts;
				PrototypeVertexCounts.SetNumZeroed(PrototypePaths.size());

				{
					FReadScopeLock ScopeLock(Impl.InfoMapLock);
					for (uint32 PrototypeIndex = 0; PrototypeIndex < PrototypePaths.size(); ++PrototypeIndex)
					{
						const pxr::SdfPath& PrototypePath = PrototypePaths[PrototypeIndex];

						// Skip invisible prototypes here to mirror how they're skipped within
						// USDGeomMeshConversion.cpp, in the RecursivelyCollapseChildMeshes function. Those two
						// traversals have to match at least with respect to the material slots, so that we can use
						// the data collected here to apply material overrides to the meshes generated for the point
						// instancers when they're collapsed
						pxr::UsdPrim PrototypePrim = UsdPrim.GetStage()->GetPrimAtPath(PrototypePath);
						if (pxr::UsdGeomImageable UsdGeomImageable = pxr::UsdGeomImageable(PrototypePrim))
						{
							if (pxr::UsdAttribute VisibilityAttr = UsdGeomImageable.GetVisibilityAttr())
							{
								pxr::TfToken VisibilityToken;
								if (VisibilityAttr.Get(&VisibilityToken) && VisibilityToken == pxr::UsdGeomTokens->invisible)
								{
									continue;
								}
							}
						}

						// If we're calling this for a point instancer we should have parsed the results for our
						// prototype subtrees already
						if (const UE::UsdInfoCache::Private::FUsdPrimInfo* FoundInfo = Impl.GetPrimInfo(UE::FSdfPath{PrototypePath}))
						{
							FReadScopeLock PrimLock = Impl.LockForReading(*FoundInfo);

							PrototypeVertexCounts[PrototypeIndex] = FoundInfo->ExpectedVertexCountForSubtree.GetValue();
							OutMaterialSlots.Append(FoundInfo->SubtreeMaterialSlots);
						}
					}
				}

				if (pxr::UsdAttribute ProtoIndicesAttr = PointInstancer.GetProtoIndicesAttr())
				{
					pxr::VtArray<int> ProtoIndicesArr;
					if (ProtoIndicesAttr.Get(&ProtoIndicesArr, pxr::UsdTimeCode::EarliestTime()))
					{
						for (int ProtoIndex : ProtoIndicesArr)
						{
							if (PrototypeVertexCounts.IsValidIndex(ProtoIndex))
							{
								OutVertexCount += PrototypeVertexCounts[ProtoIndex];
							}
						}
					}
				}
			}
		}
	}

	void CleanUpInfoMapSubtree(const UE::FSdfPath& PrimPath, FUsdInfoCacheImpl& Impl)
	{
		using namespace UE::UsdInfoCache::Private;

		TRACE_CPUPROFILER_EVENT_SCOPE(CleanUpInfoMapSubtree);

		FWriteScopeLock ScopeLock(Impl.InfoMapLock);

		const uint64* ExistingPrimIndex = Impl.InfoMap.Find(PrimPath);

		uint64 ExistingSubtreeParentIndex = INDEX_NONE;
		if (ExistingPrimIndex)
		{
			FUsdPrimInfo& Info = Impl.PrimInfoArray[*ExistingPrimIndex];
			FReadScopeLock InfoLock = Impl.LockForReading(Info);
			ExistingSubtreeParentIndex = Info.ParentInfoIndex;
		}

		// Invalidate ancestors
		{
			uint64 AncestorIndex = ExistingSubtreeParentIndex;

			// Even if we don't have an existing subtree to remove (e.g. when resyncing and adding a brand new subtree)
			// we still need to find where this new subtree would "attach" and clean up those ancestors too,
			// as we may add new info structs later when repopulating that would invalidate their collected info
			if (!ExistingPrimIndex)
			{
				UE::FSdfPath IterPath = PrimPath.GetParentPath();
				while (!IterPath.IsEmpty())
				{
					if (const uint64* FoundAncestor = Impl.InfoMap.Find(IterPath))
					{
						AncestorIndex = *FoundAncestor;
						break;
					}
					IterPath = IterPath.GetParentPath();
				}
			}

			// Actually invalidate ancestors from the one we found all the way up to the root
			if (AncestorIndex != INDEX_NONE)
			{
				Impl.ResetPrimInfoEntry(AncestorIndex);
			}
		}

		// Remove the prim subtree, if it exists
		if (ExistingPrimIndex)
		{
			// Disconnect our invalidated subtree from its parent info (PrimPath could actually point at a prim
			// that has been fully removed from the stage)
			if (ExistingSubtreeParentIndex != INDEX_NONE)
			{
				FUsdPrimInfo& ParentInfo = Impl.PrimInfoArray[ExistingSubtreeParentIndex];
				FWriteScopeLock ParentLock = Impl.LockForWriting(ParentInfo);
				ParentInfo.ChildIndices.Remove(*ExistingPrimIndex);
			}

			Impl.RemovePrimInfoSubtree(*ExistingPrimIndex);
		}
	}

	void RepopulateInfoMapSubtree(const UE::FSdfPath& SubtreeRootPath, const UE::FUsdStage& Stage, FUsdInfoCacheImpl& Impl)
	{
		using namespace UE::UsdInfoCache::Private;

		TRACE_CPUPROFILER_EVENT_SCOPE(RepopulateInfoMapSubtree);

		if (!Stage)
		{
			return;
		}

		UE::FUsdPrim SubtreeRootPrim = Stage.GetPrimAtPath(SubtreeRootPath);
		if (!SubtreeRootPrim)
		{
			// It's possible to be called with paths to prims that don't exist on the stage, for example when handling the
			// rebuild about removing a prim spec, where USD sends a resync notice for the path to the prim that was just removed.
			// We still want the info cache build to do all the rest in those cases though (cleanup old entries, invalidate ancestors,
			// etc.), so we handle ignoring this case only in here
			return;
		}

		const bool bIsPartialBuild = SubtreeRootPath.IsAbsoluteRootPath();

		TFunction<uint64(const pxr::UsdPrim&, uint64)> ConstructInfoForPrim;
		ConstructInfoForPrim = [&Impl, &ConstructInfoForPrim, bIsPartialBuild](const pxr::UsdPrim& Prim, uint64 ParentIndex) -> uint64
		{
			// If we're affecting an instance, record that we need to visit its prototype later
			if (bIsPartialBuild && Prim.IsInstance())
			{
				UE::FUsdPrim Prototype{Prim.GetPrototype()};
				FWriteScopeLock ScopeLock(Impl.TempUpdatedPrototypesLock);
				Impl.TempUpdatedPrototypes.Add(Prototype.GetPrimPath());
			}

			// Note: We're not locking the infos here at all as our access pattern will never touch the same info more than once anyway,
			// and this function is single threaded and never calls in to any other thread-unsafe functions
			uint64 NewIndex = INDEX_NONE;
			FUsdPrimInfo& NewInfo = Impl.CreateNewInfo(UE::FSdfPath{Prim.GetPrimPath()}, NewIndex);
			NewInfo.PrimLockIndex = static_cast<int32>(NewIndex % GNumPerPrimLocks);
			NewInfo.ParentInfoIndex = ParentIndex;

			Impl.InfoMap.Add(NewInfo.PrimPath, NewIndex);

			// Note: I've tried a ParallelFor here, and it was slower than the single threaded version due to write lock
			// contention on the InfoMap itself
			TArray<uint64> ChildIndices;
			pxr::UsdPrimSiblingRange Children = Prim.GetFilteredChildren(pxr::UsdTraverseInstanceProxies(pxr::UsdPrimAllPrimsPredicate));
			for (const pxr::UsdPrim& Child : Children)
			{
				const uint64 ChildIndex = ConstructInfoForPrim(Child, NewIndex);
				ChildIndices.Add(ChildIndex);
			}
			// Have to find our NewInfo again as the recursive calls likely invalidated our NewInfo reference
			Impl.PrimInfoArray[NewIndex].ChildIndices = MoveTemp(ChildIndices);

			return NewIndex;
		};

		FWriteScopeLock ScopeLock(Impl.InfoMapLock);

		// Find parent
		UE::FSdfPath ParentPrimPath = SubtreeRootPath.GetParentPath();
		uint64* ParentIndexPtr = Impl.InfoMap.Find(ParentPrimPath);
		uint64 ParentIndex = ParentIndexPtr ? *ParentIndexPtr : INDEX_NONE;

		// Create new subtree
		uint64 SubtreeRootIndex = ConstructInfoForPrim(SubtreeRootPrim, ParentIndex);

		// Connect the new subtree to its target parent
		if (ParentIndex != INDEX_NONE)
		{
			Impl.PrimInfoArray[ParentIndex].ChildIndices.Add(SubtreeRootIndex);
		}
	}

	void RecursivePropagateVertexAndMaterialSlotCounts(
		uint64 PrimIndex,
		FUsdSchemaTranslationContext& Context,
		const pxr::TfToken& MaterialPurposeToken,
		FUsdInfoCacheImpl& Impl,
		FUsdSchemaTranslatorRegistry& Registry,
		uint64& OutSubtreeVertexCount,
		TArray<UsdUtils::FUsdPrimMaterialSlot>& OutSubtreeSlots,
		bool bPossibleInheritedBindings
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RecursivePropagateVertexAndMaterialSlotCounts);

		FScopedUsdAllocs Allocs;

		FReadScopeLock ScopeLock{Impl.InfoMapLock};
		UE::UsdInfoCache::Private::FUsdPrimInfo& Info = Impl.PrimInfoArray[PrimIndex];

		// Don't bother locking for reading PrimPath: We only ever write to this before this stage of the build
		pxr::UsdPrim UsdPrim = Context.Stage.GetPrimAtPath(Info.PrimPath);
		if (!UsdPrim)
		{
			return;
		}

		// We already visited this subtree.
		// Note that we don't need to check for MaterialUsers, material slots, or worry about point instancers here:
		// Since we fill all of those for every rebuild, if ExpectedVertexCountForSubtree is filled so is everything else
		{
			FReadScopeLock PrimLock = Impl.LockForReading(Info);
			if (Info.ExpectedVertexCountForSubtree.IsSet())
			{
				OutSubtreeVertexCount = Info.ExpectedVertexCountForSubtree.GetValue();
				OutSubtreeSlots = Info.SubtreeMaterialSlots;
				return;
			}
		}

		pxr::UsdStageRefPtr Stage = UsdPrim.GetStage();
		TFunction<void(const UE::FSdfPath&, TSet<UE::FSdfPath>&)> TryAddMaterialUser =
			[&Stage](const UE::FSdfPath& User, TSet<UE::FSdfPath>& Material)
		{
			pxr::UsdPrim UserPrim = Stage->GetPrimAtPath(User);

			if (UserPrim.IsA<pxr::UsdGeomImageable>())
			{
				// Do this filtering here because Collection.ComputeIncludedPaths() can be very aggressive and return
				// literally *all prims* below an included prim path. That's fine and it really does mean that any Mesh prim
				// in there could use the collection-based material binding, but nevertheless we don't want to register that
				// e.g. Shader prims or SkelAnimation prims are "material users"
				Material.Add(User);
			}
			else if (UserPrim.IsA<pxr::UsdGeomSubset>())
			{
				// If a UsdGeomSubset is a material user, make its Mesh parent prim into a user too.
				// Our notice handling is somewhat stricter now, and we have no good way of upgrading a simple material info change
				// into a resync change of the StaticMeshComponent when we change a material that is bound directly to a
				// UsdGeomSubset, since the GeomMesh translator doesn't collapse. We'll unwind this path later when fetching material
				// users, so collapsed static meshes are handled OK, skeletal meshes are handled OK, we just need this one exception
				// for handling uncollapsed static meshes, because by default Mesh prims don't "collapse" their child UsdGeomSubsets
				Material.Add(User.GetParentPath());
			}
		};

		// Material bindings are inherited down to child prims, so if we detect a binding on a parent Xform,
		// we should register the child Mesh prims as users of the material too (regardless of collapsing).
		// Note that we only consider this for direct bindings: Collection-based bindings will already provide the exhaustive
		// list of all the prims that they should apply to when we call ComputeIncludedPaths
		bool bPrimHasInheritableMaterialBindings = false;

		// Try restoring old material users for this prim if it's a Material
		if (UsdPrim.IsA<pxr::UsdShadeMaterial>())
		{
			Impl.TryRestoreMaterialUserLinks(Info);
		}

		// Register material users for other Material prims if this prim is a user
		if (!UsdPrim.IsPseudoRoot())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(CheckingMaterialUsers);

			TMap<UE::FSdfPath, TSet<UE::FSdfPath>> NewMaterialUsers;

			pxr::UsdShadeMaterialBindingAPI BindingAPI{UsdPrim};

			// Check for material users via collections-based material bindings
			if (BindingAPI || bPossibleInheritedBindings)
			{
				// When retrieving the relationships directly we'll always need to check the universal render context
				// manually, as it won't automatically "compute the fallback" for us like when we ComputeBoundMaterial()
				std::unordered_set<pxr::TfToken, pxr::TfHash> MaterialPurposeTokens{
					MaterialPurposeToken,
					pxr::UsdShadeTokens->universalRenderContext
				};
				for (const pxr::TfToken& SomeMaterialPurposeToken : MaterialPurposeTokens)
				{
					// Each one of those relationships must have two targets: A collection, and a material
					for (const pxr::UsdRelationship& Rel : BindingAPI.GetCollectionBindingRels(SomeMaterialPurposeToken))
					{
						const pxr::SdfPath* CollectionPath = nullptr;
						const pxr::SdfPath* MaterialPath = nullptr;

						std::vector<pxr::SdfPath> PathVector;
						if (Rel.GetTargets(&PathVector))
						{
							for (const pxr::SdfPath& Path : PathVector)
							{
								if (Path.IsPrimPath())
								{
									MaterialPath = &Path;
								}
								else if (Path.IsPropertyPath())
								{
									CollectionPath = &Path;
								}
							}
						}

						if (!CollectionPath || !MaterialPath || PathVector.size() != 2)
						{
							// Emit this warning here as USD doesn't seem to and just seems to just ignores this relationship instead
							USD_LOG_USERWARNING(FText::Format(
								LOCTEXT(
									"InvalidCollection",
									"Prim '{0}' describes a collection-based material binding, but the relationship '{1}' is invalid: It should "
									"contain exactly one Material path and one path to a collection relationship"
								),
								FText::FromString(Info.PrimPath.GetString()),
								FText::FromString(UsdToUnreal::ConvertToken(Rel.GetName()))
							));
							continue;
						}

						if (pxr::UsdCollectionAPI Collection = pxr::UsdCollectionAPI::Get(Stage, *CollectionPath))
						{
							TSet<UE::FSdfPath>& MaterialUsers = NewMaterialUsers.FindOrAdd(UE::FSdfPath{*MaterialPath});

							std::set<pxr::SdfPath> IncludedPaths = Collection.ComputeIncludedPaths(Collection.ComputeMembershipQuery(), Stage);
							for (const pxr::SdfPath& IncludedPath : IncludedPaths)
							{
								TryAddMaterialUser(UE::FSdfPath{IncludedPath}, MaterialUsers);
							}
						}
						else
						{
							USD_LOG_USERWARNING(FText::Format(
								LOCTEXT(
									"MissingCollection",
									"Failed to find collection at path '{0}' when processing collection-based material bindings on prim '{1}'"
								),
								FText::FromString(UsdToUnreal::ConvertPath(CollectionPath->GetPrimPath())),
								FText::FromString(Info.PrimPath.GetString())
							));
						}
					}
				}
			}

			// Check for material bindings directly for this prim.
			//
			// Note how we don't actually check the BindingAPI here: This is intentional as it will cover the case where the prim
			// doesn't have the MaterialBindingAPI applied but still has the material relationships, which is also how
			// UsdUtils::GetPrimMaterialAssignments also operates. The MaterialBindingAPI schema code will still work even if the
			// schema is not actually applied to the prim.
			//
			// We used to check for a relationship directly here using the "material:binding" token, but that is inferior to using ComputeBoundMaterial
			// as it will not use the MaterialPurposeToken: If the prim only had "material:binding:preview", we would fail to find a relationship when
			// just querying with "material:binding"
			//
			// USD will emit a warning for the missing MaterialBindingAPI though.
			if (pxr::UsdShadeMaterial ShadeMaterial = BindingAPI.ComputeBoundMaterial(MaterialPurposeToken))
			{
				bPrimHasInheritableMaterialBindings = true;

				TSet<UE::FSdfPath>& MaterialUsers = NewMaterialUsers.FindOrAdd(UE::FSdfPath{ShadeMaterial.GetPrim().GetPath()});
				TryAddMaterialUser(Info.PrimPath, MaterialUsers);
			}

			for (const TPair<UE::FSdfPath, TSet<UE::FSdfPath>>& NewMaterialToUsers : NewMaterialUsers)
			{
				Impl.RegisterMaterialUserPrims(NewMaterialToUsers.Key, NewMaterialToUsers.Value);
			}
		}

		const uint32 NumChildren = Info.ChildIndices.Num();

		TArray<uint64> ChildSubtreeVertexCounts;
		ChildSubtreeVertexCounts.SetNumZeroed(NumChildren);	   // Zero instead of Uninitialized here because if we run into e.g a Material prim our
															   // code will mostly early out, and we don't want to return an uninitialized int

		TArray<TArray<UsdUtils::FUsdPrimMaterialSlot>> ChildSubtreeMaterialSlots;
		ChildSubtreeMaterialSlots.SetNum(NumChildren);

		const int32 MinBatchSize = 1;
		ParallelFor(
			TEXT("RecursivePropagateVertexAndMaterialSlotCounts"),
			Info.ChildIndices.Num(),
			MinBatchSize,
			[&](int32 Index)
			{
				RecursivePropagateVertexAndMaterialSlotCounts(
					Info.ChildIndices[Index],
					Context,
					MaterialPurposeToken,
					Impl,
					Registry,
					ChildSubtreeVertexCounts[Index],
					ChildSubtreeMaterialSlots[Index],
					bPrimHasInheritableMaterialBindings || bPossibleInheritedBindings
				);
			}
		);

		OutSubtreeVertexCount = 0;
		OutSubtreeSlots.Empty();

		// We will still step into invisible prims to collect all info we can, but we won't count their material slots
		// or vertex counts: The main usage of those counts is to handle collapsed meshes, and during collapse we just
		// early out whenever we encounter an invisible prim
		bool bIsPointInstancer = false;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(GettingVertexCountAndSlots);

			bool bPrimIsInvisible = false;
			if (pxr::UsdGeomImageable UsdGeomImageable = pxr::UsdGeomImageable(UsdPrim))
			{
				if (pxr::UsdAttribute VisibilityAttr = UsdGeomImageable.GetVisibilityAttr())
				{
					pxr::TfToken VisibilityToken;
					if (VisibilityAttr.Get(&VisibilityToken) && VisibilityToken == pxr::UsdGeomTokens->invisible)
					{
						bPrimIsInvisible = true;
					}
				}
			}

			// If the mesh prim has an unselected geometry purpose, it is also essentially invisible
			if (!EnumHasAllFlags(Context.PurposesToLoad, IUsdPrim::GetPurpose(UsdPrim)))
			{
				bPrimIsInvisible = true;
			}

			if (pxr::UsdGeomPointInstancer PointInstancer{UsdPrim})
			{
				bIsPointInstancer = true;
			}
			else if (!bPrimIsInvisible)
			{
				GetPrimVertexCountAndSlots(UsdPrim, Context, Impl, OutSubtreeVertexCount, OutSubtreeSlots);

				for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
				{
					OutSubtreeVertexCount += ChildSubtreeVertexCounts[ChildIndex];
					OutSubtreeSlots.Append(ChildSubtreeMaterialSlots[ChildIndex]);
				}
			}
		}

		{
			// For point instancers we can't guarantee we parsed the prototypes yet because they
			// could technically be anywhere, so store them here for a later pass
			if (bIsPointInstancer)
			{
				FWriteScopeLock PointInstancerLock(Impl.TempPointInstancerPathsLock);
				Impl.TempPointInstancerPaths.Emplace(Info.PrimPath.GetString());
			}
			// While we will compute the totals for any and all children normally, don't just append the regular
			// traversal vertex count to the point instancer prim itself just yet, as that doesn't really represent
			// what will happen. We'll later do another pass to handle point instancers where we'll properly instance
			// stuff, and then we'll updadte all ancestors
			else
			{
				FWriteScopeLock PrimLock = Impl.LockForWriting(Info);
				Info.ExpectedVertexCountForSubtree = OutSubtreeVertexCount;
				Info.SubtreeMaterialSlots.Append(OutSubtreeSlots);
			}
		}
	}

	/**
	 * Updates the subtree counts with point instancer instancing info.
	 *
	 * This has to be done outside of the main recursion because point instancers may reference any prim in the
	 * stage to be their prototypes (including other point instancers), so we must first parse the entire
	 * stage (forcing point instancer vertex/material slot counts to zero), and only then use the parsed counts
	 * of prim subtrees all over to build the final counts of point instancers that use them as prototypes, and
	 * then update their parents.
	 */
	void UpdateInfoForPointInstancers(const FUsdSchemaTranslationContext& Context, FUsdInfoCacheImpl& Impl)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UpdateInfoForPointInstancers);

		pxr::UsdStageRefPtr Stage = pxr::UsdStageRefPtr{Context.Stage};
		if (!Stage)
		{
			return;
		}

		// We must sort point instancers in a particular order in case they depend on each other.
		// At least we know that an ordering like this should be possible, because A with B as a prototype and B with A
		// as a prototype leads to an invalid USD stage.
		TFunction<bool(const FString&, const FString&)> SortFunction = [Stage](const FString& LHS, const FString& RHS)
		{
			FScopedUsdAllocs Allocs;

			pxr::SdfPath LPath = UnrealToUsd::ConvertPath(*LHS).Get();
			pxr::SdfPath RPath = UnrealToUsd::ConvertPath(*RHS).Get();

			pxr::UsdGeomPointInstancer LPointInstancer = pxr::UsdGeomPointInstancer{Stage->GetPrimAtPath(LPath)};
			pxr::UsdGeomPointInstancer RPointInstancer = pxr::UsdGeomPointInstancer{Stage->GetPrimAtPath(RPath)};
			if (LPointInstancer && RPointInstancer)
			{
				const pxr::UsdRelationship& LPrototypes = LPointInstancer.GetPrototypesRel();
				pxr::SdfPathVector LPrototypePaths;
				if (LPrototypes.GetTargets(&LPrototypePaths))
				{
					for (const pxr::SdfPath& LPrototypePath : LPrototypePaths)
					{
						// Consider RPointInstancer at RPath "/LPointInstancer/Prototypes/Nest/RPointInstancer", and
						// LPointInstancer has prototype "/LPointInstancer/Prototypes/Nest". If RPath has the LPrototypePath as prefix,
						// we should have R come before L in the sort order.
						// Of course, in this scenario we could get away with just sorting by length, but that wouldn't help if the
						// point instancers were not inside each other (e.g. siblings).
						if (RPath.HasPrefix(LPrototypePath))
						{
							return false;
						}
					}

					// Give it the benefit of the doubt here and say that if R doesn't *need* to come before L, let's ensure L
					// goes before R just in case
					return true;
				}
			}

			return LHS < RHS;
		};
		{
			FWriteScopeLock PointInstancerLock{Impl.TempPointInstancerPathsLock};
			Impl.TempPointInstancerPaths.Sort(SortFunction);
		}

		FReadScopeLock PointInstancerLock{Impl.TempPointInstancerPathsLock};
		for (const FString& PointInstancerPath : Impl.TempPointInstancerPaths)
		{
			UE::FSdfPath UsdPointInstancerPath{*PointInstancerPath};

			if (pxr::UsdPrim PointInstancer = Stage->GetPrimAtPath(UnrealToUsd::ConvertPath(*PointInstancerPath).Get()))
			{
				uint64 PointInstancerVertexCount = 0;
				TArray<UsdUtils::FUsdPrimMaterialSlot> PointInstancerMaterialSlots;

				GetPrimVertexCountAndSlots(PointInstancer, Context, Impl, PointInstancerVertexCount, PointInstancerMaterialSlots);

				FReadScopeLock Lock{Impl.InfoMapLock};
				if (UE::UsdInfoCache::Private::FUsdPrimInfo* Info = Impl.GetPrimInfo(UsdPointInstancerPath))
				{
					{
						FWriteScopeLock PrimLock = Impl.LockForWriting(*Info);
						Info->ExpectedVertexCountForSubtree = PointInstancerVertexCount;
						Info->SubtreeMaterialSlots.Append(PointInstancerMaterialSlots);
					}

					// Now that we have info on the point instancer itself, update the counts of all ancestors.
					// Note: The vertex/material slot count for the entire point instancer subtree are just the counts
					// for the point instancer itself, as we stop regular traversal when we hit them
					UE::FSdfPath ParentPath = UsdPointInstancerPath.GetParentPath();
					pxr::UsdPrim Prim = Stage->GetPrimAtPath(ParentPath);
					while (Prim)
					{
						// If our ancestor is a point instancer itself, just abort as we'll only get the actual counts
						// when we handle that ancestor directly. We don't want to update the ancestor point instancer's
						// ancestors with incorrect values
						if (Prim.IsA<pxr::UsdGeomPointInstancer>())
						{
							break;
						}

						if (UE::UsdInfoCache::Private::FUsdPrimInfo* ParentInfo = Impl.GetPrimInfo(ParentPath))
						{
							FWriteScopeLock PrimLock = Impl.LockForWriting(*ParentInfo);
							ParentInfo->ExpectedVertexCountForSubtree.GetValue() += PointInstancerVertexCount;
							ParentInfo->SubtreeMaterialSlots.Append(PointInstancerMaterialSlots);
						}

						// Break only here so we update the pseudoroot too
						if (Prim.IsPseudoRoot())
						{
							break;
						}

						ParentPath = ParentPath.GetParentPath();
						Prim = Stage->GetPrimAtPath(ParentPath);
					}
				}
			}
		}
	}

	/**
	 * Removes duplicate material slots for the subtree below RecursiveCollectMaterialSlotCounts, in case
	 * we're allowed to merge material slots.
	 *
	 * We do this after the main pass because then the main material slot collecting code on
	 * the main recursive pass just adds them to arrays, and we're allowed to handle bMergeIdenticalSlots
	 * only here.
	 */
	void RecursiveCollectMaterialSlotCounts(
		uint64 PrimIndex,
		FUsdInfoCacheImpl& Impl,
		const FUsdSchemaTranslationContext& Context,
		bool bPrimIsCollapsedOrCollapseRoot
	)
	{
#if USE_USD_SDK
		using namespace UE::UsdInfoCache::Private;

		TRACE_CPUPROFILER_EVENT_SCOPE(RecursiveCollectMaterialSlotCounts);

		if (!Context.bMergeIdenticalMaterialSlots || PrimIndex == INDEX_NONE)
		{
			return;
		}

		FUsdPrimInfo& Info = Impl.PrimInfoArray[PrimIndex];
		{
			FReadScopeLock PrimLock = Impl.LockForReading(Info);
			if (Info.bSlotsWereMerged)
			{
				// This info has already been processed (we do partial info cache builds now)
				return;
			}
		}

		// For now we only ever merge material slots when collapsing and if parsing LODs (and not if we're collapsing due to being a geometry cache)
		bPrimIsCollapsedOrCollapseRoot |= Info.bCollapsesChildren;
		bool bCanMergeSlotsForThisPrim = bPrimIsCollapsedOrCollapseRoot || Info.PrimPath.IsAbsoluteRootPath()
										 || (Context.bAllowInterpretingLODs
											 && UsdUtils::DoesPrimContainMeshLODs(Context.Stage.GetPrimAtPath(Info.PrimPath)));
		if (bCanMergeSlotsForThisPrim)
		{
			if (Impl.IsPotentialGeometryCacheRootInner(Info, Context.Stage.GetPrimAtPath(Info.PrimPath)))
			{
				bCanMergeSlotsForThisPrim = false;
			}
		}

		// Actually update the slot count
		if (bCanMergeSlotsForThisPrim)
		{
			FWriteScopeLock WritePrimLock = Impl.LockForWriting(Info);
			Info.SubtreeMaterialSlots = TSet<UsdUtils::FUsdPrimMaterialSlot>{Info.SubtreeMaterialSlots}.Array();
			Info.bSlotsWereMerged = true;
		}

		const int32 MinBatchSize = 1;
		ParallelFor(
			TEXT("RecursiveCollectMaterialSlotCounts"),
			Info.ChildIndices.Num(),
			MinBatchSize,
			[&](int32 Index)
			{
				RecursiveCollectMaterialSlotCounts(Info.ChildIndices[Index], Impl, Context, bPrimIsCollapsedOrCollapseRoot);
			}
		);
#endif	  // USE_USD_SDK
	}

	bool CanMeshSubtreeBeCollapsed(
		const pxr::UsdPrim& UsdPrim,
		FUsdSchemaTranslationContext& Context,
		FUsdInfoCacheImpl& Impl,
		const TSharedPtr<FUsdSchemaTranslator>& Translator
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CanMeshSubtreeBeCollapsed);

		if (!UsdPrim)
		{
			return false;
		}

		// We should never be able to collapse SkelRoots because the UsdSkelSkeletonTranslator doesn't collapse
		if (UsdPrim.IsA<pxr::UsdSkelRoot>())
		{
			return false;
		}

		pxr::SdfPath UsdPrimPath = UsdPrim.GetPrimPath();

		FReadScopeLock ScopeLock(Impl.InfoMapLock);
		if (UE::UsdInfoCache::Private::FUsdPrimInfo* Info = Impl.GetPrimInfo(UE::FSdfPath{UsdPrimPath}))
		{
			FReadScopeLock PrimLock = Impl.LockForReading(*Info);

			if (Info->ExpectedVertexCountForSubtree.GetValue() > GMaxNumVerticesCollapsedMesh)
			{
				return false;
			}
		}

		return true;
	}

	void RecursiveQueryCollapsesChildren(
		uint64 PrimIndex,
		FUsdSchemaTranslationContext& Context,
		FUsdInfoCacheImpl& Impl,
		FUsdSchemaTranslatorRegistry& Registry
	)
	{
#if USE_USD_SDK
		using namespace UE::UsdInfoCache::Private;

		TRACE_CPUPROFILER_EVENT_SCOPE(UE::USDInfoCache::Private::RecursiveQueryCollapsesChildren);

		FReadScopeLock ScopeLock{Impl.InfoMapLock};
		FUsdPrimInfo& Info = Impl.PrimInfoArray[PrimIndex];
		{
			FReadScopeLock PrimLock = Impl.LockForReading(Info);
			if (Info.bCollapsesChildren)
			{
				// We invalidate this when the prim is updated, so if we're here it means we know this prim
				// collapses children, and that it hasn't been updated, so we can return
				return;
			}
		}

		FScopedUsdAllocs Allocs;

		pxr::UsdPrim UsdPrim = Context.Stage.GetPrimAtPath(Info.PrimPath);
		if (!UsdPrim)
		{
			return;
		}

		bool bCollapsesChildren = false;

		TSharedPtr<FUsdSchemaTranslator> SchemaTranslator = Registry.CreateTranslatorForSchema(Context.AsShared(), UE::FUsdTyped(UsdPrim));
		if (SchemaTranslator)
		{
			const bool bIsPotentialGeometryCacheRoot = Impl.IsPotentialGeometryCacheRootInner(Info, UsdPrim);

			// The potential geometry cache root is checked first since the FUsdGeometryCacheTranslator::CollapsesChildren
			// has no logic of its own
			if (bIsPotentialGeometryCacheRoot
				|| (SchemaTranslator->CollapsesChildren(ECollapsingType::Assets)
					&& CanMeshSubtreeBeCollapsed(UsdPrim, Context, Impl, SchemaTranslator)))
			{
				bCollapsesChildren = true;
			}
		}

		if (bCollapsesChildren)
		{
			FWriteScopeLock PrimLock = Impl.LockForWriting(Info);
			Info.bCollapsesChildren = true;
		}
		// We only need to visit our children if we don't collapse. We'll leave the AssetCollapsedRoot fields unset on
		// the InfoMap, and whenever we query info about a particular prim will fill that in on-demand by just traveling
		// upwards until we run into our collapse root
		else
		{
			const int32 MinBatchSize = 1;
			ParallelFor(
				TEXT("RecursiveQueryCollapsesChildren"),
				Info.ChildIndices.Num(),
				MinBatchSize,
				[&](int32 Index)
				{
					RecursiveQueryCollapsesChildren(Info.ChildIndices[Index], Context, Impl, Registry);
				}
			);
		}

		// We only do this for uncollapsed prims or collapse roots (since RecursiveQueryCollapsesChildren never steps into a
		// collapsed prim). This because whenever the collapse root registers its auxiliary prims here, it will already account
		// for all of the relevant child prims in the entire subtree, according to the translator type. The links between prims
		// inside of a collapsed subtree aren't really useful, because if anything inside the collapsed subtree updates, we'll
		// always just need to update from the collapsed root anyway
		if (SchemaTranslator)
		{
			Impl.RegisterAuxiliaryPrims(Info.PrimPath, SchemaTranslator->CollectAuxiliaryPrims());
		}
#endif	  // USE_USD_SDK
	}

	// Returns the paths to all prims on the same local layer stack, that are used as sources for composition
	// arcs that are non-root (i.e. the arcs that are either reference, payload, inherits, etc.).
	// In other words, "instanceable composition arcs from local prims"
	TSet<UE::FSdfPath> GetLocalNonRootCompositionArcSourcePaths(const pxr::UsdPrim& UsdPrim)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GetLocalNonRootCompositionArcSourcePaths);

		TSet<UE::FSdfPath> Result;

		if (!UsdPrim)
		{
			return Result;
		}

		pxr::PcpLayerStackRefPtr RootLayerStack;

		pxr::UsdPrimCompositionQuery PrimCompositionQuery = pxr::UsdPrimCompositionQuery(UsdPrim);
		std::vector<pxr::UsdPrimCompositionQueryArc> Arcs = PrimCompositionQuery.GetCompositionArcs();
		Result.Reserve(Arcs.size());
		for (const pxr::UsdPrimCompositionQueryArc& Arc : Arcs)
		{
			pxr::PcpNodeRef TargetNode = Arc.GetTargetNode();

			if (Arc.GetArcType() == pxr::PcpArcTypeRoot)
			{
				RootLayerStack = TargetNode.GetLayerStack();
			}
			// We use this function to collect aux/main prim links for instanceables, and we don't have
			// to track instanceable arcs to outside the local layer stack because those don't generate
			// source prims on the stage that the user could edit anyway!
			else if (TargetNode.GetLayerStack() == RootLayerStack)
			{
				Result.Add(UE::FSdfPath{Arc.GetTargetPrimPath()});
			}
		}

		return Result;
	}

	void RegisterInstanceableAuxPrims(FUsdSchemaTranslationContext& Context, bool bPartialRebuild, FUsdInfoCacheImpl& Impl)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::USDInfoCache::Private::RegisterInstanceableAuxPrims);
		FScopedUsdAllocs Allocs;

		pxr::UsdStageRefPtr Stage = pxr::UsdStageRefPtr{Context.Stage};
		if (!Stage)
		{
			return;
		}

		FReadScopeLock ScopeLock{Impl.InfoMapLock};

		std::vector<pxr::UsdPrim> Prototypes = Stage->GetPrototypes();
		const int32 MinBatchSize = 1;
		ParallelFor(
			TEXT("RegisterInstanceableAuxPrimsPrototypes"),
			Prototypes.size(),
			MinBatchSize,
			[&Prototypes, &Context, &Impl, &Stage, bPartialRebuild](int32 Index)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(RegisterInstanceableAuxPrims::Prototype);

				FScopedUsdAllocs Allocs;

				const pxr::UsdPrim& Prototype = Prototypes[Index];
				if (!Prototype)
				{
					return;
				}

				// If we're on a partial rebuild and none of our instances got updated, then we don't
				// have anything to do (prototypes can't be resynced by themselves)
				if (bPartialRebuild)
				{
					UE::FSdfPath PrototypePath{Prototype.GetPrimPath()};

					FWriteScopeLock ScopeLock(Impl.TempUpdatedPrototypesLock);
					if (!Impl.TempUpdatedPrototypes.Contains(PrototypePath))
					{
						return;
					}
				}

				std::vector<pxr::UsdPrim> Instances = Prototype.GetInstances();
				if (Instances.size() < 1)
				{
					return;
				}

				TArray<UE::FSdfPath> InstancePaths;
				InstancePaths.SetNum(Instances.size());

				// Really what we want is to find the source prim that generated this prototype though. Instances always work
				// through some kind of composition arc, so here we collect all references/payloads/inherits/specializes/etc.
				// There's a single source prim shared across all instances, so just fetch it from the first one
				TSet<UE::FSdfPath> SourcePaths = GetLocalNonRootCompositionArcSourcePaths(Instances[0]);
				if (SourcePaths.Num() == 0)
				{
					return;
				}

				// Step into every instance of this prototype on the stage
				ParallelFor(
					TEXT("RegisterInstanceableAuxPrimsInstances"),
					Instances.size(),
					MinBatchSize,
					[&Instances, &InstancePaths, &Context, &Impl, &Stage, &SourcePaths](int32 Index)
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(RegisterInstanceableAuxPrims::PrototypeInstance);

						FScopedUsdAllocs Allocs;

						const pxr::UsdPrim& Instance = Instances[Index];

						UE::FSdfPath InstancePath{Instance.GetPrimPath()};
						InstancePaths[Index] = InstancePath;

						if (UE::UsdInfoCache::Private::FUsdPrimInfo* MainPrim = Impl.GetPrimInfo(InstancePath))
						{
							FWriteScopeLock PrimLock = Impl.LockForWriting(*MainPrim);

							if (MainPrim->bInstanceAuxPrimsRegistered)
							{
								// We already processed this particular instance on a previous info cache build
								return;
							}
							MainPrim->bInstanceAuxPrimsRegistered = true;

							MainPrim->AuxPrims.Append(SourcePaths);
						}

						// Here we'll traverse the entire subtree of the instance
						pxr::UsdPrimRange PrimRange(Instance, pxr::UsdTraverseInstanceProxies());
						for (pxr::UsdPrimRange::iterator InstanceChildIt = ++PrimRange.begin(); InstanceChildIt != PrimRange.end(); ++InstanceChildIt)
						{
							TRACE_CPUPROFILER_EVENT_SCOPE(RegisterInstanceableAuxPrims::InstanceChild);

							pxr::SdfPath SdfChildPrimPath = InstanceChildIt->GetPrimPath();
							UE::FSdfPath ChildPrimPath{SdfChildPrimPath};

							// Register a dependency from child prim to analogue prims on the sources used for the instance.
							// We have to do some path surgery to discover what the analogue paths on the source prims are though
							pxr::SdfPath RelativeChildPath = SdfChildPrimPath.MakeRelativePath(InstancePath);
							for (const UE::FSdfPath& SourcePath : SourcePaths)
							{
								pxr::SdfPath ChildOnSourcePath = pxr::SdfPath{SourcePath}.AppendPath(RelativeChildPath);
								if (pxr::UsdPrim ChildOnSource = Stage->GetPrimAtPath(ChildOnSourcePath))
								{
									Impl.RegisterAuxiliaryPrims(ChildPrimPath, {UE::FSdfPath{ChildOnSourcePath}});
								}
							}
						}
					}
				);

				// Append all the instance paths in one go for these source paths
				for (const UE::FSdfPath& AuxPrimPath : SourcePaths)
				{
					if (UE::UsdInfoCache::Private::FUsdPrimInfo* AuxPrim = Impl.GetPrimInfo(AuxPrimPath))
					{
						FWriteScopeLock PrimLock = Impl.LockForWriting(*AuxPrim);
						AuxPrim->MainPrims.Append(InstancePaths);
					}
				}
			}
		);
	}

	void FindValidGeometryCacheRoot(
		const pxr::UsdPrim& UsdPrim,
		FUsdSchemaTranslationContext& Context,
		FUsdInfoCacheImpl& Impl,
		UE::UsdInfoCache::Private::EGeometryCachePrimState& OutState
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FindValidGeometryCacheRoot);

		using namespace UE::UsdInfoCache::Private;

		FScopedUsdAllocs Allocs;

		pxr::SdfPath UsdPrimPath = UsdPrim.GetPrimPath();
		UE::FSdfPath PrimPath{UsdPrimPath};
		{
			FReadScopeLock ScopeLock(Impl.InfoMapLock);
			if (UE::UsdInfoCache::Private::FUsdPrimInfo* Info = Impl.GetPrimInfo(UE::FSdfPath(UsdPrim.GetPrimPath())))
			{
				FWriteScopeLock PrimLock = Impl.LockForWriting(*Info);

				// A prim is considered a valid root if its subtree has no uncollapsible branch and a valid depth.
				// A valid depth is positive, meaning it has an animated mesh, and doesn't exceed the limit.
				bool bIsValidDepth = Info->GeometryCacheDepth > -1 && Info->GeometryCacheDepth <= GGeometryCacheMaxDepth;
				if (!EnumHasAnyFlags(Info->GeometryCacheState, EGeometryCachePrimState::Uncollapsible) && bIsValidDepth)
				{
					OutState = EGeometryCachePrimState::ValidRoot;
					Info->GeometryCacheState = EGeometryCachePrimState::ValidRoot;
					return;
				}
				// The prim is not a valid root so it's flagged as uncollapsible since the root will be among its children
				// and the eventual geometry cache cannot be collapsed.
				else
				{
					OutState = EGeometryCachePrimState::Uncollapsible;
					Info->GeometryCacheState = EGeometryCachePrimState::Uncollapsible;
				}
			}
		}

		pxr::UsdPrimSiblingRange PrimChildren = UsdPrim.GetFilteredChildren(pxr::UsdTraverseInstanceProxies(pxr::UsdPrimAllPrimsPredicate));

		// Continue the search for a valid root among the children
		for (pxr::UsdPrim Child : PrimChildren)
		{
			bool bIsCollapsible = false;
			{
				FReadScopeLock ScopeLock(Impl.InfoMapLock);
				if (const UE::UsdInfoCache::Private::FUsdPrimInfo* Info = Impl.GetPrimInfo(UE::FSdfPath(Child.GetPrimPath())))
				{
					FReadScopeLock Lock = Impl.LockForReading(*Info);

					bIsCollapsible = EnumHasAnyFlags(Info->GeometryCacheState, EGeometryCachePrimState::Collapsible);
				}
			}

			// A subtree is considered only if it has anything collapsible in the first place
			if (bIsCollapsible)
			{
				FindValidGeometryCacheRoot(Child, Context, Impl, OutState);
			}
		}

		OutState = EGeometryCachePrimState::Uncollapsible;
	}

	void RecursiveCheckForGeometryCache(
		uint64 PrimIndex,
		FUsdSchemaTranslationContext& Context,
		FUsdInfoCacheImpl& Impl,
		bool bIsInsideSkelRoot,
		int32& OutDepth,
		UE::UsdInfoCache::Private::EGeometryCachePrimState& OutState
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RecursiveCheckForGeometryCache);

		using namespace UE::UsdInfoCache::Private;

		FScopedUsdAllocs Allocs;

		// With this recursive check for geometry cache, we want to find branches with an animated mesh at the leaf and find the root where they can
		// meet. This root prim will collapses the static and animated meshes under it into a single geometry cache.

		FReadScopeLock ScopeLock{Impl.InfoMapLock};
		UE::UsdInfoCache::Private::FUsdPrimInfo& Info = Impl.PrimInfoArray[PrimIndex];

		pxr::UsdPrim UsdPrim = Context.Stage.GetPrimAtPath(Info.PrimPath);
		if (!UsdPrim)
		{
			return;
		}
		bIsInsideSkelRoot |= UsdPrim.IsA<pxr::UsdSkelRoot>();

		TArray<int32> Depths;
		Depths.SetNum(Info.ChildIndices.Num());

		TArray<EGeometryCachePrimState> States;
		States.SetNum(Info.ChildIndices.Num());

		const int32 MinBatchSize = 1;
		ParallelFor(
			TEXT("RecursiveCheckForGeometryCache"),
			Info.ChildIndices.Num(),
			MinBatchSize,
			[&Info, &Context, &Impl, bIsInsideSkelRoot, &Depths, &States](int32 Index)
			{
				RecursiveCheckForGeometryCache(Info.ChildIndices[Index], Context, Impl, bIsInsideSkelRoot, Depths[Index], States[Index]);
			}
		);

		// A geometry cache "branch" starts from an animated mesh prim for which we assign a depth of 0
		// Other branches, without any animated mesh, we don't care about and will remain at -1
		int32 Depth = -1;
		if (UsdUtils::IsAnimatedMesh(UsdPrim))
		{
			Depth = 0;
		}
		else
		{
			// The depth is propagated from children to parent, incremented by 1 at each level,
			// with the parent depth being the deepest of its children depth
			int32 ChildDepth = -1;
			for (int32 Index = 0; Index < Depths.Num(); ++Index)
			{
				if (Depths[Index] > -1)
				{
					ChildDepth = FMath::Max(ChildDepth, Depths[Index] + 1);
				}
			}
			Depth = ChildDepth;
		}

		// Along with the depth, we want some hints on the content of the subtree of the prim as this will tell us
		// if the prim can serve as a root and collapse its children into a GeometryCache. The sole condition for
		// being a valid root is that all the branches of the subtree are collapsible.
		EGeometryCachePrimState ChildrenState = EGeometryCachePrimState::None;
		for (EGeometryCachePrimState ChildState : States)
		{
			ChildrenState |= ChildState;
		}

		EGeometryCachePrimState PrimState = EGeometryCachePrimState::None;
		const bool bIsMesh = !!pxr::UsdGeomMesh(UsdPrim);
		const bool bIsXform = !!pxr::UsdGeomXform(UsdPrim);
		if (bIsMesh)
		{
			// A skinned mesh can never be considered part of a geometry cache.
			// Now that we use the UsdSkelSkeletonTranslator instead of the old UsdSkelRootTranslator we may run into these
			// skinned meshes that were already handled by a SkeletonTranslator elsewhere, and need to manually skip them
			if (GIsEditor && bIsInsideSkelRoot && UsdPrim.HasAPI<pxr::UsdSkelBindingAPI>())
			{
				PrimState = EGeometryCachePrimState::Uncollapsible;
			}
			else
			{
				// Animated or static mesh. Static meshes could potentially be animated by transforms in their hierarchy.
				// A mesh prim should be a leaf, but it can have GeomSubset prims as children, but those don't
				// affect the collapsibility status.
				PrimState = EGeometryCachePrimState::Mesh;
			}
		}
		else if (bIsXform)
		{
			// An xform prim is considered collapsible since it could have a mesh prim under it. It has to bubble up its children state.
			PrimState = ChildrenState != EGeometryCachePrimState::None ? ChildrenState | EGeometryCachePrimState::Xform
																	   : EGeometryCachePrimState::Xform;
		}
		else
		{
			// This prim is not considered collapsible with some exception
			// Like a Scope could have some meshes under it, so it has to bubble up its children state
			const bool bIsException = !!pxr::UsdGeomScope(UsdPrim);
			if (bIsException && EnumHasAnyFlags(ChildrenState, EGeometryCachePrimState::Mesh))
			{
				PrimState = ChildrenState;
			}
			else
			{
				PrimState = EGeometryCachePrimState::Uncollapsible;
			}
		}

		// A prim could be a potential root if it has a reference or payload to an allowed file type for GeometryCache
		bool bIsPotentialRoot = false;
		{
			pxr::UsdPrimCompositionQuery PrimCompositionQuery = pxr::UsdPrimCompositionQuery::GetDirectReferences(UsdPrim);
			for (const pxr::UsdPrimCompositionQueryArc& CompositionArc : PrimCompositionQuery.GetCompositionArcs())
			{
				if (CompositionArc.GetArcType() == pxr::PcpArcTypeReference)
				{
					pxr::SdfReferenceEditorProxy ReferenceEditor;
					pxr::SdfReference UsdReference;

					if (CompositionArc.GetIntroducingListEditor(&ReferenceEditor, &UsdReference))
					{
						FString FilePath = UsdToUnreal::ConvertString(UsdReference.GetAssetPath());
						FString Extension = FPaths::GetExtension(FilePath);

						if (Impl.AllowedExtensionsForGeometryCacheSource.Contains(Extension))
						{
							bIsPotentialRoot = true;
							break;
						}
					}
				}
				else if (CompositionArc.GetArcType() == pxr::PcpArcTypePayload)
				{
					pxr::SdfPayloadEditorProxy PayloadEditor;
					pxr::SdfPayload UsdPayload;

					if (CompositionArc.GetIntroducingListEditor(&PayloadEditor, &UsdPayload))
					{
						FString FilePath = UsdToUnreal::ConvertString(UsdPayload.GetAssetPath());
						FString Extension = FPaths::GetExtension(FilePath);

						if (Impl.AllowedExtensionsForGeometryCacheSource.Contains(Extension))
						{
							bIsPotentialRoot = true;
							break;
						}
					}
				}
			}
		}

		{
			FWriteScopeLock PrimLock = Impl.LockForWriting(Info);
			Info.GeometryCacheDepth = Depth;
			Info.GeometryCacheState = PrimState;
		}

		// We've encountered a potential root and the subtree has a geometry cache branch, so find its root
		if (bIsPotentialRoot && Depth > -1)
		{
			if (Depth > GGeometryCacheMaxDepth)
			{
				USD_LOG_USERWARNING(FText::Format(
					LOCTEXT(
						"DeepGeometryCache",
						"Prim '{0}' is potentially a geometry cache {1} levels deep, which exceeds the limit of {2}. "
						"This could affect its imported animation. The limit can be increased with the cvar USD.GeometryCache.MaxDepth if needed."
					),
					FText::FromString(Info.PrimPath.GetString()),
					Depth,
					GGeometryCacheMaxDepth
				));
			}
			FindValidGeometryCacheRoot(UsdPrim, Context, Impl, PrimState);
			Depth = -1;
		}

		OutDepth = Depth;
		OutState = PrimState;
	}

	void CheckForGeometryCache(FUsdSchemaTranslationContext& Context, FUsdInfoCacheImpl& Impl)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CheckForGeometryCache);

		using namespace UE::UsdInfoCache::Private;

		if (!GUseGeometryCacheUSD)
		{
			return;
		}

		static IConsoleVariable* ForceImportCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("USD.GeometryCache.ForceImport"));
		const bool bIsImporting = Context.bIsImporting || (ForceImportCvar && ForceImportCvar->GetBool());
		if (!bIsImporting)
		{
			// We only collapse subtrees into a single geometry cache when "importing".
			//
			// If we're not importing, all we need to know is whether the prim is an animated mesh or not. We'll do that on-demand
			// when the first call to IsPotentialGeometryCacheRoot() executes, because we don't want to spend time finding all animated
			// meshes in the entire stage only to never actually need that information
			return;
		}

		FScopedUsdAllocs UsdAllocs;
		pxr::UsdPrim PseudoRoot = Context.Stage.GetPseudoRoot();

		// If the stage doesn't contain any animated mesh prims, then don't bother doing a full check
		bool bHasAnimatedMesh = false;
		{
			TArray<TUsdStore<pxr::UsdPrim>> ChildPrims = UsdUtils::GetAllPrimsOfType(PseudoRoot, pxr::TfType::Find<pxr::UsdGeomMesh>());
			for (const TUsdStore<pxr::UsdPrim>& ChildPrim : ChildPrims)
			{
				if (UsdUtils::IsAnimatedMesh(ChildPrim.Get()))
				{
					bHasAnimatedMesh = true;
					break;
				}
			}
		}
		if (!bHasAnimatedMesh)
		{
			return;
		}

		int32 Depth = -1;
		EGeometryCachePrimState State = EGeometryCachePrimState::None;
		const bool bIsInsideSkelRoot = false;
		RecursiveCheckForGeometryCache(0, Context, Impl, bIsInsideSkelRoot, Depth, State);

		// If we end up with a positive depth, it means the check found an animated mesh somewhere
		// but no potential root before reaching the pseudoroot, so find one
		if (Depth > -1)
		{
			if (Depth > GGeometryCacheMaxDepth)
			{
				USD_LOG_USERWARNING(FText::Format(
					LOCTEXT(
						"DeepGeometryCacheInStage",
						"The stage has a geometry cache {0} levels deep, which exceeds the limit of {1}. "
						"This could affect its imported animation. The limit can be increased with the cvar USD.GeometryCache.MaxDepth if needed."
					),
					Depth,
					GGeometryCacheMaxDepth
				));
			}

			// The pseudoroot itself cannot be a root for the geometry cache so start from its children
			pxr::UsdPrimSiblingRange PrimChildren = PseudoRoot.GetFilteredChildren(pxr::UsdTraverseInstanceProxies(pxr::UsdPrimAllPrimsPredicate));
			for (pxr::UsdPrim Child : PrimChildren)
			{
				FindValidGeometryCacheRoot(Child, Context, Impl, State);
			}
		}
	}
#endif	  // USE_SD_SDK
}	 // namespace UE::USDInfoCache::Private

bool FUsdInfoCache::IsPotentialGeometryCacheRoot(const UE::FUsdPrim& Prim) const
{
	using namespace UE::UsdInfoCache::Private;

#if USE_USD_SDK
	if (FUsdInfoCacheImpl* ImplPtr = Impl.Get())
	{
		FReadScopeLock ScopeLock(ImplPtr->InfoMapLock);

		UE::FSdfPath PrimPath = Prim.GetPrimPath();
		if (FUsdPrimInfo* FoundInfo = ImplPtr->GetPrimInfo(PrimPath))
		{
			return ImplPtr->IsPotentialGeometryCacheRootInner(*FoundInfo, Prim);
		}

		// This should never happen: We should have cached the entire tree
		ensureMsgf(false, TEXT("Prim path '%s' has not been cached!"), *PrimPath.GetString());
	}
#endif	  // USE_USD_SDK

	return false;
}

void FUsdInfoCache::ResetTranslatedPrototypes()
{
	if (FUsdInfoCacheImpl* ImplPtr = Impl.Get())
	{
		FWriteScopeLock ScopeLock(ImplPtr->TranslatedPrototypesLock);
		return ImplPtr->TranslatedPrototypes.Reset();
	}
}

bool FUsdInfoCache::IsPrototypeTranslated(const UE::FSdfPath& PrototypePath)
{
	if (FUsdInfoCacheImpl* ImplPtr = Impl.Get())
	{
		FReadScopeLock ScopeLock(ImplPtr->TranslatedPrototypesLock);
		return ImplPtr->TranslatedPrototypes.Contains(PrototypePath);
	}

	return false;
}

void FUsdInfoCache::MarkPrototypeAsTranslated(const UE::FSdfPath& PrototypePath)
{
	if (FUsdInfoCacheImpl* ImplPtr = Impl.Get())
	{
		FWriteScopeLock ScopeLock(ImplPtr->TranslatedPrototypesLock);
		ImplPtr->TranslatedPrototypes.Add(PrototypePath);
	}
}

TOptional<uint64> FUsdInfoCache::GetSubtreeVertexCount(const UE::FSdfPath& Path)
{
	if (FUsdInfoCacheImpl* ImplPtr = Impl.Get())
	{
		FReadScopeLock ScopeLock(ImplPtr->InfoMapLock);

		if (const UE::UsdInfoCache::Private::FUsdPrimInfo* FoundInfo = ImplPtr->GetPrimInfo(Path))
		{
			FReadScopeLock PrimLock = ImplPtr->LockForReading(*FoundInfo);

			return FoundInfo->ExpectedVertexCountForSubtree;
		}

		// This should never happen: We should have cached the entire tree
		ensureMsgf(false, TEXT("Prim path '%s' has not been cached!"), *Path.GetString());
	}

	return {};
}

TOptional<uint64> FUsdInfoCache::GetSubtreeMaterialSlotCount(const UE::FSdfPath& Path)
{
	if (FUsdInfoCacheImpl* ImplPtr = Impl.Get())
	{
		FReadScopeLock ScopeLock(ImplPtr->InfoMapLock);

		if (const UE::UsdInfoCache::Private::FUsdPrimInfo* FoundInfo = ImplPtr->GetPrimInfo(Path))
		{
			FReadScopeLock PrimLock = ImplPtr->LockForReading(*FoundInfo);

			return FoundInfo->SubtreeMaterialSlots.Num();
		}

		// This should never happen: We should have cached the entire tree
		ensureMsgf(false, TEXT("Prim path '%s' has not been cached!"), *Path.GetString());
	}

	return {};
}

TOptional<TArray<UsdUtils::FUsdPrimMaterialSlot>> FUsdInfoCache::GetSubtreeMaterialSlots(const UE::FSdfPath& Path)
{
	if (FUsdInfoCacheImpl* ImplPtr = Impl.Get())
	{
		FReadScopeLock ScopeLock(ImplPtr->InfoMapLock);

		if (const UE::UsdInfoCache::Private::FUsdPrimInfo* FoundInfo = ImplPtr->GetPrimInfo(Path))
		{
			FReadScopeLock PrimLock = ImplPtr->LockForReading(*FoundInfo);

			return FoundInfo->SubtreeMaterialSlots;
		}

		// This should never happen: We should have cached the entire tree
		ensureMsgf(false, TEXT("Prim path '%s' has not been cached!"), *Path.GetString());
	}

	return {};
}

void FUsdInfoCache::RebuildCacheForSubtree(const UE::FUsdPrim& Prim, FUsdSchemaTranslationContext& Context)
{
	RebuildCacheForSubtrees({Prim.GetPrimPath()}, Context);
}

namespace UE::USDInfoCache::Private
{
	// Computes an efficient list of root prims to rebuild, so that we don't try rebuilding any one prim more than once
	TSet<UE::FSdfPath> GetSubtreeRootsToRebuild(const TArray<UE::FSdfPath>& SubtreeRootPaths)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GetSubtreeRootsToRebuild);

		TSet<UE::FSdfPath> Prefixes;
		for (const UE::FSdfPath& SubtreeRootPath : SubtreeRootPaths)
		{
			// If one of the paths is the pseudoroot there's not point doing anything else in here: It's a full rebuild
			if (SubtreeRootPath.IsAbsoluteRootPath())
			{
				return {SubtreeRootPath};
			}

			const UE::FSdfPath* PrefixToRemove = nullptr;

			bool bKeepNewRootPath = true;
			for (const UE::FSdfPath& Prefix : Prefixes)
			{
				// We already have a path that is a prefix of SubtreeRootPath: We'll already resync
				// SubtreeRootPath as part of resyncing Prefix anyway, so we don't need to use it
				if (SubtreeRootPath.HasPrefix(Prefix))
				{
					bKeepNewRootPath = false;
					break;
				}

				// This new subtree root path is an actual parent of another path we already saw.
				// We immediately know we want this path, because if we have added Prefix to our set in the past,
				// then we know we never had a prefix to it, and so we can't have a prefix to SubtreeRootPath either.
				//
				// Furthermore, we can get rid of this Prefix and just use SubtreeRootPath instead!
				if (Prefix.HasPrefix(SubtreeRootPath))
				{
					PrefixToRemove = &Prefix;
					break;
				}
			}
			if (!bKeepNewRootPath)
			{
				continue;
			}

			if (PrefixToRemove)
			{
				Prefixes.Remove(*PrefixToRemove);
			}
			Prefixes.Add(SubtreeRootPath);
		}

		return Prefixes;
	}
}	 // namespace UE::USDInfoCache::Private

void FUsdInfoCache::RebuildCacheForSubtrees(const TArray<UE::FSdfPath>& SubtreeRootPaths, FUsdSchemaTranslationContext& Context)
{
#if USE_USD_SDK
	using namespace UE::USDInfoCache::Private;

	TRACE_CPUPROFILER_EVENT_SCOPE(FUsdInfoCache::RebuildCacheForSubtrees);

	if (SubtreeRootPaths.Num() == 0)
	{
		return;
	}

	FUsdInfoCacheImpl* ImplPtr = Impl.Get();
	if (!ImplPtr)
	{
		return;
	}

	// We can't deallocate our info cache pointer with the Usd allocator
	FScopedUnrealAllocs UEAllocs;

	TGuardValue<bool> Guard{Context.bIsBuildingInfoCache, true};
	{
		FUsdSchemaTranslatorRegistry& Registry = FUsdSchemaTranslatorRegistry::Get();

		pxr::TfToken MaterialPurposeToken = pxr::UsdShadeTokens->allPurpose;
		if (!Context.MaterialPurpose.IsNone())
		{
			MaterialPurposeToken = UnrealToUsd::ConvertToken(*Context.MaterialPurpose.ToString()).Get();
		}

		TSet<UE::FSdfPath> ProcessedRootPaths = GetSubtreeRootsToRebuild(SubtreeRootPaths);
		const bool bIsFullRebuild = (ProcessedRootPaths.Num() == 1 && ProcessedRootPaths.Contains(UE::FSdfPath::AbsoluteRootPath()));

		// We need to rebuild everything, don't bother carefully cleaning things up
		if (bIsFullRebuild)
		{
			Clear();

			RepopulateInfoMapSubtree(UE::FSdfPath::AbsoluteRootPath(), Context.Stage, *ImplPtr);
		}
		// We need to rebuild just some subtrees of the existing cache
		else
		{
			// Prepare for cleanup
			ImplPtr->ClearTransientInfo();

			// Reset/remove everything
			for (const UE::FSdfPath& RootPath : ProcessedRootPaths)
			{
				CleanUpInfoMapSubtree(RootPath, *ImplPtr);
			}
			ImplPtr->DisconnectResetPrimAuxLinks();

			// Rebuild the new info struct nodes and connect parent/child links
			for (const UE::FSdfPath& RootPath : ProcessedRootPaths)
			{
				RepopulateInfoMapSubtree(RootPath, Context.Stage, *ImplPtr);
			}
		}

		if (!Context.Stage)
		{
			// Quit only here so that the AUsdStageActor can just blindly do a "full rebuild" when it wants to cleanup as well
			return;
		}
		UE::FUsdPrim PseudoRoot = Context.Stage.GetPseudoRoot();
		const uint64 PseudoRootIndex = 0;

		// Propagate vertex and material slot counts before we query CollapsesChildren because the Xformable
		// translator needs to know when it would generate too large a static mesh
		uint64 SubtreeVertexCount = 0;
		TArray<UsdUtils::FUsdPrimMaterialSlot> SubtreeSlots;
		const bool bPossibleInheritedBindings = false;
		RecursivePropagateVertexAndMaterialSlotCounts(
			PseudoRootIndex,
			Context,
			MaterialPurposeToken,
			*ImplPtr,
			Registry,
			SubtreeVertexCount,
			SubtreeSlots,
			bPossibleInheritedBindings
		);

		UpdateInfoForPointInstancers(Context, *ImplPtr);

		CheckForGeometryCache(Context, *ImplPtr);

		RecursiveQueryCollapsesChildren(PseudoRootIndex, Context, *ImplPtr, Registry);

		RegisterInstanceableAuxPrims(Context, bIsFullRebuild, *ImplPtr);

		const bool bIsPrimCollapsedOrCollapseRoot = false;
		RecursiveCollectMaterialSlotCounts(PseudoRootIndex, *ImplPtr, Context, bIsPrimCollapsedOrCollapseRoot);
	}
#endif	  // USE_USD_SDK
}

void FUsdInfoCache::Clear()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUsdInfoCache::Clear);

	if (FUsdInfoCacheImpl* ImplPtr = Impl.Get())
	{
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(InfoMapEmpty);
			FWriteScopeLock ScopeLock(ImplPtr->InfoMapLock);
			ImplPtr->PrimInfoArray.Empty();
			ImplPtr->StaleInfoIndices.Empty();
			ImplPtr->InfoMap.Empty();
		}

		ImplPtr->ClearTransientInfo();

		ResetTranslatedPrototypes();
	}
}

bool FUsdInfoCache::IsEmpty()
{
	if (FUsdInfoCacheImpl* ImplPtr = Impl.Get())
	{
		FReadScopeLock ScopeLock(ImplPtr->InfoMapLock);
		return ImplPtr->InfoMap.IsEmpty();
	}

	return true;
}

TOptional<bool> FUsdInfoCache::CanXformableSubtreeBeCollapsed(const UE::FSdfPath& RootPath, FUsdSchemaTranslationContext& Context) const
{
#if USE_USD_SDK
	using namespace UE::UsdInfoCache::Private;

	TRACE_CPUPROFILER_EVENT_SCOPE(FUsdInfoCache::CanSubtreeBeCollapsed);

	// The only reason this function exists is that FUsdGeomXformableTranslator::CollapsesChildren() needs to check if all
	// GeomXformable prims in its subtree return true for CanBeCollapsed().
	//
	// We don't want to compute this for the entire stage on the main info cache build, because it may not be needed.
	// However, we definitely do not want each call to FUsdGeomXformableTranslator::CollapsesChildren() to traverse its entire
	// subtree of prims calling CanBeCollapsed() on their own: That would be a massive waste since the output is going to
	// be the same regardless of the caller.
	//
	// This is the awkward compromise where the first call to FUsdGeomXformableTranslator::CollapsesChildren() will traverse
	// its entire subtree and fill this in, and subsequent calls can just use those results, or fill in additional subtrees, etc.

	if (FUsdInfoCacheImpl* ImplPtr = Impl.Get())
	{
		FReadScopeLock ScopeLock(ImplPtr->InfoMapLock);

		if (const FUsdPrimInfo* FoundInfo = ImplPtr->GetPrimInfo(RootPath))
		{
			FReadScopeLock PrimLock = ImplPtr->LockForReading(*FoundInfo);
			if (FoundInfo->bXformSubtreeCanBeCollapsed.IsSet())
			{
				return FoundInfo->bXformSubtreeCanBeCollapsed.GetValue();
			}
		}

		TOptional<bool> bCanBeCollapsed;

		// Fill in missing entries for CanBeCollapsed on-demand and compute the value for the prim at RootPath,
		// if we can still access our stage
		pxr::UsdStageRefPtr Stage = pxr::UsdStageRefPtr{Context.Stage};
		if (Stage)
		{
			if (pxr::UsdPrim Prim = Stage->GetPrimAtPath(RootPath))
			{
				FUsdSchemaTranslatorRegistry& Registry = FUsdSchemaTranslatorRegistry::Get();

				bCanBeCollapsed = RecursiveQueryCanBeCollapsed(Prim, Context, *ImplPtr, Registry);
			}
		}

		// We can potentially still fail to find this here, in case our stage reference is broken (i.e. called outside of the
		// main infocache build callstack).
		//
		// There shouldn't be any point in checking our FoundInfo again though: If we didn't return anything valid from
		// our call to RecursiveQueryCanBeCollapsed, then we didn't put anything new on the InfoMap either
		if (bCanBeCollapsed.IsSet())
		{
			return bCanBeCollapsed;
		}

		USD_LOG_WARNING(
			TEXT(
				"Failed to find whether subtree '%s' can be collapsed or not. Note: This function is meant to be used only during the main FUsdInfoCache build!"
			),
			*RootPath.GetString()
		);
	}
#endif	  // USE_USD_SDK

	return {};
}

#undef LOCTEXT_NAMESPACE
