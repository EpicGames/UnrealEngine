// Copyright Epic Games, Inc. All Rights Reserved.

#include "Objects/USDPrimLinkCache.h"

#include "UsdWrappers/SdfPath.h"

#include "Misc/ScopeRWLock.h"

struct FUsdPrimLinkCache::FUsdPrimLinkCacheImpl
{
	FUsdPrimLinkCacheImpl() = default;
	FUsdPrimLinkCacheImpl(const FUsdPrimLinkCacheImpl& Other)
		: FUsdPrimLinkCacheImpl()
	{
		FReadScopeLock OtherScopedPrimPathToAssetsLock(Other.PrimPathToAssetsLock);
		FWriteScopeLock ThisScopedPrimPathToAssetsLock(PrimPathToAssetsLock);

		PrimPathToAssets = Other.PrimPathToAssets;
		AssetToPrimPaths = Other.AssetToPrimPaths;
	}

	// Information we may have about a subset of prims
	TMap<UE::FSdfPath, TArray<TWeakObjectPtr<UObject>>> PrimPathToAssets;
	TMap<TWeakObjectPtr<UObject>, TArray<UE::FSdfPath>> AssetToPrimPaths;
	mutable FRWLock PrimPathToAssetsLock;
};

FUsdPrimLinkCache::FUsdPrimLinkCache()
{
	Impl = MakeUnique<FUsdPrimLinkCache::FUsdPrimLinkCacheImpl>();
}

FUsdPrimLinkCache::~FUsdPrimLinkCache()
{
}

void FUsdPrimLinkCache::Serialize(FArchive& Ar)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUsdPrimLinkCache::Serialize);

	if (FUsdPrimLinkCacheImpl* ImplPtr = Impl.Get())
	{
		FWriteScopeLock ScopeLock(ImplPtr->PrimPathToAssetsLock);
		Ar << ImplPtr->PrimPathToAssets;
		Ar << ImplPtr->AssetToPrimPaths;
	}
}

bool FUsdPrimLinkCache::ContainsInfoAboutPrim(const UE::FSdfPath& Path) const
{
	if (FUsdPrimLinkCacheImpl* ImplPtr = Impl.Get())
	{
		FReadScopeLock ScopeLock(ImplPtr->PrimPathToAssetsLock);
		return ImplPtr->PrimPathToAssets.Contains(Path);
	}

	return false;
}

void FUsdPrimLinkCache::LinkAssetToPrim(const UE::FSdfPath& Path, UObject* Asset)
{
	FUsdPrimLinkCacheImpl* ImplPtr = Impl.Get();
	if (!ImplPtr)
	{
		return;
	}

	FWriteScopeLock ScopeLock(ImplPtr->PrimPathToAssetsLock);

	ImplPtr->PrimPathToAssets.FindOrAdd(Path).AddUnique(Asset);
	ImplPtr->AssetToPrimPaths.FindOrAdd(Asset).AddUnique(Path);
}

void FUsdPrimLinkCache::UnlinkAssetFromPrim(const UE::FSdfPath& Path, UObject* Asset)
{
	FUsdPrimLinkCacheImpl* ImplPtr = Impl.Get();
	if (!ImplPtr)
	{
		return;
	}

	FWriteScopeLock ScopeLock(ImplPtr->PrimPathToAssetsLock);

	if (TArray<TWeakObjectPtr<UObject>>* FoundAssetsForPrim = ImplPtr->PrimPathToAssets.Find(Path))
	{
		FoundAssetsForPrim->Remove(Asset);
	}
	if (TArray<UE::FSdfPath>* FoundPrimPathsForAsset = ImplPtr->AssetToPrimPaths.Find(Asset))
	{
		FoundPrimPathsForAsset->Remove(Path);
	}
}

TArray<TWeakObjectPtr<UObject>> FUsdPrimLinkCache::RemoveAllAssetPrimLinks(const UE::FSdfPath& Path)
{
	FUsdPrimLinkCacheImpl* ImplPtr = Impl.Get();
	if (!ImplPtr)
	{
		return {};
	}

	FWriteScopeLock ScopeLock(ImplPtr->PrimPathToAssetsLock);

	TArray<TWeakObjectPtr<UObject>> Assets;
	ImplPtr->PrimPathToAssets.RemoveAndCopyValue(Path, Assets);

	for (const TWeakObjectPtr<UObject>& Asset : Assets)
	{
		if (TArray<UE::FSdfPath>* PrimPaths = ImplPtr->AssetToPrimPaths.Find(Asset))
		{
			PrimPaths->Remove(Path);
		}
	}

	return Assets;
}

TArray<UE::FSdfPath> FUsdPrimLinkCache::RemoveAllAssetPrimLinks(const UObject* Asset)
{
	FUsdPrimLinkCacheImpl* ImplPtr = Impl.Get();
	if (!ImplPtr)
	{
		return {};
	}

	FWriteScopeLock ScopeLock(ImplPtr->PrimPathToAssetsLock);

	TArray<UE::FSdfPath> PrimPaths;
	ImplPtr->AssetToPrimPaths.RemoveAndCopyValue(const_cast<UObject*>(Asset), PrimPaths);

	for (const UE::FSdfPath& Path : PrimPaths)
	{
		if (TArray<TWeakObjectPtr<UObject>>* Assets = ImplPtr->PrimPathToAssets.Find(Path))
		{
			Assets->Remove(const_cast<UObject*>(Asset));
		}
	}

	return PrimPaths;
}

void FUsdPrimLinkCache::RemoveAllAssetPrimLinks()
{
	FUsdPrimLinkCacheImpl* ImplPtr = Impl.Get();
	if (!ImplPtr)
	{
		return;
	}

	FWriteScopeLock ScopeLock(ImplPtr->PrimPathToAssetsLock);

	ImplPtr->PrimPathToAssets.Empty();
	ImplPtr->AssetToPrimPaths.Empty();
}

TArray<TWeakObjectPtr<UObject>> FUsdPrimLinkCache::GetAllAssetsForPrim(const UE::FSdfPath& Path) const
{
	FUsdPrimLinkCacheImpl* ImplPtr = Impl.Get();
	if (!ImplPtr)
	{
		return {};
	}
	FReadScopeLock ScopeLock(ImplPtr->PrimPathToAssetsLock);

	if (const TArray<TWeakObjectPtr<UObject>>* FoundAssets = ImplPtr->PrimPathToAssets.Find(Path))
	{
		return *FoundAssets;
	}

	return {};
}

TArray<UE::FSdfPath> FUsdPrimLinkCache::GetPrimsForAsset(const UObject* Asset) const
{
	if (!Asset)
	{
		return {};
	}

	FUsdPrimLinkCacheImpl* ImplPtr = Impl.Get();
	if (!ImplPtr)
	{
		return {};
	}
	FReadScopeLock ScopeLock(ImplPtr->PrimPathToAssetsLock);

	if (const TArray<UE::FSdfPath>* FoundPrims = ImplPtr->AssetToPrimPaths.Find(Asset))
	{
		return *FoundPrims;
	}

	return {};
}

TMap<UE::FSdfPath, TArray<TWeakObjectPtr<UObject>>> FUsdPrimLinkCache::GetAllAssetPrimLinks() const
{
	FUsdPrimLinkCacheImpl* ImplPtr = Impl.Get();
	if (!ImplPtr)
	{
		return {};
	}

	return ImplPtr->PrimPathToAssets;
}

void FUsdPrimLinkCache::Clear()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(AUsdStageActor::UnloadUsdStage);

	if (FUsdPrimLinkCacheImpl* ImplPtr = Impl.Get())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PrimPathToAssetsEmpty);
		FWriteScopeLock ScopeLock(ImplPtr->PrimPathToAssetsLock);
		ImplPtr->PrimPathToAssets.Empty();
		ImplPtr->AssetToPrimPaths.Empty();
	}
}

bool FUsdPrimLinkCache::IsEmpty()
{
	if (FUsdPrimLinkCacheImpl* ImplPtr = Impl.Get())
	{
		FReadScopeLock ScopeLock(ImplPtr->PrimPathToAssetsLock);
		return ImplPtr->PrimPathToAssets.IsEmpty();
	}

	return true;
}
