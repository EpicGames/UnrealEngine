// Copyright Epic Games, Inc. All Rights Reserved.

#include "Objects/USDSchemaTranslator.h"

#include "Objects/USDInfoCache.h"
#include "UnrealUSDWrapper.h"
#include "USDMaterialUtils.h"
#include "USDMemory.h"
#include "USDTypesConversion.h"
#include "USDConversionUtils.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdTyped.h"

#include "Algo/Find.h"
#include "Async/Async.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/base/tf/token.h"
#include "pxr/base/tf/type.h"
#include "pxr/usd/usd/prim.h"
#include "pxr/usd/usdShade/tokens.h"
#include "USDIncludesEnd.h"
#endif	  // #if USE_USD_SDK

#define LOCTEXT_NAMESPACE "USDSchemaTranslator"

int32 FRegisteredSchemaTranslatorHandle::CurrentSchemaTranslatorId = 0;

FUsdSchemaTranslatorRegistry& FUsdSchemaTranslatorRegistry::Get()
{
	static FUsdSchemaTranslatorRegistry Singleton;
	return Singleton;
}

TSharedPtr<FUsdSchemaTranslator> FUsdSchemaTranslatorRegistry::CreateTranslatorForSchema(
	TSharedRef<FUsdSchemaTranslationContext> InTranslationContext,
	const UE::FUsdTyped& InSchema
)
{
#if USE_USD_SDK
	TUsdStore<pxr::UsdPrim> Prim = pxr::UsdPrim(InSchema.GetPrim());
	if (!Prim.Get())
	{
		return {};
	}

	for (TPair<FString, FSchemaTranslatorsStack>& RegisteredSchemasStack : RegisteredSchemaTranslators)
	{
		pxr::TfToken RegisteredSchemaToken(UnrealToUsd::ConvertString(*RegisteredSchemasStack.Key).Get());
		pxr::TfType RegisteredSchemaType = pxr::UsdSchemaRegistry::GetTypeFromName(RegisteredSchemaToken);

		if (!RegisteredSchemaType.IsUnknown() && Prim.Get().IsA(RegisteredSchemaType) && RegisteredSchemasStack.Value.Num() > 0)
		{
			return RegisteredSchemasStack.Value.Top().CreateFunction(InTranslationContext, InSchema);
		}
	}
#endif	  // #if USE_USD_SDK

	return {};
}

FRegisteredSchemaTranslatorHandle FUsdSchemaTranslatorRegistry::Register(const FString& SchemaName, FCreateTranslator CreateFunction)
{
#if USE_USD_SDK
	FSchemaTranslatorsStack* SchemaTranslatorsStack = FindSchemaTranslatorStack(SchemaName);

	if (!SchemaTranslatorsStack)
	{
		// Insert most specialized first
		int32 SchemaRegistryIndex = 0;

		pxr::TfToken SchemaToRegisterToken(UnrealToUsd::ConvertString(*SchemaName).Get());
		pxr::TfType SchemaToRegisterType = pxr::UsdSchemaRegistry::GetTypeFromName(SchemaToRegisterToken);

		for (TPair<FString, FSchemaTranslatorsStack>& RegisteredSchemasStack : RegisteredSchemaTranslators)
		{
			pxr::TfToken RegisteredSchemaToken(UnrealToUsd::ConvertString(*RegisteredSchemasStack.Key).Get());
			pxr::TfType RegisteredSchemaType = pxr::UsdSchemaRegistry::GetTypeFromName(RegisteredSchemaToken);

			if (SchemaToRegisterType.IsA(RegisteredSchemaType))
			{
				// We need to be registered before our ancestor types
				break;
			}
			else
			{
				++SchemaRegistryIndex;
			}
		}

		SchemaTranslatorsStack = &RegisteredSchemaTranslators.EmplaceAt_GetRef(SchemaRegistryIndex, SchemaName, FSchemaTranslatorsStack()).Value;
	}

	FRegisteredSchemaTranslator RegisteredSchemaTranslator;
	RegisteredSchemaTranslator.Handle = FRegisteredSchemaTranslatorHandle(SchemaName);
	RegisteredSchemaTranslator.CreateFunction = CreateFunction;

	SchemaTranslatorsStack->Push(RegisteredSchemaTranslator);

	ExternalSchemaTranslatorCount++;

	return RegisteredSchemaTranslator.Handle;
#else
	return FRegisteredSchemaTranslatorHandle();
#endif	  // #if USE_USD_SDK
}

void FUsdSchemaTranslatorRegistry::Unregister(const FRegisteredSchemaTranslatorHandle& TranslatorHandle)
{
	FSchemaTranslatorsStack* SchemaTranslatorsStack = FindSchemaTranslatorStack(TranslatorHandle.GetSchemaName());

	if (!SchemaTranslatorsStack)
	{
		return;
	}

	for (FSchemaTranslatorsStack::TIterator RegisteredSchemaTranslatorIt = SchemaTranslatorsStack->CreateIterator(); RegisteredSchemaTranslatorIt;
		 ++RegisteredSchemaTranslatorIt)
	{
		if (RegisteredSchemaTranslatorIt->Handle.GetId() == TranslatorHandle.GetId())
		{
			RegisteredSchemaTranslatorIt.RemoveCurrent();
			break;
		}
	}
}

int32 FUsdSchemaTranslatorRegistry::GetExternalSchemaTranslatorCount()
{
	return ExternalSchemaTranslatorCount;
}

FUsdSchemaTranslationContext::FUsdSchemaTranslationContext(const UE::FUsdStage& InStage)
	: Stage(InStage)
{
	RenderContext = UnrealIdentifiers::UniversalRenderContext;
}

FUsdSchemaTranslationContext::FUsdSchemaTranslationContext(const UE::FUsdStage& InStage, UUsdAssetCache3& InAssetCache)
	: Stage(InStage)
	, UsdAssetCache(&InAssetCache)
{
	RenderContext = UnrealIdentifiers::UniversalRenderContext;
}

FUsdSchemaTranslatorRegistry::FSchemaTranslatorsStack* FUsdSchemaTranslatorRegistry::FindSchemaTranslatorStack(const FString& SchemaName)
{
	TPair<FString, FSchemaTranslatorsStack>* Result = Algo::FindByPredicate(
		RegisteredSchemaTranslators,
		[&SchemaName](const TPair<FString, FSchemaTranslatorsStack>& Element) -> bool
		{
			return Element.Key == SchemaName;
		}
	);

	if (Result)
	{
		return &Result->Value;
	}
	else
	{
		return nullptr;
	}
}

void FUsdSchemaTranslatorRegistry::ResetExternalTranslatorCount()
{
	ExternalSchemaTranslatorCount = 0;
}

void FUsdSchemaTranslationContext::CompleteTasks()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUsdSchemaTranslationContext::CompleteTasks);

	FScopedSlowTask SlowTask(TranslatorTasks.Num(), LOCTEXT("TasksProgress", "Executing USD Schema tasks"));

	// These are just pointers into the TranslatorTasks items that are task chains that are in the ESchemaTranslationStatus::Pending state.
	// We'll use this below to execute tasks until all items in TranslatorTasks are pending, and then we switch to/from an ExclusiveSync pass.
	// This because we need to ensure that all ExclusiveSync tasks are run in isolation.
	TSet<FUsdSchemaTranslatorTaskChain*> PendingTaskChains;

	// Note that this first pass is for tasks that allow concurrent execution (so *not* exclusive sync tasks). If this is ever changed,
	// we would also need to change the behavior of StartIfAsync to delay until the proper async pass, and not start right away
	bool bExclusiveSyncTasks = false;

	bool bFinished = (TranslatorTasks.Num() == 0);
	while (!bFinished)
	{
		while (TranslatorTasks.Num() > PendingTaskChains.Num())
		{
			for (TArray<TSharedPtr<FUsdSchemaTranslatorTaskChain>>::TIterator TaskChainIterator = TranslatorTasks.CreateIterator(); TaskChainIterator;
				 ++TaskChainIterator)
			{
				TSharedPtr<FUsdSchemaTranslatorTaskChain>& TaskChain = *TaskChainIterator;

				ESchemaTranslationStatus TaskChainStatus = TaskChain->Execute(bExclusiveSyncTasks);

				if (TaskChainStatus == ESchemaTranslationStatus::Done)
				{
					SlowTask.EnterProgressFrame();
					TaskChainIterator.RemoveCurrent();
				}
				else if (TaskChainStatus == ESchemaTranslationStatus::Pending)
				{
					PendingTaskChains.Add(TaskChain.Get());
				}
			}
		}

		bExclusiveSyncTasks = !bExclusiveSyncTasks;
		PendingTaskChains.Reset();

		bFinished = (TranslatorTasks.Num() == 0);
	}
}

bool FUsdSchemaTranslator::IsCollapsed(ECollapsingType CollapsingType) const
{
#if USE_USD_SDK
	TRACE_CPUPROFILER_EVENT_SCOPE(FUsdSchemaTranslator::IsCollapsed);

	if (!Context->bIsBuildingInfoCache)
	{
		return Context->UsdInfoCache->IsPathCollapsed(PrimPath, CollapsingType);
	}

	// This is merely a fallback, and we should never need this
	return CanBeCollapsed(CollapsingType);
#else	  // #if USE_USD_SDK
	return false;
#endif	  // #if USE_USD_SDK
}

UE::FSdfPath FUsdSchemaTranslator::GetPrototypePrimPath() const
{
	return UsdUtils::GetPrototypePrimPath(GetPrim());
}

bool FUsdSchemaTranslator::ShouldSkipInstance() const
{
	if (!UsdUtils::IsInstancingAwareTranslationEnabled())
	{
		return false;
	}

	UE::FSdfPath PrototypePath = GetPrototypePrimPath();
	if (!PrototypePath.IsEmpty())
	{
		if (Context->UsdInfoCache->IsPrototypeTranslated(PrototypePath))
		{
			return true;
		}

		Context->UsdInfoCache->MarkPrototypeAsTranslated(PrototypePath);
	}

	return false;
}

void FSchemaTranslatorTask::Start()
{
	if (LaunchPolicy == ESchemaTranslationLaunchPolicy::Async && IsInGameThread())
	{
		Result = Async(
#if WITH_EDITOR
			EAsyncExecution::LargeThreadPool,
#else
			EAsyncExecution::ThreadPool,
#endif	  // WITH_EDITOR
			[this]() -> bool
			{
				return DoWork();
			}
		);
	}
	else
	{
		// Execute on this thread
		if (!DoWork())
		{
			Continuation.Reset();
		}
	}
}

void FSchemaTranslatorTask::StartIfAsync()
{
	if (LaunchPolicy == ESchemaTranslationLaunchPolicy::Async)
	{
		Start();
	}
}

bool FSchemaTranslatorTask::DoWork()
{
	ensure(bIsDone == false);
	bool bContinue = Callable();
	bIsDone = true;

	return bContinue;
}

FUsdSchemaTranslatorTaskChain& FUsdSchemaTranslatorTaskChain::Do(ESchemaTranslationLaunchPolicy InPolicy, TFunction<bool()> Callable)
{
	if (!CurrentTask)
	{
		CurrentTask = MakeShared<FSchemaTranslatorTask>(InPolicy, Callable);

		CurrentTask->StartIfAsync();	// Queue it right now if async
	}
	else
	{
		Then(InPolicy, Callable);
	}

	return *this;
}

FUsdSchemaTranslatorTaskChain& FUsdSchemaTranslatorTaskChain::Then(ESchemaTranslationLaunchPolicy InPolicy, TFunction<bool()> Callable)
{
	TSharedPtr<FSchemaTranslatorTask> LastTask = CurrentTask;

	while (LastTask->Continuation.IsValid())
	{
		LastTask = LastTask->Continuation;
	}

	if (LastTask)
	{
		LastTask->Continuation = MakeShared<FSchemaTranslatorTask>(InPolicy, Callable);
	}

	return *this;
}

namespace UsdSchemaTranslatorTaskChainImpl
{
	FORCEINLINE bool CanStart(FSchemaTranslatorTask* Task, bool bExclusiveSyncTasks)
	{
		return (Task->LaunchPolicy == ESchemaTranslationLaunchPolicy::ExclusiveSync) == bExclusiveSyncTasks;
	}
}

ESchemaTranslationStatus FUsdSchemaTranslatorTaskChain::Execute(bool bExclusiveSyncTasks)
{
	FSchemaTranslatorTask* TranslatorTask = CurrentTask.Get();

	if (TranslatorTask == nullptr)
	{
		return ESchemaTranslationStatus::Done;
	}

	if (!TranslatorTask->IsDone())
	{
		if (!TranslatorTask->IsStarted())
		{
			if (UsdSchemaTranslatorTaskChainImpl::CanStart(TranslatorTask, bExclusiveSyncTasks))
			{
				TranslatorTask->Start();
			}
			else
			{
				return ESchemaTranslationStatus::Pending;
			}
		}

		return ESchemaTranslationStatus::InProgress;
	}
	else
	{
		if (CurrentTask->Result.IsSet())
		{
			CurrentTask = CurrentTask->Result->Get() ? CurrentTask->Continuation : nullptr;
		}
		else
		{
			CurrentTask = CurrentTask->Continuation;
		}

		if ((TranslatorTask = CurrentTask.Get()) != nullptr)
		{
			if (UsdSchemaTranslatorTaskChainImpl::CanStart(TranslatorTask, bExclusiveSyncTasks))
			{
				if (IsInGameThread())
				{
					TranslatorTask->StartIfAsync();	   // Queue the next task asap if async
				}
				else
				{
					TranslatorTask->Start();
				}
			}
			else
			{
				return ESchemaTranslationStatus::Pending;
			}
		}
	}

	return TranslatorTask ? ESchemaTranslationStatus::InProgress : ESchemaTranslationStatus::Done;
}

#undef LOCTEXT_NAMESPACE
