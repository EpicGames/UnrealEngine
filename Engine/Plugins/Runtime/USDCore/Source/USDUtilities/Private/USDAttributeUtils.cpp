// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDAttributeUtils.h"

#include "UnrealUSDWrapper.h"
#include "USDErrorUtils.h"
#include "USDLayerUtils.h"
#include "USDMemory.h"
#include "USDProjectSettings.h"
#include "USDTypesConversion.h"

#include "UsdWrappers/PcpPrimIndex.h"
#include "UsdWrappers/SdfLayer.h"
#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/SdfPrimSpec.h"
#include "UsdWrappers/UsdAttribute.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdStage.h"

#include "Framework/Notifications/NotificationManager.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Notifications/SNotificationList.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/pcp/primIndex.h"
#include "pxr/usd/sdf/layerUtils.h"
#include "pxr/usd/usd/attribute.h"
#include "pxr/usd/usd/editContext.h"
#include "pxr/usd/usd/stage.h"
#include "USDIncludesEnd.h"
#endif	  // #if USE_USD_SDK

#define LOCTEXT_NAMESPACE "UsdAttributeUtils"

namespace UsdUtils
{
#if USE_USD_SDK
	const pxr::TfToken MutedToken = UnrealToUsd::ConvertToken(TEXT("UE:Muted")).Get();
#endif	  // #if USE_USD_SDK
}

bool UsdUtils::MuteAttribute(UE::FUsdAttribute& Attribute, const UE::FUsdStage& Stage)
{
#if USE_USD_SDK
	FScopedUsdAllocs Allocs;

	const pxr::UsdAttribute& UsdAttribute = static_cast<const pxr::UsdAttribute&>(Attribute);
	const pxr::UsdStageRefPtr UsdStage{Stage};
	if (!UsdAttribute || !UsdStage)
	{
		return false;
	}

	pxr::SdfLayerRefPtr UEPersistentState = UsdUtils::GetUEPersistentStateSublayer(Stage);
	if (!UEPersistentState)
	{
		return false;
	}

	pxr::SdfLayerRefPtr UESessionState = UsdUtils::GetUESessionStateSublayer(Stage);
	if (!UESessionState)
	{
		return false;
	}

	pxr::SdfChangeBlock ChangeBlock;

	// Mark it as muted on the persistent state
	{
		pxr::UsdEditContext Context(Stage, UEPersistentState);

		UsdAttribute.SetCustomDataByKey(MutedToken, pxr::VtValue{true});
	}

	// Actually author the opinions that cause it to be muted on the session state
	{
		pxr::UsdEditContext Context(Stage, UESessionState);

		pxr::VtValue Value;
		UsdAttribute.Get(&Value, pxr::UsdTimeCode::Default());

		// Clear the attribute so that it also gets rid of any time samples it may have
		UsdAttribute.Clear();

		if (Value.IsEmpty())
		{
			// It doesn't have any default value, so just mute the attribute completely
			UsdAttribute.Block();
		}
		else
		{
			// It has a default, non-animated value from a weaker opinion: Use that instead
			UsdAttribute.Set(Value);
		}
	}

	return true;
#else
	return false;
#endif	  // #if USE_USD_SDK
}

bool UsdUtils::UnmuteAttribute(UE::FUsdAttribute& Attribute, const UE::FUsdStage& Stage)
{
#if USE_USD_SDK
	FScopedUsdAllocs Allocs;

	pxr::UsdAttribute& UsdAttribute = static_cast<pxr::UsdAttribute&>(Attribute);
	const pxr::UsdStageRefPtr UsdStage{Stage};
	if (!UsdAttribute || !UsdStage)
	{
		return false;
	}

	if (!IsAttributeMuted(Attribute, Stage))
	{
		return true;
	}

	pxr::SdfLayerRefPtr UEPersistentState = UsdUtils::GetUEPersistentStateSublayer(Stage);
	if (!UEPersistentState)
	{
		return false;
	}

	pxr::SdfLayerRefPtr UESessionState = UsdUtils::GetUESessionStateSublayer(Stage);
	if (!UESessionState)
	{
		return false;
	}

	pxr::SdfChangeBlock ChangeBlock;

	// Remove the mute tag on the persistent state layer
	{
		pxr::UsdEditContext Context(Stage, UEPersistentState);
		UsdAttribute.ClearCustomDataByKey(MutedToken);
	}

	// Clear our opinion of it on our session state layer
	{
		pxr::UsdEditContext Context(Stage, UESessionState);
		UsdAttribute.Clear();
	}

	return true;
#else
	return false;
#endif	  // #if USE_USD_SDK
}

bool UsdUtils::IsAttributeMuted(const UE::FUsdAttribute& Attribute, const UE::FUsdStage& Stage)
{
#if USE_USD_SDK
	FScopedUsdAllocs Allocs;

	const pxr::UsdAttribute& UsdAttribute = static_cast<const pxr::UsdAttribute&>(Attribute);
	if (!UsdAttribute)
	{
		return false;
	}

	pxr::VtValue Data = UsdAttribute.GetCustomDataByKey(MutedToken);
	if (Data.IsHolding<bool>())
	{
		return Data.Get<bool>();
	}
#endif	  // #if USE_USD_SDK

	return false;
}

bool UsdUtils::HasAuthoredDefaultOpinion(const UE::FUsdAttribute& Attribute)
{
#if USE_USD_SDK
	pxr::UsdAttribute PxrAttr{Attribute};

	if (!PxrAttr.HasAuthoredValue())
	{
		return false;
	}

	pxr::UsdResolveInfo Info = PxrAttr.GetResolveInfo(pxr::UsdTimeCode::Default());
	if (Info.GetSource() == pxr::UsdResolveInfoSourceDefault)
	{
		return true;
	}
#endif	  // #if USE_USD_SDK

	return false;
}

bool UsdUtils::HasAuthoredAnimatedOpinion(const UE::FUsdAttribute& Attribute)
{
	return Attribute.HasAuthoredValue() && Attribute.ValueMightBeTimeVarying();
}

#if USE_USD_SDK
bool UsdUtils::ClearAllTimeSamples(const pxr::UsdAttribute& Attribute)
{
	FScopedUsdAllocs Allocs;

	std::vector<double> Times;
	if (Attribute.GetTimeSamples(&Times))
	{
		for (double Time : Times)
		{
			Attribute.ClearAtTime(Time);
		}

		return true;
	}

	return false;
}

void UsdUtils::NotifyIfOverriddenOpinion(const pxr::UsdProperty& Property)
{
	FScopedUsdAllocs Allocs;

	if (!Property)
	{
		return;
	}

	pxr::UsdStageRefPtr Stage = Property.GetPrim().GetStage();
	if (!Stage)
	{
		return;
	}

	const pxr::UsdEditTarget& EditTarget = Stage->GetEditTarget();
	const pxr::SdfLayerHandle& Layer = EditTarget.GetLayer();
	if (!Layer)
	{
		return;
	}

	// Currently this will only warn in case our opinion and the strongest one both come from the local layer stack.
	// This is good enough for us at this point though, because we can't edit outside of the local layer stack anyway,
	// which due to LIVRPS is always stronger than the other composition arc types like references, payloads, etc.
	// Alternatively this also means that any other opinion on a non-local composition arc is never going to be strong
	// enough to override anything that we can author.
	// References:
	// - https://graphics.pixar.com/usd/release/glossary.html#livrps-strength-ordering
	// - https://groups.google.com/g/usd-interest/c/xTxFYQA_bRs/m/qbGkvx3yAgAJ
	std::vector<pxr::SdfPropertySpecHandle> SpecStack = Property.GetPropertyStack();
	for (const pxr::SdfPropertySpecHandle& Spec : SpecStack)
	{
		if (!Spec)
		{
			continue;
		}

		const pxr::SdfLayerHandle& SpecLayer = Spec->GetLayer();
		if (SpecLayer != Layer)
		{
			const FText Text = LOCTEXT("OverridenOpinionText", "USD: Overridden opinion");

			const FText SubText = FText::Format(
				LOCTEXT(
					"OverridenOpinionSubText",
					"Opinion authored for this attribute:\n\n{0}\n\nAt this layer:\n\n{1}\n\nIs overridden by another spec at this "
					"layer:\n\n{2}\n\nAnd so may not be visible on the composed stage. This means this edit may not be visible once the stage is "
					"reloaded."
				),
				FText::FromString(UsdToUnreal::ConvertPath(Spec->GetPath())),
				FText::FromString(UsdToUnreal::ConvertString(Layer->GetIdentifier())),
				FText::FromString(UsdToUnreal::ConvertString(SpecLayer->GetIdentifier()))
			);

			USD_LOG_USERWARNING(FText::FromString(SubText.ToString().Replace(TEXT("\n\n"), TEXT(" "))));

			const UUsdProjectSettings* Settings = GetDefault<UUsdProjectSettings>();
			if (Settings && Settings->bShowOverriddenOpinionsWarning)
			{
				static TWeakPtr<SNotificationItem> Notification;

				FNotificationInfo Toast(Text);
				Toast.SubText = SubText;
				Toast.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Warning"));
				Toast.CheckBoxText = LOCTEXT("DontAskAgain", "Don't prompt again");
				Toast.bUseLargeFont = false;
				Toast.bFireAndForget = false;
				Toast.FadeOutDuration = 0.0f;
				Toast.ExpireDuration = 0.0f;
				Toast.bUseThrobber = false;
				Toast.bUseSuccessFailIcons = false;
				Toast.ButtonDetails.Emplace(
					LOCTEXT("OverridenOpinionMessageOk", "Ok"),
					FText::GetEmpty(),
					FSimpleDelegate::CreateLambda(
						[]()
						{
							if (TSharedPtr<SNotificationItem> PinnedNotification = Notification.Pin())
							{
								PinnedNotification->SetCompletionState(SNotificationItem::CS_Success);
								PinnedNotification->ExpireAndFadeout();
							}
						}
					)
				);
				// This is flipped because the default checkbox message is "Don't prompt again"
				Toast.CheckBoxState = Settings->bShowOverriddenOpinionsWarning ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
				Toast.CheckBoxStateChanged = FOnCheckStateChanged::CreateStatic(
					[](ECheckBoxState NewState)
					{
						if (UUsdProjectSettings* Settings = GetMutableDefault<UUsdProjectSettings>())
						{
							// This is flipped because the default checkbox message is "Don't prompt again"
							Settings->bShowOverriddenOpinionsWarning = NewState == ECheckBoxState::Unchecked;
							Settings->SaveConfig();
						}
					}
				);

				// Only show one at a time
				if (!Notification.IsValid())
				{
					Notification = FSlateNotificationManager::Get().AddNotification(Toast);
				}

				if (TSharedPtr<SNotificationItem> PinnedNotification = Notification.Pin())
				{
					PinnedNotification->SetCompletionState(SNotificationItem::CS_Pending);
				}
			}
		}
		else
		{
			break;
		}
	}
}

bool UsdUtils::NotifyIfInstanceProxy(const pxr::UsdPrim& Prim)
{
	if (!Prim)
	{
		return false;
	}

	TFunction<void()> RemoveInstanceables = [Prim = UE::FUsdPrim{Prim}]()
	{
		if (!Prim)
		{
			return;
		}

		UE::FUsdStage Stage = Prim.GetStage();
		if (!Stage)
		{
			return;
		}

		// We have to track paths and not prims directly, because these will actually be the instance
		// proxies in case we have an instance parent, and even if the parents are not instances anymore
		// trying to author to these instance proxies would be an error. If we call GetPrimAtPath *after*
		// clearing the parent instanceable, then we get a regular prim
		TArray<UE::FSdfPath> LeafToRoot;

		UE::FUsdPrim Iter = Prim;
		while (Iter && !Iter.IsPseudoRoot())
		{
			LeafToRoot.Add(Iter.GetPrimPath());
			Iter = Iter.GetParent();
		}

		// Annoyingly we have to break from root downwards, as otherwise we'd be trying
		// to author inside instances ourselves!
		// Note: We also can't use a change block here, because we could have nested instanceables,
		// and we need USD to fully respond to the outer instanceable being cleared before it lets
		// us clear the inner one
		for (int32 Index = LeafToRoot.Num() - 1; Index >= 0; --Index)
		{
			const UE::FUsdPrim& SomePrim = Stage.GetPrimAtPath(LeafToRoot[Index]);
			if (SomePrim.IsInstanceable())
			{
				// We force false here instead of just clearing the authored opinion because the instanceable=true
				// opinion may come from a referenced layer or some other place we can't just clear on our current
				// edit target
				SomePrim.SetInstanceable(false);
			}
		}
	};

	if (Prim.IsInstanceProxy())
	{
		const UUsdProjectSettings* Settings = GetDefault<UUsdProjectSettings>();
		if (Settings)
		{
			switch (Settings->EditInInstanceableBehavior)
			{
				case EUsdEditInInstanceBehavior::Ignore:
				{
					USD_LOG_INFO(TEXT("Ignoring some edits to prim '%s' as it is an instance proxy"), *UsdToUnreal::ConvertPath(Prim.GetPrimPath()));
					return true;
				}
				case EUsdEditInInstanceBehavior::RemoveInstanceable:
				{
					USD_LOG_INFO(TEXT("Removing all instanceable flags from ancestors of prim '%s'"), *UsdToUnreal::ConvertPath(Prim.GetPrimPath()));
					RemoveInstanceables();

					// We shouldn't be instanceable now, so we can probably author whatever we wanted to author
					return false;
				}
				default:
				case EUsdEditInInstanceBehavior::ShowPrompt:
				{
					const FText Text = LOCTEXT("AuthoringInsideInstanceText", "USD: Authoring inside instance");

					FString FirstInstancePath;
					{
						FScopedUsdAllocs Allocs;
						pxr::UsdPrim Iter = Prim;
						while (Iter && !Iter.IsPseudoRoot())
						{
							if (Iter.IsInstance())
							{
								break;
							}
							Iter = Iter.GetParent();
						}
						FirstInstancePath = UsdToUnreal::ConvertPath(Iter.GetPrimPath());
					}

					const FText SubText = FText::Format(
						LOCTEXT(
							"AuthoringInsideInstanceSubText",
							"Trying to author an opinion below prim:\n\n{0}\n\nThis prim is an instance, so its child prim hierarchy cannot be modified directly.\n\nIf you wish to modify the hierarchy below just this particular instance, you can remove the instanceable flag from '{0}' and try again. If you wish to modify all instance hierarchies at the same time, please edit the prims referenced by prim '{0}' directly (i.e. open the referenced/payload layer directly, if any)."
						),
						FText::FromString(FirstInstancePath)
					);

					USD_LOG_USERWARNING(FText::FromString(SubText.ToString().Replace(TEXT("\n\n"), TEXT(" "))));

					static TWeakPtr<SNotificationItem> Notification;

					FNotificationInfo Toast(Text);
					Toast.SubText = SubText;
					Toast.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Warning"));
					Toast.CheckBoxText = LOCTEXT("DontAskAgain", "Don't prompt again");
					Toast.bUseLargeFont = false;
					Toast.bFireAndForget = false;
					Toast.FadeOutDuration = 0.0f;
					Toast.ExpireDuration = 0.0f;
					Toast.bUseThrobber = false;
					Toast.bUseSuccessFailIcons = false;
					Toast.ButtonDetails.Emplace(
						LOCTEXT("OverridenOpinionMessageRemove", "Remove instanceable flag"),
						FText::GetEmpty(),
						FSimpleDelegate::CreateLambda(
							[RemoveInstanceables]()
							{
								RemoveInstanceables();

								if (TSharedPtr<SNotificationItem> PinnedNotification = Notification.Pin())
								{
									PinnedNotification->SetCompletionState(SNotificationItem::CS_Success);
									PinnedNotification->ExpireAndFadeout();
								}

								if (UUsdProjectSettings* Settings = GetMutableDefault<UUsdProjectSettings>())
								{
									// We'll only set this to "ShowPrompt" if the checkbox to "Don't prompt again" is unchecked
									if (Settings->EditInInstanceableBehavior != EUsdEditInInstanceBehavior::ShowPrompt)
									{
										Settings->EditInInstanceableBehavior = EUsdEditInInstanceBehavior::RemoveInstanceable;
									}
									Settings->SaveConfig();
								}
							}
						)
					);
					Toast.ButtonDetails.Emplace(
						LOCTEXT("OverridenOpinionMessageCancel", "Cancel"),
						FText::GetEmpty(),
						FSimpleDelegate::CreateLambda(
							[]()
							{
								if (TSharedPtr<SNotificationItem> PinnedNotification = Notification.Pin())
								{
									PinnedNotification->SetCompletionState(SNotificationItem::CS_Success);
									PinnedNotification->ExpireAndFadeout();
								}
								if (UUsdProjectSettings* Settings = GetMutableDefault<UUsdProjectSettings>())
								{
									if (Settings->EditInInstanceableBehavior != EUsdEditInInstanceBehavior::ShowPrompt)
									{
										Settings->EditInInstanceableBehavior = EUsdEditInInstanceBehavior::Ignore;
									}
									Settings->SaveConfig();
								}
							}
						)
					);
					// This is flipped because the default checkbox message is "Don't prompt again"
					Toast.CheckBoxState = Settings->EditInInstanceableBehavior == EUsdEditInInstanceBehavior::ShowPrompt ? ECheckBoxState::Unchecked
																														 : ECheckBoxState::Checked;
					Toast.CheckBoxStateChanged = FOnCheckStateChanged::CreateStatic(
						[](ECheckBoxState NewState)
						{
							if (UUsdProjectSettings* Settings = GetMutableDefault<UUsdProjectSettings>())
							{
								// This is flipped because the default checkbox message is "Don't prompt again"
								Settings->EditInInstanceableBehavior = (NewState == ECheckBoxState::Unchecked)
																		   ? EUsdEditInInstanceBehavior::ShowPrompt
																		   : EUsdEditInInstanceBehavior::Ignore;	// Either would do here, we have
																													// to press one of the buttons to
																													// close the prompt, which will
																													// set the right one
								Settings->SaveConfig();
							}
						}
					);

					// Only show one at a time
					if (!Notification.IsValid())
					{
						Notification = FSlateNotificationManager::Get().AddNotification(Toast);
					}

					if (TSharedPtr<SNotificationItem> PinnedNotification = Notification.Pin())
					{
						PinnedNotification->SetCompletionState(SNotificationItem::CS_Pending);
					}

					return true;
				}
			}
		}
	}

	return false;
}

UE::FUsdAttribute UsdUtils::GetStrongestAttribute(const TArray<UE::FUsdAttribute>& Attributes, double TimeCode)
{
	if (Attributes.Num() == 0)
	{
		return {};
	}

	TMap<UE::FSdfLayer, TArray<UE::FUsdAttribute>> LayerToAttribute;
	LayerToAttribute.Reserve(Attributes.Num());

	UE::FUsdPrim CommonPrim = Attributes[0].GetPrim();

	// Collect the layers where we have prim stacks
	TArray<UE::FSdfPrimSpec> SpecStack = CommonPrim.GetPrimStack();
	TArray<UE::FSdfLayer> PrimStackLayers;
	PrimStackLayers.Reserve(SpecStack.Num());
	for (const UE::FSdfPrimSpec& Spec : SpecStack)
	{
		if (!Spec)
		{
			continue;
		}

		UE::FSdfLayer Layer = Spec.GetLayer();
		if (!Layer)
		{
			continue;
		}

		PrimStackLayers.Add(Layer);
		LayerToAttribute.Add(Layer);
	}

	// Collect all the layers where we have attribute specs, and in which layers those attributes have their strongest specs
	for (const UE::FUsdAttribute& Attribute : Attributes)
	{
		UE::FUsdPrim Prim = Attributes[0].GetPrim();
		if (Prim != CommonPrim)
		{
			USD_LOG_WARNING(
				TEXT("Found different prims '%s' and '%s' when trying to collect strongest attributes"),
				*Prim.GetPrimPath().GetString(),
				*CommonPrim.GetPrimPath().GetString()
			);
			return {};
		}

		FScopedUsdAllocs UsdAllocs;

		TArray<UE::FSdfLayer> AttributeStackLayers;
		std::vector<pxr::SdfPropertySpecHandle> PropertyStack = pxr::UsdAttribute{Attribute}.GetPropertyStack(TimeCode);
		for (const pxr::SdfPropertySpecHandle& PropertySpec : PropertyStack)
		{
			if (!PropertySpec)
			{
				continue;
			}

			if (!PropertySpec->HasDefaultValue() && PropertySpec->GetLayer()->GetNumTimeSamplesForPath(PropertySpec->GetPath()) == 0)
			{
				continue;
			}

			pxr::SdfLayerHandle SpecLayer = PropertySpec->GetLayer();
			if (!SpecLayer)
			{
				continue;
			}

			AttributeStackLayers.Add(UE::FSdfLayer{SpecLayer});
		}
		if (AttributeStackLayers.Num() == 0)
		{
			continue;
		}

		if (TArray<UE::FUsdAttribute>* FoundAttributes = LayerToAttribute.Find(AttributeStackLayers[0]))
		{
			FoundAttributes->Add(Attribute);
		}
		else
		{
			// In the general case if we have an attribute spec for something, we'll have a prim spec for its prim in the
			// same layer. For value clips this seems to not be the case: We can have an attribute spec without an owning
			// prim spec in the same layer somehow...
			//
			// Here we try to work around that by merging the layers we know about: If we find an attribute spec layer that we
			// don't know about, we pretend we have a corresponding prim spec with the same strength
			for (int32 AttributeIndex = 0, PrimIndex = 0; AttributeIndex < AttributeStackLayers.Num(); ++AttributeIndex, ++PrimIndex)
			{
				if (!ensure(PrimStackLayers.IsValidIndex(PrimIndex)))
				{
					// Shouldn't ever happen but it's hard to predict all USD scenarios
					break;
				}

				const UE::FSdfLayer& AttributeLayer = AttributeStackLayers[AttributeIndex];
				const UE::FSdfLayer& PrimLayer = PrimStackLayers[PrimIndex];

				if (LayerToAttribute.Contains(AttributeLayer))
				{
					// We already have this layer in the prim stack, resync our indices here so we continue traversing
					// prim and layer stacks in tandem until the next different layer
					if (PrimLayer != AttributeLayer)
					{
						AttributeIndex = PrimStackLayers.IndexOfByKey(PrimLayer);
					}
					continue;
				}
				else
				{
					PrimStackLayers.Insert(AttributeLayer, PrimIndex);
					LayerToAttribute.FindOrAdd(AttributeLayer).Add(Attribute);

					// We only really care about the strongest spec anyway
					break;
				}
			}
		}
	}

	// Early out here in case all our attributes come from the same layer anyway
	if (LayerToAttribute.Num() == 1)
	{
		const TArray<UE::FUsdAttribute>& FoundAttrs = (LayerToAttribute.CreateIterator())->Value;
		if (FoundAttrs.Num() > 0)
		{
			// Defer to the order the attributes were in the Attributes array in case of ties
			return FoundAttrs[0];
		}

		return {};
	}

	// Return the attribute that has a spec on our strongest prim spec layer
	for (const UE::FSdfLayer& Layer : PrimStackLayers)
	{
		TArray<UE::FUsdAttribute>* FoundAttribute = LayerToAttribute.Find(Layer);
		if (FoundAttribute && FoundAttribute->Num() > 0)
		{
			return (*FoundAttribute)[0];
		}
	}

	return {};
}

#endif	  // #if USE_USD_SDK

#undef LOCTEXT_NAMESPACE
