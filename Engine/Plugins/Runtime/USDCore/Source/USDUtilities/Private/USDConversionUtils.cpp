// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDConversionUtils.h"

#include "USDAssetImportData.h"
#include "USDAssetUserData.h"
#include "USDClassesModule.h"
#include "USDDrawModeComponent.h"
#include "USDDuplicateType.h"
#include "USDErrorUtils.h"
#include "USDGeomMeshConversion.h"
#include "USDIntegrationUtils.h"
#include "USDLayerUtils.h"
#include "USDMemory.h"
#include "USDObjectUtils.h"
#include "USDProjectSettings.h"
#include "USDShadeConversion.h"
#include "USDSkeletalDataConversion.h"
#include "USDTypesConversion.h"
#include "USDUnrealAssetInfo.h"
#include "USDUtilitiesModule.h"
#include "USDValueConversion.h"
#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdStage.h"
#include "UsdWrappers/VtValue.h"

#include "AnalyticsEventAttribute.h"
#include "Animation/AnimBlueprint.h"
#include "Animation/AnimSequence.h"
#include "Animation/Skeleton.h"
#include "CineCameraActor.h"
#include "CineCameraComponent.h"
#include "Components/AudioComponent.h"
#include "Components/DirectionalLightComponent.h"
#include "Components/HeterogeneousVolumeComponent.h"
#include "Components/PointLightComponent.h"
#include "Components/RectLightComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/SkyLightComponent.h"
#include "Components/SpotLightComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/DirectionalLight.h"
#include "Engine/PointLight.h"
#include "Engine/RectLight.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkyLight.h"
#include "Engine/SpotLight.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GeometryCache.h"
#include "GroomAsset.h"
#include "GroomCache.h"
#include "Hash/Fnv.h"
#include "InstancedFoliageActor.h"
#include "LandscapeProxy.h"
#include "Materials/Material.h"
#include "Misc/PackageName.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "Sound/AmbientSound.h"
#include "SparseVolumeTexture/SparseVolumeTexture.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Text/STextBlock.h"
#if WITH_EDITOR
#include "Editor.h"
#endif	  // WITH_EDITOR

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/base/tf/stringUtils.h"
#include "pxr/base/tf/token.h"
#include "pxr/base/tf/unicodeUtils.h"
#include "pxr/usd/kind/registry.h"
#include "pxr/usd/sdf/copyUtils.h"
#include "pxr/usd/sdf/schema.h"
#include "pxr/usd/usd/attribute.h"
#include "pxr/usd/usd/editContext.h"
#include "pxr/usd/usd/modelAPI.h"
#include "pxr/usd/usd/payloads.h"
#include "pxr/usd/usd/primCompositionQuery.h"
#include "pxr/usd/usd/primRange.h"
#include "pxr/usd/usd/stage.h"
#include "pxr/usd/usd/stageCache.h"
#include "pxr/usd/usd/variantSets.h"
#include "pxr/usd/usdGeom/camera.h"
#include "pxr/usd/usdGeom/gprim.h"
#include "pxr/usd/usdGeom/imageable.h"
#include "pxr/usd/usdGeom/mesh.h"
#include "pxr/usd/usdGeom/metrics.h"
#include "pxr/usd/usdGeom/modelAPI.h"
#include "pxr/usd/usdGeom/primvar.h"
#include "pxr/usd/usdGeom/primvarsAPI.h"
#include "pxr/usd/usdLux/diskLight.h"
#include "pxr/usd/usdLux/distantLight.h"
#include "pxr/usd/usdLux/domeLight.h"
#include "pxr/usd/usdLux/domeLight_1.h"
#include "pxr/usd/usdLux/rectLight.h"
#include "pxr/usd/usdLux/shapingAPI.h"
#include "pxr/usd/usdLux/sphereLight.h"
#include "pxr/usd/usdMedia/spatialAudio.h"
#include "pxr/usd/usdSkel/animation.h"
#include "pxr/usd/usdSkel/cache.h"
#include "pxr/usd/usdSkel/root.h"
#include "pxr/usd/usdSkel/skeletonQuery.h"
#include "pxr/usd/usdUtils/stageCache.h"
#include "pxr/usd/usdVol/openVDBAsset.h"
#include "pxr/usd/usdVol/volume.h"
#include "USDIncludesEnd.h"

#include <string>

#define LOCTEXT_NAMESPACE "USDConversionUtils"

static bool GParseUVSetsFromFloat2Primvars = true;
static FAutoConsoleVariableRef CVarParseUVSetsFromFloat2Primvars(
	TEXT("USD.ParseUVSetsFromFloat2Primvars"),
	GParseUVSetsFromFloat2Primvars,
	TEXT("Primvars with the 'texCoord2f' role will always be parsed when handling potential UV sets. If this cvar is enabled, we'll also handle "
		 "primvars declared as just 'float2' however. You could disable this cvar if your pipeline emits many 'float2' primvars that you do not wish "
		 "to be parsed as UV sets.")
);

static bool GCheapUniquePrimPathGeneration = false;
static FAutoConsoleVariableRef CVarCheapUniquePrimPathGeneration(
	TEXT("USD.CheapUniquePrimPathGeneration"),
	GCheapUniquePrimPathGeneration,
	TEXT(
		"When exporting Levels and LevelSequences, we'll by default use a mechanism of ensuring unique prim paths that guarantees a unique path for each UObject. It can be somewhat expensive depending on the use-case, so if you have other ways of ensuring actor labels are unique, you can set this to true to use another method of producing unique prim paths that is much faster, but can't handle some kinds of actor label collisions."
	)
);

static bool bRemoveDuplicates = true;
static FAutoConsoleVariableRef CVarRemoveDuplicates(
	TEXT("USD.Volume.RemoveDuplicateAnimatedFrames"),
	bRemoveDuplicates,
	TEXT(
		"If this is true (default), the contents of a .VDB file are added only once to animated Sparse Volume Textures (SVT), even if the same file shows in multiple different time samples. If this is false, every OpenVDBAsset prim filePath time sample is parsed as a new frame on the animated SVT."
	)
);

static bool GInstancingAwareTranslation = true;
static FAutoConsoleVariableRef CVarInstancingAwareTranslation(
	TEXT("USD.InstancingAwareTranslation"),
	GInstancingAwareTranslation,
	TEXT(
		"Enabling this lets the USDImporter skip some extra steps during translation when it encounters multiple instance prims of the same (static) Mesh prototype prim."
	)
);

static bool GCollapsePrimsWithoutKind = true;
static FAutoConsoleVariableRef CVarCollapsePrimsWithoutKind(
	TEXT("USD.CollapsePrimsWithoutKind"),
	GCollapsePrimsWithoutKind,
	TEXT(
		"If we're collapsing according to kind (i.e. when UsePrimKindsForCollapsing is enabled), this controls how prims should behave if they don't have any authored 'Kind' value. Set this to false, and a prim without kind can't be collapsed by its parent, even if the parent has a collapsible kind."
	)
);

namespace USDConversionUtilsImpl
{
	/** Show some warnings if the UVSet primvars show some unsupported/problematic behavior */
	void CheckUVSetPrimvars(
		TMap<int32, TArray<pxr::UsdGeomPrimvar>> UsablePrimvars,
		TMap<int32, TArray<pxr::UsdGeomPrimvar>> UsedPrimvars,
		const FString& MeshPath
	)
	{
		// Show a warning if the mesh has a primvar that could be used as a UV set but will actually be ignored because it targets a UV set with index
		// larger than MAX_STATIC_TEXCOORDS - 1
		TArray<FString> IgnoredPrimvarNames;
		for (const TPair<int32, TArray<pxr::UsdGeomPrimvar>>& UsedPrimvar : UsedPrimvars)
		{
			if (UsedPrimvar.Key > MAX_STATIC_TEXCOORDS - 1)
			{
				for (const pxr::UsdGeomPrimvar& Primvar : UsedPrimvar.Value)
				{
					IgnoredPrimvarNames.AddUnique(UsdToUnreal::ConvertToken(Primvar.GetBaseName()));
				}
			}
		}
		for (const TPair<int32, TArray<pxr::UsdGeomPrimvar>>& UsablePrimvar : UsablePrimvars)
		{
			if (UsablePrimvar.Key > MAX_STATIC_TEXCOORDS - 1)
			{
				for (const pxr::UsdGeomPrimvar& Primvar : UsablePrimvar.Value)
				{
					// Only consider texcoord2f here because the user may have some other float2[] for some other reason
					if (Primvar.GetTypeName().GetRole() == pxr::SdfValueTypeNames->TexCoord2f.GetRole())
					{
						IgnoredPrimvarNames.AddUnique(UsdToUnreal::ConvertToken(Primvar.GetBaseName()));
					}
				}
			}
		}
		if (IgnoredPrimvarNames.Num() > 0)
		{
			FString PrimvarNames = FString::Join(IgnoredPrimvarNames, TEXT(", "));
			USD_LOG_USERWARNING(FText::Format(
				LOCTEXT(
					"TooHighUVIndex",
					"Mesh '{0}' has some valid UV set primvars ({1}) that will be ignored because they target an UV index larger than the "
					"highest supported ({2})"
				),
				FText::FromString(MeshPath),
				FText::FromString(PrimvarNames),
				MAX_STATIC_TEXCOORDS - 1
			));
		}

		// Show a warning if the mesh does not contain the exact primvars the material wants
		for (const TPair<int32, TArray<pxr::UsdGeomPrimvar>>& UVAndPrimvars : UsedPrimvars)
		{
			const int32 UVIndex = UVAndPrimvars.Key;
			const TArray<pxr::UsdGeomPrimvar>& UsedPrimvarsForIndex = UVAndPrimvars.Value;
			if (UsedPrimvarsForIndex.Num() < 1)
			{
				continue;
			}

			// If we have multiple, we'll pick the first one and show a warning about this later
			const pxr::UsdGeomPrimvar& UsedPrimvar = UsedPrimvarsForIndex[0];

			bool bFoundUsablePrimvar = false;
			if (const TArray<pxr::UsdGeomPrimvar>* FoundUsablePrimvars = UsablePrimvars.Find(UVIndex))
			{
				// We will only ever use the first one, but will show more warnings in case there are multiple
				if (FoundUsablePrimvars->Contains(UsedPrimvar))
				{
					bFoundUsablePrimvar = true;
				}
			}

			if (!bFoundUsablePrimvar)
			{
				USD_LOG_USERWARNING(FText::Format(
					LOCTEXT("DidNotFindPrimvar", "Could not find primvar '{0}' on mesh '{1}', used by its bound material"),
					FText::FromString(UsdToUnreal::ConvertString(UsedPrimvar.GetBaseName())),
					FText::FromString(MeshPath)
				));
			}
		}

		// Show a warning if the mesh has multiple primvars that want to write to the same UV set (e.g. 'st', 'st_0' and 'st0' at the same time)
		for (const TPair<int32, TArray<pxr::UsdGeomPrimvar>>& UVAndPrimvars : UsablePrimvars)
		{
			const int32 UVIndex = UVAndPrimvars.Key;
			const TArray<pxr::UsdGeomPrimvar>& Primvars = UVAndPrimvars.Value;
			if (Primvars.Num() > 1)
			{
				// Find out what primvar we'll actually end up using, as UsedPrimvars will take precedence. Note that in the best case scenario,
				// UsablePrimvars will *contain* UsedPrimvars, so that really we're just picking which of the UsedPrimvars we'll choose. If we're not
				// in that scenario, then we will show another warning about it
				const pxr::UsdGeomPrimvar* UsedPrimvar = nullptr;
				bool bUsedByMaterial = false;
				if (const TArray<pxr::UsdGeomPrimvar>* FoundUsedPrimvars = UsedPrimvars.Find(UVIndex))
				{
					if (FoundUsedPrimvars->Num() > 0)
					{
						UsedPrimvar = &(*FoundUsedPrimvars)[0];
						bUsedByMaterial = true;
					}
				}
				else
				{
					UsedPrimvar = &Primvars[0];
				}

				USD_LOG_USERWARNING(FText::Format(
					LOCTEXT(
						"MoreThanOnePrimvarForIndex",
						"Mesh '{0}' has more than one primvar used as UV set with index '{1}'. The UV set will use the values from primvar "
						"'{2}'{3}"
					),
					FText::FromString(MeshPath),
					UVAndPrimvars.Key,
					FText::FromString(UsdToUnreal::ConvertString(UsedPrimvar->GetBaseName())),
					bUsedByMaterial ? FText::FromString(TEXT(", as its used by its bound material")) : FText::GetEmpty()
				));
			}
		}
	}

	// Shows a notification saying that some specs of the provided prims won't be duplicated due to being on external
	// layers
	void NotifySpecsWontBeDuplicated(const TArray<UE::FUsdPrim>& Prims)
	{
		if (Prims.Num() == 0)
		{
			return;
		}

		const FText Text = LOCTEXT("IncompleteDuplicationText", "USD: Incomplete duplication");

		FString PrimNamesString;
		const static FString Delimiter = TEXT(", ");
		for (const UE::FUsdPrim& Prim : Prims)
		{
			PrimNamesString += Prim.GetName().ToString() + Delimiter;
		}
		PrimNamesString.RemoveFromEnd(Delimiter);

		const int32 NumPrims = Prims.Num();

		const FText SubText = FText::Format(
			LOCTEXT(
				"IncompleteDuplicationSubText",
				"{0}|plural(one=This,other=These) duplicated {0}|plural(one=prim,other=prims):\n\n{1}\n\n{0}|plural(one=Has,other=Have) some specs "
				"within layers that are outside of the stage's local layer stack, and so will not be duplicated.\n\nIf you wish to modify referenced "
				"or payload layers, please open those layers as USD stages directly."
			),
			NumPrims,
			FText::FromString(PrimNamesString)
		);

		USD_LOG_USERWARNING(FText::FromString(SubText.ToString().Replace(TEXT("\n\n"), TEXT(" "))));

		const UUsdProjectSettings* Settings = GetDefault<UUsdProjectSettings>();
		if (Settings && Settings->bShowWarningOnIncompleteDuplication)
		{
			static TWeakPtr<SNotificationItem> Notification;

			FNotificationInfo Toast(Text);
			Toast.SubText = SubText;
			Toast.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Warning"));
			Toast.CheckBoxText = LOCTEXT("DontAskAgain", "Don't prompt again");
			Toast.bUseLargeFont = false;
			Toast.bFireAndForget = false;
			Toast.FadeOutDuration = 0.0f;
			Toast.ExpireDuration = 0.0f;
			Toast.bUseThrobber = false;
			Toast.bUseSuccessFailIcons = false;
			Toast.ButtonDetails.Emplace(
				LOCTEXT("OverridenOpinionMessageOk", "Ok"),
				FText::GetEmpty(),
				FSimpleDelegate::CreateLambda(
					[]()
					{
						if (TSharedPtr<SNotificationItem> PinnedNotification = Notification.Pin())
						{
							PinnedNotification->SetCompletionState(SNotificationItem::CS_Success);
							PinnedNotification->ExpireAndFadeout();
						}
					}
				)
			);
			// This is flipped because the default checkbox message is "Don't prompt again"
			Toast.CheckBoxState = Settings->bShowWarningOnIncompleteDuplication ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
			Toast.CheckBoxStateChanged = FOnCheckStateChanged::CreateStatic(
				[](ECheckBoxState NewState)
				{
					if (UUsdProjectSettings* Settings = GetMutableDefault<UUsdProjectSettings>())
					{
						// This is flipped because the default checkbox message is "Don't prompt again"
						Settings->bShowWarningOnIncompleteDuplication = NewState == ECheckBoxState::Unchecked;
						Settings->SaveConfig();
					}
				}
			);

			// Only show one at a time
			if (!Notification.IsValid())
			{
				Notification = FSlateNotificationManager::Get().AddNotification(Toast);
			}

			if (TSharedPtr<SNotificationItem> PinnedNotification = Notification.Pin())
			{
				PinnedNotification->SetCompletionState(SNotificationItem::CS_Pending);
			}
		}
	}
}	 // namespace USDConversionUtilsImpl

template<typename ValueType>
ValueType UsdUtils::GetUsdValue(const pxr::UsdAttribute& Attribute, pxr::UsdTimeCode TimeCode)
{
	ValueType Value{};
	if (Attribute)
	{
		Attribute.Get(&Value, TimeCode);
	}

	return Value;
}

// Explicit template instantiation
template USDUTILITIES_API bool UsdUtils::GetUsdValue<bool>(const pxr::UsdAttribute& Attribute, pxr::UsdTimeCode TimeCode);
template USDUTILITIES_API float UsdUtils::GetUsdValue<float>(const pxr::UsdAttribute& Attribute, pxr::UsdTimeCode TimeCode);
template USDUTILITIES_API double UsdUtils::GetUsdValue<double>(const pxr::UsdAttribute& Attribute, pxr::UsdTimeCode TimeCode);
template USDUTILITIES_API pxr::TfToken UsdUtils::GetUsdValue<pxr::TfToken>(const pxr::UsdAttribute& Attribute, pxr::UsdTimeCode TimeCode);
template USDUTILITIES_API pxr::GfVec2f UsdUtils::GetUsdValue<pxr::GfVec2f>(const pxr::UsdAttribute& Attribute, pxr::UsdTimeCode TimeCode);
template USDUTILITIES_API pxr::GfVec3f UsdUtils::GetUsdValue<pxr::GfVec3f>(const pxr::UsdAttribute& Attribute, pxr::UsdTimeCode TimeCode);
template USDUTILITIES_API pxr::GfMatrix4d UsdUtils::GetUsdValue<pxr::GfMatrix4d>(const pxr::UsdAttribute& Attribute, pxr::UsdTimeCode TimeCode);
template USDUTILITIES_API pxr::SdfAssetPath UsdUtils::GetUsdValue<pxr::SdfAssetPath>(const pxr::UsdAttribute& Attribute, pxr::UsdTimeCode TimeCode);
template USDUTILITIES_API pxr::VtArray<pxr::GfVec3f> UsdUtils::GetUsdValue<pxr::VtArray<pxr::GfVec3f>>(
	const pxr::UsdAttribute& Attribute,
	pxr::UsdTimeCode TimeCode
);
template USDUTILITIES_API pxr::VtArray<float> UsdUtils::GetUsdValue<pxr::VtArray<float>>(
	const pxr::UsdAttribute& Attribute,
	pxr::UsdTimeCode TimeCode
);
template USDUTILITIES_API pxr::VtArray<int> UsdUtils::GetUsdValue<pxr::VtArray<int>>(const pxr::UsdAttribute& Attribute, pxr::UsdTimeCode TimeCode);

pxr::TfToken UsdUtils::GetUsdStageUpAxis(const pxr::UsdStageRefPtr& Stage)
{
	return pxr::UsdGeomGetStageUpAxis(Stage);
}

EUsdUpAxis UsdUtils::GetUsdStageUpAxisAsEnum(const pxr::UsdStageRefPtr& Stage)
{
	pxr::TfToken UpAxisToken = pxr::UsdGeomGetStageUpAxis(Stage);
	return UpAxisToken == pxr::UsdGeomTokens->z ? EUsdUpAxis::ZAxis : EUsdUpAxis::YAxis;
}

void UsdUtils::SetUsdStageUpAxis(const pxr::UsdStageRefPtr& Stage, pxr::TfToken Axis)
{
	pxr::UsdGeomSetStageUpAxis(Stage, Axis);
}

void UsdUtils::SetUsdStageUpAxis(const pxr::UsdStageRefPtr& Stage, EUsdUpAxis Axis)
{
	pxr::TfToken UpAxisToken = Axis == EUsdUpAxis::ZAxis ? pxr::UsdGeomTokens->z : pxr::UsdGeomTokens->y;
	SetUsdStageUpAxis(Stage, UpAxisToken);
}

double UsdUtils::GetUsdStageMetersPerUnit(const pxr::UsdStageRefPtr& Stage)
{
	return pxr::UsdGeomGetStageMetersPerUnit(Stage);
}

void UsdUtils::SetUsdStageMetersPerUnit(const pxr::UsdStageRefPtr& Stage, double MetersPerUnit)
{
	if (!Stage || !Stage->GetRootLayer())
	{
		return;
	}

	pxr::UsdEditContext Context(Stage, Stage->GetRootLayer());
	pxr::UsdGeomSetStageMetersPerUnit(Stage, MetersPerUnit);
}

int32 UsdUtils::GetUsdStageNumFrames(const pxr::UsdStageRefPtr& Stage)
{
	// USD time code range is inclusive on both ends
	return Stage ? FMath::Abs(FMath::CeilToInt32(Stage->GetEndTimeCode()) - FMath::FloorToInt32(Stage->GetStartTimeCode()) + 1) : 0;
}

int64 UsdUtils::GetUsdUtilsStageCacheStageId(const pxr::UsdStageRefPtr& Stage)
{
	pxr::UsdStageCache& StageCache = pxr::UsdUtilsStageCache::Get();
	pxr::UsdStageCache::Id Id = StageCache.GetId(Stage);
	return Id.IsValid() ? Id.ToLongInt() : INDEX_NONE;
}

UE::FUsdStage UsdUtils::FindUsdUtilsStageCacheStageId(int64 Id)
{
	pxr::UsdStageCache& StageCache = pxr::UsdUtilsStageCache::Get();
	pxr::UsdStageRefPtr Stage = StageCache.Find(pxr::UsdStageCache::Id::FromLongInt(Id));
	return UE::FUsdStage{Stage};
}

int64 UsdUtils::InsertStageIntoUsdUtilsStageCache(const pxr::UsdStageRefPtr& Stage)
{
	pxr::UsdStageCache& StageCache = pxr::UsdUtilsStageCache::Get();
	return StageCache.Insert(Stage).ToLongInt();
}

bool UsdUtils::RemoveStageFromUsdUtilsStageCache(int64 StageId)
{
	pxr::UsdStageCache& StageCache = pxr::UsdUtilsStageCache::Get();
	return StageCache.Erase(pxr::UsdStageCache::Id::FromLongInt(StageId));
}

bool UsdUtils::HasCompositionArcs(const pxr::UsdPrim& Prim)
{
	if (!Prim || !Prim.IsActive())
	{
		return false;
	}

	return Prim.HasAuthoredReferences() || Prim.HasAuthoredPayloads() || Prim.HasAuthoredInherits() || Prim.HasAuthoredSpecializes()
		   || Prim.HasVariantSets();
}

bool UsdUtils::HasCompositionArcs(const pxr::SdfPrimSpecHandle& PrimSpec)
{
	if (!PrimSpec || !PrimSpec->GetActive())
	{
		return false;
	}

	return PrimSpec->HasReferences() || PrimSpec->HasPayloads() || PrimSpec->HasInheritPaths() || PrimSpec->HasSpecializes()
		   || PrimSpec->HasVariantSetNames();
}

UClass* UsdUtils::GetActorTypeForPrim(const pxr::UsdPrim& Prim)
{
	// If we have this attribute and a valid child camera prim then we'll assume
	// we correspond to the root scene component of an exported cine camera actor. Let's assume
	// then that we have an actual ACineCameraActor class so that the schema translators can
	// reuse the main UCineCameraComponent for the actual child camera prim
	bool bIsCineCameraActorRootComponent = false;
	if (pxr::UsdAttribute Attr = Prim.GetAttribute(UnrealToUsd::ConvertToken(TEXT("unrealCameraPrimName")).Get()))
	{
		pxr::TfToken CameraComponentPrim;
		if (Attr.Get<pxr::TfToken>(&CameraComponentPrim))
		{
			pxr::UsdPrim ChildCameraPrim = Prim.GetChild(CameraComponentPrim);
			if (ChildCameraPrim && ChildCameraPrim.IsA<pxr::UsdGeomCamera>())
			{
				bIsCineCameraActorRootComponent = true;
			}
		}
	}

	if (Prim.IsA<pxr::UsdGeomCamera>() || bIsCineCameraActorRootComponent)
	{
		return ACineCameraActor::StaticClass();
	}
	else if (Prim.IsA<pxr::UsdLuxDistantLight>())
	{
		return ADirectionalLight::StaticClass();
	}
	else if (Prim.IsA<pxr::UsdLuxRectLight>() || Prim.IsA<pxr::UsdLuxDiskLight>())
	{
		return ARectLight::StaticClass();
	}
	else if (Prim.IsA<pxr::UsdLuxSphereLight>())
	{
		if (Prim.HasAPI<pxr::UsdLuxShapingAPI>())
		{
			return ASpotLight::StaticClass();
		}
		else
		{
			return APointLight::StaticClass();
		}
	}
	else if (Prim.IsA<pxr::UsdLuxDomeLight>() || Prim.IsA<pxr::UsdLuxDomeLight_1>())
	{
		return ASkyLight::StaticClass();
	}
	else if (Prim.IsA<pxr::UsdVolVolume>())
	{
		return AHeterogeneousVolume::StaticClass();
	}
	else if (Prim.IsA<pxr::UsdMediaSpatialAudio>())
	{
		return AAmbientSound::StaticClass();
	}
	else
	{
		return AActor::StaticClass();
	}
}

UClass* UsdUtils::GetComponentTypeForPrim(const pxr::UsdPrim& Prim)
{
	if (Prim.IsA<pxr::UsdSkelSkeleton>())
	{
		return USkeletalMeshComponent::StaticClass();
	}
	else if (Prim.IsA<pxr::UsdGeomGprim>())
	{
		return UStaticMeshComponent::StaticClass();
	}
	else if (Prim.IsA<pxr::UsdGeomCamera>())
	{
		return UCineCameraComponent::StaticClass();
	}
	else if (Prim.IsA<pxr::UsdLuxDistantLight>())
	{
		return UDirectionalLightComponent::StaticClass();
	}
	else if (Prim.IsA<pxr::UsdLuxRectLight>() || Prim.IsA<pxr::UsdLuxDiskLight>())
	{
		return URectLightComponent::StaticClass();
	}
	else if (Prim.IsA<pxr::UsdLuxSphereLight>())
	{
		if (Prim.HasAPI<pxr::UsdLuxShapingAPI>())
		{
			return USpotLightComponent::StaticClass();
		}
		else
		{
			return UPointLightComponent::StaticClass();
		}
	}
	else if (Prim.IsA<pxr::UsdLuxDomeLight>() || Prim.IsA<pxr::UsdLuxDomeLight_1>())
	{
		return USkyLightComponent::StaticClass();
	}
	else if (Prim.IsA<pxr::UsdGeomXformable>())
	{
		return USceneComponent::StaticClass();
	}
	else
	{
		return nullptr;
	}
}

FString UsdUtils::GetSchemaNameForComponent(const USceneComponent& Component)
{
	AActor* OwnerActor = Component.GetOwner();
	if (OwnerActor->IsA<AInstancedFoliageActor>())
	{
		return TEXT("PointInstancer");
	}
	else if (OwnerActor->IsA<ALandscapeProxy>())
	{
		return TEXT("Mesh");
	}

	if (Component.IsA<USkinnedMeshComponent>())
	{
		return TEXT("SkelRoot");
	}
	else if (Component.IsA<UInstancedStaticMeshComponent>())
	{
		// The original ISM component becomes just a regular Xform prim, so that we can handle
		// its children correctly. We'll manually create a new child PointInstancer prim to it
		// however, and convert the ISM data onto that prim.
		return TEXT("Xform");
	}
	else if (const UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(&Component))
	{
		UStaticMesh* Mesh = StaticMeshComponent->GetStaticMesh();
		if (Mesh && Mesh->GetNumLODs() > 1)
		{
			// Don't export 'Mesh' if we're going to export LODs, as those will also be Mesh prims.
			// We need at least an Xform schema though as this component may still have a transform of its own
			return TEXT("Xform");
		}
		return TEXT("Mesh");
	}
	else if (Component.IsA<UCineCameraComponent>())
	{
		return TEXT("Camera");
	}
	else if (Component.IsA<UDirectionalLightComponent>())
	{
		return TEXT("DistantLight");
	}
	else if (Component.IsA<URectLightComponent>())
	{
		return TEXT("RectLight");
	}
	else if (Component.IsA<UPointLightComponent>())
	{
		return TEXT("SphereLight");
	}
	else if (Component.IsA<USkyLightComponent>())
	{
		return TEXT("DomeLight_1");
	}
	else if (Component.IsA<UAudioComponent>())
	{
		return TEXT("SpatialAudio");
	}

	return TEXT("Xform");
}

FString UsdUtils::GetPrimPathForObject(
	const UObject* ActorOrComponent,
	const FString& ParentPrimPath,
	bool bUseActorFolders,
	const FString& RootPrimName
)
{
	if (!ActorOrComponent)
	{
		return {};
	}

	// Get component and its owner actor
	const USceneComponent* Component = Cast<const USceneComponent>(ActorOrComponent);
	const AActor* Owner = nullptr;
	if (Component)
	{
		Owner = Component->GetOwner();
	}
	else
	{
		Owner = Cast<AActor>(ActorOrComponent);
		if (Owner)
		{
			Component = Owner->GetRootComponent();
		}
	}
	if (!Component || !Owner)
	{
		return {};
	}

	// Get component name. Use actor label if the component is its root component
	FString Path;
#if WITH_EDITOR
	if (Component == Owner->GetRootComponent())
	{
		UObject* OwnerOuter = Owner->GetOuter();
		FString OwnerLabel = Owner->GetActorLabel();

		if (GCheapUniquePrimPathGeneration)
		{
			// This guarantees uniqueness only if all actors that have the same label also have the
			// same FName text part (i.e. If we had a Directional Light named "Foo" and a StaticMeshActor
			// named "Foo", their IDNames could end up being "DirectionalLight_2" and "StaticMeshActor_2",
			// so this method would have generated the prim name "Foo_2" for both...)
			Path = Owner->GetActorLabel() + TEXT("_") + LexToString(Owner->GetFName().GetNumber());
		}
		else
		{
			TArray<UObject*> SiblingActors;
			GetObjectsWithOuter(OwnerOuter, SiblingActors, EGetObjectsFlags::None);

			TSet<FString> SeenLabels;
			SeenLabels.Reserve(SiblingActors.Num());

			TArray<UObject*> SiblingsWithSameLabel;
			for (UObject* Sibling : SiblingActors)
			{
				if (AActor* ActorSibling = Cast<AActor>(Sibling))
				{
					FString SiblingLabel = ActorSibling->GetActorLabel();
					SeenLabels.Add(SiblingLabel);

					if (ActorSibling == Owner || SiblingLabel == OwnerLabel)
					{
						SiblingsWithSameLabel.Add(Sibling);
					}
				}
			}

			// Sorting is important because we'll call this from e.g. LevelSequence export, and it should
			// match the unique names that were generated on the Level export too
			SiblingsWithSameLabel.Sort(
				[](const UObject& LHS, const UObject& RHS)
				{
					return LHS.GetName() < RHS.GetName();
				}
			);

			int32 Index = SiblingsWithSameLabel.IndexOfByKey(Owner);
			if (Index == 0)
			{
				Path = Owner->GetActorLabel();
			}
			else
			{
				Path = Owner->GetActorLabel();

				// Imagine we have the sibling actors with labels "Cube", "Cube", "Cube" and "Cube_0". In here suppose
				// we're trying to come up with a prim name for the second of the "Cube"s. Normally we'd come up with "Cube_0",
				// but there's already an actor with this label, so we can't use it. Unfortunately though, we can't
				// just increment our index and use "Cube_1" either: When we sanitize the third "Cube" we'd end up also
				// trying to name it "Cube_1" (remember, we don't keep any "state" between calls to this function)
				//
				// We also don't want to just add the "Cube_0" actor to the same list of name collisions and handle it in
				// the same "group" as other "Cube"s because we want to preserve the user-set label if possible
				// (i.e. we don't want one of the "Cube"s to end up exported with the previously existing name "Cube_0").
				//
				// The solution used here is to not increment our index, but to make sure we always add *a new* trailing
				// suffix with it. That way, the labels for the actors in the example will end up being, respectively:
				// "Cube", "Cube_0_0", "Cube_1", "Cube_0". It looks a bit goofy, but we don't need to preserve any state
				// or global "used prim names" set anywhere, and it preserves "Cube_0" and even a "Cube" label
				do
				{
					Path += TEXT("_") + LexToString(Index - 1);
				} while (SeenLabels.Contains(Path));
			}
		}
	}
	else
#endif	  // WITH_EDITOR
	{
		Path = Component->GetName();
	}
	Path = UsdUtils::SanitizeUsdIdentifier(*Path);

	// Get a clean folder path string if we have and need one
	FString FolderPathString;
#if WITH_EDITOR
	if (bUseActorFolders && Component == Owner->GetRootComponent())
	{
		const FName& FolderPath = Owner->GetFolderPath();
		if (!FolderPath.IsNone())
		{
			FolderPathString = FolderPath.ToString();

			TArray<FString> FolderSegments;
			FolderPathString.ParseIntoArray(FolderSegments, TEXT("/"));

			for (FString& Segment : FolderSegments)
			{
				Segment = UsdUtils::SanitizeUsdIdentifier(*Segment);
			}

			FolderPathString = FString::Join(FolderSegments, TEXT("/"));

			if (!FolderPathString.IsEmpty())
			{
				Path = FolderPathString / Path;
			}
		}
	}
#endif	  // WITH_EDITOR

	// Get parent prim path if we need to
	if (!ParentPrimPath.IsEmpty())
	{
		Path = ParentPrimPath / Path;
	}
	else
	{
		FString FoundParentPath;

		if (USceneComponent* ParentComp = Component->GetAttachParent())
		{
			FoundParentPath = GetPrimPathForObject(ParentComp, TEXT(""), bUseActorFolders, RootPrimName);
		}
		else
		{
			FoundParentPath = TEXT("/") + UsdUtils::SanitizeUsdIdentifier(*RootPrimName);
		}

		Path = FoundParentPath / Path;
	}

	return Path;
}

TUsdStore<pxr::TfToken> UsdUtils::GetUVSetName(int32 UVChannelIndex)
{
	FScopedUnrealAllocs UnrealAllocs;

	FString UVSetName = TEXT("primvars:st");

	if (UVChannelIndex > 0)
	{
		UVSetName += LexToString(UVChannelIndex);
	}

	TUsdStore<pxr::TfToken> UVSetNameToken = MakeUsdStore<pxr::TfToken>(UnrealToUsd::ConvertString(*UVSetName).Get());

	return UVSetNameToken;
}

TArray<TUsdStore<pxr::UsdGeomPrimvar>> UsdUtils::GetUVSetPrimvars(const pxr::UsdPrim& UsdPrim, int32 MaxNumPrimvars)
{
	if (!UsdPrim)
	{
		return {};
	}

	FScopedUsdAllocs Allocs;

	TArray<TUsdStore<pxr::UsdGeomPrimvar>> TexCoord2fPrimvars;
	TArray<TUsdStore<pxr::UsdGeomPrimvar>> Float2Primvars;

	// Collect all primvars that could be used as UV sets
	pxr::UsdGeomPrimvarsAPI PrimvarsAPI{UsdPrim};
	for (const pxr::UsdGeomPrimvar& Primvar : PrimvarsAPI.GetPrimvars())
	{
		if (!Primvar || !Primvar.HasValue())
		{
			continue;
		}

		// We only care about primvars that can be used as float2[]. TexCoord2f is included
		const pxr::SdfValueTypeName& TypeName = Primvar.GetTypeName();
		if (!TypeName.GetType().IsA(pxr::SdfValueTypeNames->Float2Array.GetType()))
		{
			continue;
		}

		if (Primvar.GetTypeName().GetRole() == pxr::SdfValueTypeNames->TexCoord2f.GetRole())
		{
			TexCoord2fPrimvars.Add(Primvar);
		}
		else if (GParseUVSetsFromFloat2Primvars)
		{
			Float2Primvars.Add(Primvar);
		}
	}

	TexCoord2fPrimvars.Sort(
		[](const TUsdStore<pxr::UsdGeomPrimvar>& A, const TUsdStore<pxr::UsdGeomPrimvar>& B)
		{
			return A.Get().GetName() < B.Get().GetName();
		}
	);
	Float2Primvars.Sort(
		[](const TUsdStore<pxr::UsdGeomPrimvar>& A, const TUsdStore<pxr::UsdGeomPrimvar>& B)
		{
			return A.Get().GetName() < B.Get().GetName();
		}
	);

	TArray<TUsdStore<pxr::UsdGeomPrimvar>> Result;
	Result.Reserve(FMath::Min(TexCoord2fPrimvars.Num() + Float2Primvars.Num(), MaxNumPrimvars));

	int32 TexCoordPrimvarIndex = 0;
	while (Result.Num() < MaxNumPrimvars && TexCoord2fPrimvars.IsValidIndex(TexCoordPrimvarIndex))
	{
		Result.Add(TexCoord2fPrimvars[TexCoordPrimvarIndex++]);
	}

	int32 Float2PrimvarIndex = 0;
	while (Result.Num() < MaxNumPrimvars && Float2Primvars.IsValidIndex(Float2PrimvarIndex))
	{
		Result.Add(Float2Primvars[Float2PrimvarIndex++]);
	}

	return Result;
}

TArray<TUsdStore<pxr::UsdGeomPrimvar>> UsdUtils::AssemblePrimvarsIntoUVSets(
	const TArray<TUsdStore<pxr::UsdGeomPrimvar>>& AllMeshUVPrimvars,
	const TMap<FString, int32>& AllowedPrimvarsToUVIndex
)
{
	TArray<TUsdStore<pxr::UsdGeomPrimvar>> PrimvarsByUVIndex;

	if (AllowedPrimvarsToUVIndex.Num() > 0)
	{
		for (const TUsdStore<pxr::UsdGeomPrimvar>& MeshUVPrimvar : AllMeshUVPrimvars)
		{
			FString PrimvarName = UsdToUnreal::ConvertToken(MeshUVPrimvar.Get().GetName());
			PrimvarName.RemoveFromStart(TEXT("primvars:"));

			if (const int32* FoundTargetUVIndex = AllowedPrimvarsToUVIndex.Find(PrimvarName))
			{
				int32 TargetUVIndex = *FoundTargetUVIndex;
				if (TargetUVIndex < 0)
				{
					continue;
				}

				if (!PrimvarsByUVIndex.IsValidIndex(TargetUVIndex))
				{
					if (TargetUVIndex < USD_PREVIEW_SURFACE_MAX_UV_SETS)
					{
						PrimvarsByUVIndex.SetNum(TargetUVIndex + 1);
					}
					else
					{
						continue;
					}
				}

				TUsdStore<pxr::UsdGeomPrimvar>& ExistingPrimvar = PrimvarsByUVIndex[TargetUVIndex];
				if (!ExistingPrimvar.Get())
				{
					PrimvarsByUVIndex[TargetUVIndex] = MeshUVPrimvar;
				}
			}
		}
	}

	return PrimvarsByUVIndex;
}

TMap<FString, int32> UsdUtils::AssemblePrimvarsIntoPrimvarToUVIndexMap(const TArray<TUsdStore<pxr::UsdGeomPrimvar>>& AllMeshUVPrimvars)
{
	TMap<FString, int32> Result;
	Result.Reserve(AllMeshUVPrimvars.Num());

	for (int32 UVIndex = 0; UVIndex < AllMeshUVPrimvars.Num(); ++UVIndex)
	{
		const TUsdStore<pxr::UsdGeomPrimvar>& Primvar = AllMeshUVPrimvars[UVIndex];
		FString PrimvarName = UsdToUnreal::ConvertToken(Primvar.Get().GetName());
		PrimvarName.RemoveFromStart(TEXT("primvars:"));

		Result.Add(PrimvarName, UVIndex);
	}

	return Result;
}

TMap<FString, int32> UsdUtils::CombinePrimvarsIntoUVSets(const TSet<FString>& AllPrimvars, const TSet<FString>& PreferredPrimvars)
{
	TArray<FString> SortedPrimvars = AllPrimvars.Array();

	// Promote a deterministic primvar-to-UV-index assignment preferring texCoord2f primvars
	SortedPrimvars.Sort(
		[&PreferredPrimvars](const FString& LHS, const FString& RHS)
		{
			const bool bLHSPreferred = PreferredPrimvars.Contains(LHS);
			const bool bRHSPreferred = PreferredPrimvars.Contains(RHS);
			if (bLHSPreferred == bRHSPreferred)
			{
				return LHS < RHS;
			}
			else
			{
				return bLHSPreferred < bRHSPreferred;
			}
		}
	);

	// We can only have up to USD_PREVIEW_SURFACE_MAX_UV_SETS UV sets
	SortedPrimvars.SetNum(FMath::Min(SortedPrimvars.Num(), (int32)USD_PREVIEW_SURFACE_MAX_UV_SETS));

	TMap<FString, int32> PrimvarToUVIndex;
	PrimvarToUVIndex.Reserve(SortedPrimvars.Num());
	int32 UVIndex = 0;
	for (const FString& Primvar : SortedPrimvars)
	{
		PrimvarToUVIndex.Add(Primvar, UVIndex++);
	}

	return PrimvarToUVIndex;
}

TMap<FString, int32> UsdUtils::GetPrimvarToUVIndexMap(const pxr::UsdPrim& UsdPrim, int32 MaxNumPrimvars)
{
	TArray<TUsdStore<pxr::UsdGeomPrimvar>> PrimvarsToUse = UsdUtils::GetUVSetPrimvars(UsdPrim, MaxNumPrimvars);
	return UsdUtils::AssemblePrimvarsIntoPrimvarToUVIndexMap(PrimvarsToUse);
}

bool UsdUtils::IsAnimated(const pxr::UsdPrim& Prim)
{
	if (!Prim || !Prim.IsActive())
	{
		return false;
	}

	FScopedUsdAllocs UsdAllocs;

	if (HasAnimatedTransform(Prim))
	{
		return true;
	}

	if (HasAnimatedAttributes(Prim))
	{
		return true;
	}

	if (pxr::UsdSkelSkeleton Skeleton{Prim})
	{
		if (pxr::UsdSkelRoot ClosestParentSkelRoot = pxr::UsdSkelRoot{UsdUtils::GetClosestParentSkelRoot(Prim)})
		{
			pxr::UsdSkelCache SkeletonCache;
			SkeletonCache.Populate(ClosestParentSkelRoot, pxr::UsdTraverseInstanceProxies());

			pxr::UsdSkelSkeletonQuery SkelQuery = SkeletonCache.GetSkelQuery(Skeleton);
			if (pxr::UsdSkelAnimQuery AnimQuery = SkelQuery.GetAnimQuery())
			{
				std::vector<double> JointTimeSamples;
				std::vector<double> BlendShapeTimeSamples;
				if ((AnimQuery.GetJointTransformTimeSamples(&JointTimeSamples) && JointTimeSamples.size() > 0)
					|| (AnimQuery.GetBlendShapeWeightTimeSamples(&BlendShapeTimeSamples) && BlendShapeTimeSamples.size() > 0))
				{
					return true;
				}
			}
		}
	}
	else if (pxr::UsdVolVolume Volume{Prim})
	{
		pxr::UsdStageRefPtr Stage = Prim.GetStage();

		const std::map<pxr::TfToken, pxr::SdfPath>& FieldMap = Volume.GetFieldPaths();
		for (std::map<pxr::TfToken, pxr::SdfPath>::const_iterator Iter = FieldMap.cbegin(); Iter != FieldMap.cend(); ++Iter)
		{
			const pxr::SdfPath& AssetPrimPath = Iter->second;

			if (pxr::UsdVolOpenVDBAsset OpenVDBAsset{Stage->GetPrimAtPath(AssetPrimPath)})
			{
				std::vector<double> TimeSamples;
				pxr::UsdAttribute FilePathAttr = OpenVDBAsset.GetFilePathAttr();
				if (FilePathAttr && FilePathAttr.GetTimeSamples(&TimeSamples) && TimeSamples.size() > 1)
				{
					return true;
				}
			}
		}
	}

	return false;
}

bool UsdUtils::HasAnimatedAttributes(const pxr::UsdPrim& Prim)
{
	if (!Prim || !Prim.IsActive())
	{
		return false;
	}

	FScopedUsdAllocs UsdAllocs;

	const std::vector<pxr::UsdAttribute>& Attributes = Prim.GetAttributes();
	for (const pxr::UsdAttribute& Attribute : Attributes)
	{
		std::vector<double> TimeSamples;
		if (Attribute.GetTimeSamples(&TimeSamples) && TimeSamples.size() > 0)
		{
			return true;
		}
	}

	return false;
}

bool UsdUtils::HasAnimatedTransform(const pxr::UsdPrim& Prim)
{
	if (!Prim || !Prim.IsActive())
	{
		return false;
	}

	FScopedUsdAllocs UsdAllocs;

	pxr::UsdGeomXformable Xformable(Prim);
	if (Xformable)
	{
		std::vector<double> TimeSamples;
		Xformable.GetTimeSamples(&TimeSamples);

		if (TimeSamples.size() > 0)
		{
			return true;
		}

		// If this xformable has an op to reset the xform stack and one of its ancestors is animated, then we need to pretend
		// its transform is also animated. This because that op effectively means "discard the parent transform and treat this
		// as a direct world transform", but when reading we'll manually recompute the relative transform to its parent anyway
		// (for simplicity's sake). If that parent (or any of its ancestors) is being animated, we'll need to recompute this
		// for every animation keyframe, which basically means we're animated too
		if (Xformable.GetResetXformStack())
		{
			pxr::UsdPrim AncestorPrim = Prim.GetParent();
			while (AncestorPrim && !AncestorPrim.IsPseudoRoot())
			{
				if (pxr::UsdGeomXformable AncestorXformable{AncestorPrim})
				{
					std::vector<double> AncestorTimeSamples;
					if (AncestorXformable.GetTimeSamples(&AncestorTimeSamples) && AncestorTimeSamples.size() > 0)
					{
						return true;
					}

					// The exception is if our ancestor also wants to reset its xform stack (i.e. its transform is meant to be
					// used as the world transform). In this case we don't need to care about higher up ancestors anymore, as
					// their transforms wouldn't affect below this prim anyway
					if (AncestorXformable.GetResetXformStack())
					{
						break;
					}
				}

				AncestorPrim = AncestorPrim.GetParent();
			}
		}
	}

	return false;
}

bool UsdUtils::HasAnimatedVisibility(const pxr::UsdPrim& Prim)
{
	if (!Prim || !Prim.IsActive())
	{
		return false;
	}

	FScopedUsdAllocs UsdAllocs;

	pxr::UsdGeomImageable Imageable(Prim);
	if (Imageable)
	{
		if (pxr::UsdAttribute Attr = Imageable.GetVisibilityAttr())
		{
			if (Attr.GetNumTimeSamples() > 0)
			{
				return true;
			}
		}
	}

	return false;
}

namespace UE::ConversionUtilsImpl::Private
{
	// Convenience function so we don't have to spell this out every time
	inline void CollectTimeSamplesIfNeeded(bool bCollectTimeSamples, const pxr::UsdAttribute& Attr, TArray<double>* OutTimeSamples)
	{
		std::vector<double> TempTimeSamples;
		if (bCollectTimeSamples && Attr.GetTimeSamples(&TempTimeSamples))
		{
			OutTimeSamples->Append(TempTimeSamples.data(), TempTimeSamples.size());
		}
	}

	bool GetOrCollectAnimatedBounds(
		const pxr::UsdPrim& Prim,
		TArray<double>* OutTimeSamples,
		bool bCollectTimeSamples,
		bool bIsParentPrim,
		EUsdPurpose IncludedPurposes,
		bool bUseExtentsHint,
		bool bIgnoreVisibility
	)
	{
		if (!Prim)
		{
			return false;
		}

		// If we want to collect timeSamples we must have some place to put them in
		if (!ensure(!bCollectTimeSamples || OutTimeSamples))
		{
			return false;
		}

		FScopedUsdAllocs UsdAllocs;

		// If the prim is fully invisible due to visibility or purpose then we shouldn't even check it
		bool bHasAnimatedVisibility = false;
		if (pxr::UsdGeomImageable Imageable{Prim}; !bIgnoreVisibility && Imageable)
		{
			if (pxr::UsdAttribute Visibility = Imageable.GetVisibilityAttr())
			{
				// Keep track of this for later
				bHasAnimatedVisibility = Visibility.ValueMightBeTimeVarying();

				if (bHasAnimatedVisibility)
				{
					CollectTimeSamplesIfNeeded(bCollectTimeSamples, Visibility, OutTimeSamples);
				}
				else
				{
					pxr::TfToken VisibilityToken;
					if (!bIsParentPrim && Visibility.Get(&VisibilityToken) && VisibilityToken == pxr::UsdGeomTokens->invisible)
					{
						// We don't "propagate the (in)visibility token", we just flat out stop recursing and abandon the subtree
						return false;
					}
				}
			}
		}
		if (!bIsParentPrim && !EnumHasAllFlags(IncludedPurposes, IUsdPrim::GetPurpose(Prim)))
		{
			return false;
		}

		// If the prim has authored animated extents we know we're fully done, because our computed bounds
		// will also need to be animated and will read *exclusively* from these anyway.
		// We don't even need to collect any further timeSamples from child prims after this, as we will be ignoring individual
		// animations on random prims in the subtree and instead just using the authored extent animation.
		// Also, extentsHint is preferred over extent, so check for that first.
		if (pxr::UsdGeomModelAPI GeomModelAPI{Prim}; bUseExtentsHint && GeomModelAPI)
		{
			if (pxr::UsdAttribute ExtentsHint = GeomModelAPI.GetExtentsHintAttr())
			{
				if (ExtentsHint.HasAuthoredValue())
				{
					CollectTimeSamplesIfNeeded(bCollectTimeSamples, ExtentsHint, OutTimeSamples);
					return ExtentsHint.ValueMightBeTimeVarying();
				}
			}
		}
		if (pxr::UsdGeomBoundable Boundable{Prim})
		{
			if (pxr::UsdAttribute Extent = Boundable.GetExtentAttr())
			{
				// If we have authored extent or extentsHint (even if not animated, i.e. just default opinions), the
				// BBoxCache will refuse to compute bounds at any timeCode and just fallback to using the authored stuff
				if (Extent.HasAuthoredValue())
				{
					CollectTimeSamplesIfNeeded(bCollectTimeSamples, Extent, OutTimeSamples);
					return Extent.ValueMightBeTimeVarying();
				}
			}
		}

		// It's visible at the default timeCode, but has animated visibility. This means
		// it could affect the bounds as it becomes visible or invisible, so just return now.
		if (!bCollectTimeSamples && bHasAnimatedVisibility)
		{
			return true;
		}

		bool bHasAnimatedBounds = bHasAnimatedVisibility;

		// Otherwise the prim may have some animated attributes that would make our parent extents animated.
		// For this function we mostly care about whether the *bounds themselves* are animated.
		// The parent prim having animated transform means we'll just put this transform on the component itself,
		// but the bounds could remain un-animated
		if (pxr::UsdGeomXformable Xformable{Prim}; !bIsParentPrim && Xformable)
		{
			if (Xformable.TransformMightBeTimeVarying())
			{
				bHasAnimatedBounds = true;

				std::vector<double> TempTimeSamples;
				if (bCollectTimeSamples && Xformable.GetTimeSamples(&TempTimeSamples))
				{
					OutTimeSamples->Append(TempTimeSamples.data(), TempTimeSamples.size());
				}
			}
		}
		else if (pxr::UsdGeomPointBased PointBased{Prim})
		{
			if (pxr::UsdAttribute Points = PointBased.GetPointsAttr())
			{
				if (Points.ValueMightBeTimeVarying())
				{
					CollectTimeSamplesIfNeeded(bCollectTimeSamples, Points, OutTimeSamples);
					bHasAnimatedBounds = true;
				}
			}
		}
		else if (pxr::UsdGeomPointInstancer PointInstancer{Prim})
		{
			if (pxr::UsdAttribute Positions = PointInstancer.GetPositionsAttr())
			{
				if (Positions.ValueMightBeTimeVarying())
				{
					CollectTimeSamplesIfNeeded(bCollectTimeSamples, Positions, OutTimeSamples);
					bHasAnimatedBounds = true;
				}
			}
		}
		// Check for a SkelRoot with SkelAnimation
		else if (UE::FUsdPrim SkelAnimationPrim = UsdUtils::FindFirstAnimationSource(UE::FUsdPrim{Prim}))
		{
			pxr::UsdSkelAnimation SkelAnim{SkelAnimationPrim};
			if (ensure(SkelAnim))
			{
				const bool bIncludeInherited = false;
				for (const pxr::TfToken& SkelAnimAttrName : SkelAnim.GetSchemaAttributeNames(bIncludeInherited))
				{
					if (pxr::UsdAttribute Attr = SkelAnim.GetPrim().GetAttribute(SkelAnimAttrName))
					{
						if (Attr.ValueMightBeTimeVarying())
						{
							bHasAnimatedBounds = true;

							if (!bCollectTimeSamples)
							{
								break;
							}
							CollectTimeSamplesIfNeeded(bCollectTimeSamples, Attr, OutTimeSamples);
						}
					}
				}
			}
		}

		// If we're not collecting timeSamples and we run into a prim with animated bounds then we know that
		// we're done, and can return then. If we're collecting timeSamples however then we want instead to remember
		// that we found those animated bounds, but still try to step into children in case they also had animated
		// bounds and additional timeSamples
		if (!bCollectTimeSamples && bHasAnimatedBounds)
		{
			return true;
		}

		for (pxr::UsdPrim Child : Prim.GetFilteredChildren(pxr::UsdTraverseInstanceProxies(pxr::UsdPrimAllPrimsPredicate)))
		{
			const bool bChildIsParentPrim = false;
			bHasAnimatedBounds |= GetOrCollectAnimatedBounds(
				Child,
				OutTimeSamples,
				bCollectTimeSamples,
				bChildIsParentPrim,
				IncludedPurposes,
				bUseExtentsHint,
				bIgnoreVisibility
			);

			// Don't need to visit any other children, we're done here
			if (!bCollectTimeSamples && bHasAnimatedBounds)
			{
				return true;
			}
		}

		return bHasAnimatedBounds;
	}
}	 // namespace UE::ConversionUtilsImpl::Private

bool UsdUtils::HasAnimatedBounds(const pxr::UsdPrim& Prim, EUsdPurpose IncludedPurposes, bool bUseExtentsHint, bool bIgnoreVisibility)
{
	// "ParentPrim" here because there are slight differences in behavior between handling the actual provided
	// prim and another random prim in its subtree (for which bIsParentPrim will be 'false')
	const bool bIsParentPrim = true;
	const bool bCollectTimeSamples = false;
	TArray<double>* OutTimeSamples = nullptr;
	return UE::ConversionUtilsImpl::Private::GetOrCollectAnimatedBounds(
		Prim,
		OutTimeSamples,
		bCollectTimeSamples,
		bIsParentPrim,
		IncludedPurposes,
		bUseExtentsHint,
		bIgnoreVisibility
	);
}

bool UsdUtils::GetAnimatedBoundsTimeSamples(
	const pxr::UsdPrim& InPrim,
	TArray<double>& OutTimeSamples,
	EUsdPurpose InIncludedPurposes,
	bool bInUseExtentsHint,
	bool bInIgnoreVisibility
)
{
	OutTimeSamples.Reset();

	// "ParentPrim" here because there are slight differences in behavior between handling the actual provided
	// prim and another random prim in its subtree (for which bIsParentPrim will be 'false')
	const bool bIsParentPrim = true;
	const bool bCollectTimeSamples = true;
	const bool bHasAnimatedBounds = UE::ConversionUtilsImpl::Private::GetOrCollectAnimatedBounds(
		InPrim,
		&OutTimeSamples,
		bCollectTimeSamples,
		bIsParentPrim,
		InIncludedPurposes,
		bInUseExtentsHint,
		bInIgnoreVisibility
	);

	OutTimeSamples.Sort();

	return bHasAnimatedBounds;
}

TArray<UE::FUsdAttribute> UsdUtils::GetAnimatedAttributes(const pxr::UsdPrim& Prim)
{
	if (!Prim || !Prim.IsActive())
	{
		return {};
	}

	TUsdStore<std::vector<pxr::UsdAttribute>> UsdAttributes = Prim.GetAttributes();

	TArray<UE::FUsdAttribute> Attributes;
	Attributes.Reserve(UsdAttributes->size());

	for (const pxr::UsdAttribute& UsdAttribute : *UsdAttributes)
	{
		TUsdStore<std::vector<double>> TimeSamples;
		if (UsdAttribute.GetTimeSamples(&TimeSamples.Get()) && TimeSamples->size() > 0)
		{
			Attributes.Add(UE::FUsdAttribute{UsdAttribute});
		}
	}

	return Attributes;
}

TArray<FString> UsdUtils::GetAppliedAPISchemaPropertyNames(const FString& SchemaName)
{
	const pxr::UsdSchemaRegistry& Registry = pxr::UsdSchemaRegistry::GetInstance();

	const pxr::UsdPrimDefinition* Definition = Registry.FindAppliedAPIPrimDefinition(UnrealToUsd::ConvertToken(*SchemaName).Get());
	if (!Definition)
	{
		return {};
	}

	const TUsdStore<std::vector<pxr::TfToken>> PxrPropertyNames = Definition->GetPropertyNames();

	TArray<FString> PropertyNames;
	PropertyNames.Reserve(PxrPropertyNames->size());

	for (const pxr::TfToken& PxrPropertyName : *PxrPropertyNames)
	{
		PropertyNames.Add(UsdToUnreal::ConvertToken(PxrPropertyName));
	}

	return PropertyNames;
}

bool UsdUtils::HasAuthoredKind(const pxr::UsdPrim& Prim)
{
	FScopedUsdAllocs Allocs;

	pxr::UsdModelAPI Model{Prim};
	pxr::TfToken KindToken;
	return Model && Model.GetKind(&KindToken);
}

EUsdDefaultKind UsdUtils::GetDefaultKind(const pxr::UsdPrim& Prim)
{
	FScopedUsdAllocs Allocs;

	pxr::UsdModelAPI Model{pxr::UsdTyped(Prim)};

	EUsdDefaultKind Result = EUsdDefaultKind::None;

	if (!Model)
	{
		return Result;
	}

	// We need KindValidationNone here or else we get inconsistent results when a prim references another prim that is a component.
	// For example, when referencing a component prim in another file, this returns 'true' if the referencer is a root prim,
	// but false if the referencer is within another Xform prim, for whatever reason.
	if (Model.IsKind(pxr::KindTokens->model, pxr::UsdModelAPI::KindValidationNone))
	{
		Result |= EUsdDefaultKind::Model;
	}

	if (Model.IsKind(pxr::KindTokens->component, pxr::UsdModelAPI::KindValidationNone))
	{
		Result |= EUsdDefaultKind::Component;
	}

	if (Model.IsKind(pxr::KindTokens->group, pxr::UsdModelAPI::KindValidationNone))
	{
		Result |= EUsdDefaultKind::Group;
	}

	if (Model.IsKind(pxr::KindTokens->assembly, pxr::UsdModelAPI::KindValidationNone))
	{
		Result |= EUsdDefaultKind::Assembly;
	}

	if (Model.IsKind(pxr::KindTokens->subcomponent, pxr::UsdModelAPI::KindValidationNone))
	{
		Result |= EUsdDefaultKind::Subcomponent;
	}

	return Result;
}

bool UsdUtils::SetDefaultKind(pxr::UsdPrim& Prim, EUsdDefaultKind NewKind)
{
	if (!Prim)
	{
		return false;
	}

	FScopedUsdAllocs Allocs;

	const int32 NewKindInt = static_cast<int32>(NewKind);
	const bool bSingleFlagSet = NewKindInt != 0 && (NewKindInt & (NewKindInt - 1)) == 0;
	if (!bSingleFlagSet)
	{
		return false;
	}

	pxr::TfToken NewKindToken;
	switch (NewKind)
	{
		default:
		case EUsdDefaultKind::Model:
		{
			NewKindToken = pxr::KindTokens->model;
			break;
		}
		case EUsdDefaultKind::Component:
		{
			NewKindToken = pxr::KindTokens->component;
			break;
		}
		case EUsdDefaultKind::Group:
		{
			NewKindToken = pxr::KindTokens->group;
			break;
		}
		case EUsdDefaultKind::Assembly:
		{
			NewKindToken = pxr::KindTokens->assembly;
			break;
		}
		case EUsdDefaultKind::Subcomponent:
		{
			NewKindToken = pxr::KindTokens->subcomponent;
			break;
		}
	}
	if (NewKindToken.IsEmpty())
	{
		return false;
	}

	return IUsdPrim::SetKind(Prim, NewKindToken);
}

UsdUtils::ECollapsingPreference UsdUtils::GetCollapsingPreference(const pxr::UsdPrim& Prim)
{
	if (Prim)
	{
		if (UsdUtils::PrimHasSchema(Prim, UnrealIdentifiers::UnrealCollapsingAPI))
		{
			FScopedUsdAllocs UsdAllocs;

			if (pxr::UsdAttribute Attr = Prim.GetAttribute(UnrealIdentifiers::UnrealCollapsingAttr))
			{
				pxr::TfToken Value;
				if (Attr.Get(&Value))
				{
					if (Value == UnrealIdentifiers::CollapsingAllow)
					{
						return ECollapsingPreference::Allow;
					}
					else if (Value == UnrealIdentifiers::CollapsingNever)
					{
						return ECollapsingPreference::Never;
					}
				}
			}
		}
	}

	return ECollapsingPreference::Default;
}

bool UsdUtils::SetCollapsingPreference(const pxr::UsdPrim& Prim, UsdUtils::ECollapsingPreference NewPreference)
{
	if (!Prim)
	{
		return false;
	}

	FScopedUsdAllocs UsdAllocs;

	// Note: Don't use an pxr::SdfChangeBlock here, as USD needs to emit separate notices for the schema addition and attribute
	// addition, otherwise it will emit an ObjectsChanged notice that *only* contains the schema application details

	const bool bAppliedSchema = UsdUtils::ApplySchema(Prim, UnrealIdentifiers::UnrealCollapsingAPI);
	if (!bAppliedSchema)
	{
		return false;
	}

	const pxr::SdfVariability Variability = pxr::SdfVariabilityUniform;
	if (pxr::UsdAttribute Attr = Prim.CreateAttribute(UnrealIdentifiers::UnrealCollapsingAttr, pxr::SdfValueTypeNames->Token, Variability))
	{
		switch (NewPreference)
		{
			case ECollapsingPreference::Allow:
			{
				return Attr.Set(UnrealIdentifiers::CollapsingAllow);
			}
			case ECollapsingPreference::Default:
			{
				return Attr.Set(UnrealIdentifiers::CollapsingDefault);
			}
			case ECollapsingPreference::Never:
			{
				return Attr.Set(UnrealIdentifiers::CollapsingNever);
			}
			default:
			{
				ensure(false);
				break;
			}
		}
	}

	return false;
}

bool UsdUtils::PrimCollapses(
	const pxr::UsdPrim& Prim,
	EUsdDefaultKind KindsToCollapse,
	bool bUsePrimKindsForCollapsing,
	bool bUseSchemaForCollapsing,
	bool bCanCollapseByDefault
)
{
	// Prioritize checking for the schema if we have one
	if (bUseSchemaForCollapsing)
	{
		UsdUtils::ECollapsingPreference Preference = UsdUtils::GetCollapsingPreference(Prim);
		switch (Preference)
		{
			case UsdUtils::ECollapsingPreference::Allow:
			{
				return true;
			}
			case UsdUtils::ECollapsingPreference::Never:
			{
				return false;
			}
			case UsdUtils::ECollapsingPreference::Default:	  // Fallthrough
			default:
			{
				break;
			}
		}
	}

	bool bCanBeCollapsed = bCanCollapseByDefault;

	if (bUsePrimKindsForCollapsing)
	{
		EUsdDefaultKind PrimKind = UsdUtils::GetDefaultKind(Prim);

		// Note that this is false if PrimKind is None
		const bool bPrimKindCollapses = EnumHasAnyFlags(KindsToCollapse, PrimKind);
		bCanBeCollapsed = (KindsToCollapse != EUsdDefaultKind::None)
						  && (bPrimKindCollapses || (PrimKind == EUsdDefaultKind::None && GCollapsePrimsWithoutKind));

		if (!bCanBeCollapsed)
		{
			pxr::UsdModelAPI Model{Prim};
			bCanBeCollapsed = bCanBeCollapsed || Model.IsKind(pxr::TfToken("prop"), pxr::UsdModelAPI::KindValidationNone);
		}
	}

	return bCanBeCollapsed;
}

EUsdDrawMode UsdUtils::GetAppliedDrawMode(const pxr::UsdPrim& Prim)
{
	// Reference: https://openusd.org/release/api/class_usd_geom_model_a_p_i.html#UsdGeomModelAPI_drawMode

	if (!Prim)
	{
		return EUsdDrawMode::Default;
	}

	FScopedUsdAllocs Allocs;

	// Only "models" should have these (i.e. uninterrupted chain of authored "kind"s back to the root prim)
	if (!Prim.IsModel())
	{
		return EUsdDrawMode::Default;
	}

	pxr::UsdGeomModelAPI GeomModelAPI{Prim};
	if (!GeomModelAPI)
	{
		return EUsdDrawMode::Default;
	}

	bool bHasAuthoredApply = false;
	bool bShouldApplyFromAttr = false;
	pxr::UsdAttribute Attr = GeomModelAPI.GetModelApplyDrawModeAttr();
	if (Attr && Attr.HasAuthoredValue() && Attr.Get(&bShouldApplyFromAttr))
	{
		if (!bShouldApplyFromAttr)
		{
			return EUsdDrawMode::Default;
		}

		bHasAuthoredApply = true;
	}

	// "Models of kind component are treated as if model:applyDrawMode were true"
	// According to UsdImagingDelegate::_IsDrawModeApplied this only works as a "fallback" though:
	// if the prim has authored whether to apply or not we always use that directly
	pxr::UsdModelAPI Model{Prim};
	const bool bIsComponentKind = Model && Model.IsKind(pxr::KindTokens->component, pxr::UsdModelAPI::KindValidationNone);
	if (!bHasAuthoredApply && !bIsComponentKind)
	{
		return EUsdDrawMode::Default;
	}

	// Note: We can provide the parent draw mode to optimize the ComputeModelDrawMode call if it becomes an issue
	pxr::TfToken DesiredDrawMode = GeomModelAPI.ComputeModelDrawMode();
	if (DesiredDrawMode == pxr::UsdGeomTokens->default_)
	{
		return EUsdDrawMode::Default;
	}
	else if (DesiredDrawMode == pxr::UsdGeomTokens->origin)
	{
		return EUsdDrawMode::Origin;
	}
	else if (DesiredDrawMode == pxr::UsdGeomTokens->bounds)
	{
		return EUsdDrawMode::Bounds;
	}
	else if (DesiredDrawMode == pxr::UsdGeomTokens->cards)
	{
		return EUsdDrawMode::Cards;
	}
	else if (DesiredDrawMode == pxr::UsdGeomTokens->inherited)
	{
		// If we're using ComputeModelDrawMode we shouldn't get inherited or anything else here
		ensure(false);
		return EUsdDrawMode::Inherited;
	}
	else
	{
		ensure(false);
		return EUsdDrawMode::Default;
	}
}

TMap<FString, UsdUtils::FVolumePrimInfo> UsdUtils::GetVolumeInfoByFilePathHash(const pxr::UsdPrim& VolumePrim)
{
	// Collect all the .vdb files that this prim wants to parse, and the desired fields/grids from them.
	//
	// In VDB terminology a "grid" is essentially a 3D texture, and can have formats like float, double3, half, etc.
	// In USD the analogous term is "field", but essentially means the same thing. Possibly the terminology is abstracted
	// to also fit the Field3D library, which we don't support it. Field/grid will be used interchangeably here.
	//
	// USD is very flexible and allows the user to reference specific grids from of each .vdb file. The syntax makes it
	// difficult to find out at once all the grids we'll need to parse from each the .vdb files, so here we need to group them
	// up first before deferring to the SparseVolumeTextureFactory.
	//
	// Note that USD allows a single Volume prim to reference grids from multiple .vdb files, and to also "timeSample" the
	// file reference to allow for volume animations. This means that in UE a "Volume" prim corresponds to a single
	// HeterogeneousVolumeActor, but which in turn can have any number of Sparse Volume Textures (one for each .vdb file referenced).

	pxr::UsdVolVolume Volume{VolumePrim};
	if (!Volume)
	{
		return {};
	}

	TMap<FString, FVolumePrimInfo> FilePathHashToInfo;

	FScopedUsdAllocs UsdAllocs;

	pxr::UsdStageRefPtr Stage = Volume.GetPrim().GetStage();

	const std::map<pxr::TfToken, pxr::SdfPath>& FieldMap = Volume.GetFieldPaths();
	for (std::map<pxr::TfToken, pxr::SdfPath>::const_iterator Iter = FieldMap.cbegin(); Iter != FieldMap.end(); ++Iter)
	{
		// This field name is the name of the field for the Volume prim, which can be anything and differ from the
		// grid name within the .vdb files
		const pxr::TfToken& FieldName = Iter->first;
		const pxr::SdfPath& AssetPrimPath = Iter->second;

		pxr::UsdPrim OpenVDBPrim = Stage->GetPrimAtPath(AssetPrimPath);
		pxr::UsdVolOpenVDBAsset OpenVDBPrimSchema = pxr::UsdVolOpenVDBAsset{OpenVDBPrim};
		if (OpenVDBPrimSchema)
		{
			pxr::UsdAttribute FilePathAttr = OpenVDBPrimSchema.GetFilePathAttr();

			FString ResolvedVDBPath = UsdUtils::GetResolvedAssetPath(FilePathAttr, pxr::UsdTimeCode::Default());

			// Find timesampled paths, if any
			TArray<double> TimeSamplePathTimeCodes;
			TArray<int32> TimeSamplePathIndices;
			TArray<FString> TimeSamplePaths;

			TMap<FString, int32> PathToIndex;

			std::vector<double> TimeSamples;
			if (FilePathAttr.GetTimeSamples(&TimeSamples) && TimeSamples.size() > 0)
			{
				UE::FSdfLayerOffset CombinedOffset = UsdUtils::GetPrimToStageOffset(UE::FUsdPrim{OpenVDBPrim});

				TimeSamplePathTimeCodes.Reserve(TimeSamples.size());
				TimeSamplePaths.Reserve(TimeSamples.size());
				TimeSamplePathIndices.Reserve(TimeSamples.size());
				for (double TimeSample : TimeSamples)
				{
					// We always want to store on the AssetUserData (which is where this stuff will end up in)
					// the time codes in the layer where the actual OpenVDBPrim is authored. If that layer is referenced
					// by a parent layer through an offset and scale, TimeSample will contain that offset and scale here,
					// which we need to undo
					double LayerLocalTimeCode = (TimeSample - CombinedOffset.Offset) / CombinedOffset.Scale;
					TimeSamplePathTimeCodes.Add(LayerLocalTimeCode);

					FString ResolvedFramePath = UsdUtils::GetResolvedAssetPath(FilePathAttr, TimeSample);

					// If we had no default time sample to act as the "main file", take the first frame
					if (ResolvedVDBPath.IsEmpty())
					{
						ResolvedVDBPath = ResolvedFramePath;
					}

					if (bRemoveDuplicates)
					{
						if (int32* FoundIndex = PathToIndex.Find(ResolvedFramePath))
						{
							TimeSamplePathIndices.Add(*FoundIndex);
						}
						else
						{
							TimeSamplePaths.Add(ResolvedFramePath);

							const int32 NewIndex = TimeSamplePaths.Num() - 1;
							PathToIndex.Add(ResolvedFramePath, NewIndex);
							TimeSamplePathIndices.Add(NewIndex);
						}
					}
					else
					{
						TimeSamplePaths.Add(ResolvedFramePath);

						const int32 NewIndex = TimeSamplePaths.Num() - 1;
						TimeSamplePathIndices.Add(NewIndex);
					}
				}
			}

			// Hash all the relevant file paths here: The collection of file paths to parse determines the SVT, and
			// we want one FSparseVolumeTextureInfo per SVT
			FString FilePathHashString;
			{
				FSHA1 SHA1;
				SHA1.UpdateWithString(*ResolvedVDBPath, ResolvedVDBPath.Len());
				for (const FString& TimeSamplePath : TimeSamplePaths)
				{
					SHA1.UpdateWithString(*TimeSamplePath, TimeSamplePath.Len());
				}
				SHA1.Final();

				FSHAHash FilePathHash;
				SHA1.GetHash(&FilePathHash.Hash[0]);

				FilePathHashString = FilePathHash.ToString();
			}

			if (!ResolvedVDBPath.IsEmpty())
			{
				FVolumePrimInfo& SparseVolumeTextureInfo = FilePathHashToInfo.FindOrAdd(FilePathHashString);
				SparseVolumeTextureInfo.SourceOpenVDBAssetPrimPaths.AddUnique(UsdToUnreal::ConvertPath(AssetPrimPath));
				SparseVolumeTextureInfo.SourceVDBFilePath = ResolvedVDBPath;
				SparseVolumeTextureInfo.TimeSamplePathTimeCodes = MoveTemp(TimeSamplePathTimeCodes);
				SparseVolumeTextureInfo.TimeSamplePathIndices = MoveTemp(TimeSamplePathIndices);
				SparseVolumeTextureInfo.TimeSamplePaths = MoveTemp(TimeSamplePaths);

				FString FieldNameStr = UsdToUnreal::ConvertToken(FieldName);
				SparseVolumeTextureInfo.VolumeFieldNames.AddUnique(FieldNameStr);

				pxr::TfToken GridName;
				pxr::UsdAttribute Attr = OpenVDBPrimSchema.GetFieldNameAttr();
				if (Attr && Attr.Get<pxr::TfToken>(&GridName))
				{
					FString GridNameStr = UsdToUnreal::ConvertToken(GridName);

					// Note we want this to add an entry to SparseVolumeTexture.GridNameToChannelNames even if we won't
					// find the schema on the prim, as we'll use these entries to make sure the generated Sparse Volume
					// Texture contains theses desired fields
					TMap<FString, FString>& ChannelToComponent = SparseVolumeTextureInfo.GridNameToChannelComponentMapping.FindOrAdd(GridNameStr);

					if (UsdUtils::PrimHasSchema(OpenVDBPrim.GetPrim(), UnrealIdentifiers::SparseVolumeTextureAPI))
					{
						// Parse desired data types for AttributesA and AttributesB channels
						TFunction<void(pxr::TfToken, TOptional<ESparseVolumeAttributesFormat>&)> HandleAttribute =
							[&OpenVDBPrim,
							 &AssetPrimPath,
							 &ResolvedVDBPath](pxr::TfToken AttrName, TOptional<ESparseVolumeAttributesFormat>& AttributeFormat)
						{
							using FormatMapType = std::unordered_map<pxr::TfToken, ESparseVolumeAttributesFormat, pxr::TfHash>;
							using InverseFormatMapType = std::unordered_map<ESparseVolumeAttributesFormat, FString>;

							// clang-format off
							const static FormatMapType FormatMap = {
								{pxr::TfToken("unorm8"),  ESparseVolumeAttributesFormat::Unorm8},
								{pxr::TfToken("float16"), ESparseVolumeAttributesFormat::Float16},
								{pxr::TfToken("float32"), ESparseVolumeAttributesFormat::Float32},
							};
							const static InverseFormatMapType InverseFormatMap = {
								{ESparseVolumeAttributesFormat::Unorm8,  TEXT("unorm8")},
								{ESparseVolumeAttributesFormat::Float16, TEXT("float16")},
								{ESparseVolumeAttributesFormat::Float32, TEXT("float32")},
							};
							// clang-format on

							pxr::TfToken DataType;
							pxr::UsdAttribute AttrA = OpenVDBPrim.GetAttribute(AttrName);
							if (AttrA && AttrA.Get<pxr::TfToken>(&DataType))
							{
								FormatMapType::const_iterator Iter = FormatMap.find(DataType);
								if (Iter != FormatMap.end())
								{
									ESparseVolumeAttributesFormat TargetFormat = Iter->second;

									// Check in case multiple OpenVDBAsset prims want different values for the data type
									const bool bIsSet = AttributeFormat.IsSet();
									if (bIsSet && AttributeFormat.GetValue() != TargetFormat)
									{
										const FString* ExistingFormat = nullptr;
										InverseFormatMapType::const_iterator ExistingIter = InverseFormatMap.find(AttributeFormat.GetValue());
										if (ExistingIter != InverseFormatMap.end())
										{
											ExistingFormat = &ExistingIter->second;
										}

										USD_LOG_USERWARNING(FText::Format(
											LOCTEXT(
												"DisagreeAttributeChannel",
												"OpenVDBAsset prims disagree on the attribute channel format for the Sparse Volume Texture generated for VDB file '{0}' (encountered '{1}' and '{2}'). If there are multiple opinions for the attribute channel formats from different OpenVDBAsset prims, they must all agree!"
											),
											FText::FromString(ResolvedVDBPath),
											FText::FromString(UsdToUnreal::ConvertToken(DataType)),
											FText::FromString(ExistingFormat ? **ExistingFormat : TEXT("unknown"))
										));
									}
									else if (!bIsSet)
									{
										AttributeFormat = TargetFormat;
									}
								}
								else
								{
									USD_LOG_USERWARNING(FText::Format(
										LOCTEXT(
											"InvalidChannelFormat",
											"Invalid Sparse Volume Texture attribute channel format '{0}'. Available formats: 'unorm8', 'float16' and 'float32'."
										),
										FText::FromString(UsdToUnreal::ConvertToken(DataType))
									));
								}
							}
						};
						HandleAttribute(UnrealIdentifiers::UnrealSVTAttributesADataType, SparseVolumeTextureInfo.AttributesAFormat);
						HandleAttribute(UnrealIdentifiers::UnrealSVTAttributesBDataType, SparseVolumeTextureInfo.AttributesBFormat);

						// Parse desired channel assignment
						pxr::VtArray<pxr::TfToken> DesiredChannels;
						pxr::VtArray<pxr::TfToken> DesiredComponents;
						pxr::UsdAttribute ComponentsAttr = OpenVDBPrim.GetAttribute(UnrealIdentifiers::UnrealSVTMappedGridComponents);
						pxr::UsdAttribute ChannelsAttr = OpenVDBPrim.GetAttribute(UnrealIdentifiers::UnrealSVTMappedAttributeChannels);
						if (ChannelsAttr &&														 //
							ComponentsAttr &&													 //
							ChannelsAttr.Get<pxr::VtArray<pxr::TfToken>>(&DesiredChannels) &&	 //
							ComponentsAttr.Get<pxr::VtArray<pxr::TfToken>>(&DesiredComponents))
						{
							// These must always match of course
							if (DesiredChannels.size() == DesiredComponents.size())
							{
								// If we have more than one OpenVDBAsset prim reading from the same VDB file, the declared component to
								// channel mappings must be compatible
								for (uint32 Index = 0; Index < DesiredChannels.size(); ++Index)
								{
									FString Channel = UsdToUnreal::ConvertString(DesiredChannels[Index]);
									FString Component = UsdToUnreal::ConvertString(DesiredComponents[Index]);

									if (const FString* ExistingComponentMapping = ChannelToComponent.Find(Channel))
									{
										if (Component != *ExistingComponentMapping)
										{
											USD_LOG_USERWARNING(FText::Format(
												LOCTEXT(
													"MultipleTargettingSameGrid",
													"Found multiple OpenVDBAsset prims (including '{0}') targetting the same grid '{1}', but with with conflicting grid component to Sparse Volume Texture attribute channel mapping (for example, both components '{2}' and '{3}' are mapped to the same channel '{4}', which is not allowed)"
												),
												FText::FromString(UsdToUnreal::ConvertPath(AssetPrimPath)),
												FText::FromString(GridNameStr),
												FText::FromString(*ExistingComponentMapping),
												FText::FromString(Component),
												FText::FromString(Channel)
											));
										}
									}
									else
									{
										ChannelToComponent.Add(Channel, Component);
									}
								}
							}
							else
							{
								USD_LOG_USERWARNING(FText::Format(
									LOCTEXT(
										"FailCustomAttributeMapping",
										"Failed to parse custom component to attribute mapping from OpenVDBAsset prim '{0}': The '{1}' and '{2}' attributes should have the same number of entries, but the former has {3} entries while the latter has {4}"
									),
									FText::FromString(UsdToUnreal::ConvertPath(AssetPrimPath)),
									FText::FromString(UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealSVTMappedGridComponents)),
									FText::FromString(UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealSVTMappedAttributeChannels)),
									static_cast<uint64>(DesiredComponents.size()),
									static_cast<uint64>(DesiredChannels.size())
								));
							}
						}
					}
				}
			}
			else
			{
				USD_LOG_USERWARNING(FText::Format(
					LOCTEXT("FailToFindFile", "Failed to find the VDB file '{0}' referenced by OpenVDBAsset prim at path '{1}'"),
					FText::FromString(ResolvedVDBPath),
					FText::FromString(UsdToUnreal::ConvertPath(AssetPrimPath))
				));
			}
		}
		else
		{
			USD_LOG_USERWARNING(FText::Format(
				LOCTEXT("FailToFindPrim", "Failed to find an OpenVDBAsset prim at path '{0}' for field '{1}' of prim '{2}'"),
				FText::FromString(UsdToUnreal::ConvertPath(AssetPrimPath)),
				FText::FromString(UsdToUnreal::ConvertToken(FieldName)),
				FText::FromString(UsdToUnreal::ConvertPath(Volume.GetPrim().GetPath()))
			));
		}
	}

	return FilePathHashToInfo;
}

TMultiMap<FString, FString> UsdUtils::GetVolumeMaterialParameterToFieldNameMap(const pxr::UsdPrim& VolumePrim)
{
	if (!VolumePrim)
	{
		return {};
	}

	FScopedUsdAllocs Allocs;

	if (!UsdUtils::PrimHasSchema(VolumePrim, UnrealIdentifiers::SparseVolumeTextureAPI))
	{
		return {};
	}

	pxr::UsdAttribute FieldsAttr = VolumePrim.GetAttribute(UnrealIdentifiers::UnrealSVTMappedFields);
	pxr::UsdAttribute ParametersAttr = VolumePrim.GetAttribute(UnrealIdentifiers::UnrealSVTMappedMaterialParameters);
	if (!FieldsAttr || !ParametersAttr)
	{
		return {};
	}

	pxr::VtArray<pxr::TfToken> MappedFields;
	pxr::VtArray<pxr::TfToken> MappedParameters;
	if (!FieldsAttr.Get<pxr::VtArray<pxr::TfToken>>(&MappedFields) || !ParametersAttr.Get<pxr::VtArray<pxr::TfToken>>(&MappedParameters))
	{
		return {};
	}

	if (MappedFields.size() != MappedParameters.size())
	{
		USD_LOG_USERWARNING(FText::Format(
			LOCTEXT(
				"FailToParseMaterialMapping",
				"Failed to parse custom parsed texture to material parameter mapping from volume prim '{0}': The '{1}' and '{2}' attributes should have the same number of entries, but the former has {3} entries while the latter has {4}"
			),
			FText::FromString(UsdToUnreal::ConvertPath(VolumePrim.GetPrimPath())),
			FText::FromString(UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealSVTMappedFields)),
			FText::FromString(UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealSVTMappedMaterialParameters)),
			static_cast<uint64>(MappedFields.size()),
			static_cast<uint64>(MappedParameters.size())
		));
		return {};
	}

	TMultiMap<FString, FString> MaterialParameterToFieldName;
	for (uint32 Index = 0; Index < MappedFields.size(); ++Index)
	{
		FString FieldName = UsdToUnreal::ConvertString(MappedFields[Index]);
		FString MaterialParameter = UsdToUnreal::ConvertString(MappedParameters[Index]);
		MaterialParameterToFieldName.Add(MaterialParameter, FieldName);
	}

	return MaterialParameterToFieldName;
}

TArray<FString> UsdUtils::GetSparseVolumeTextureParameterNames(const UMaterial* Material)
{
	if (!Material)
	{
		return {};
	}

	TArray<FString> Result;

	TMap<FMaterialParameterInfo, FMaterialParameterMetadata> SparseVolumeTextureParameters;
	Material->GetAllParametersOfType(EMaterialParameterType::SparseVolumeTexture, SparseVolumeTextureParameters);

	Result.Reserve(SparseVolumeTextureParameters.Num());
	for (const TPair<FMaterialParameterInfo, FMaterialParameterMetadata>& ParameterPair : SparseVolumeTextureParameters)
	{
		const FMaterialParameterInfo& ParameterInfo = ParameterPair.Key;
		FString ParameterName = ParameterInfo.Name.ToString();

		Result.Add(ParameterName);
	}

	return Result;
}

TArray<TUsdStore<pxr::UsdPrim>> UsdUtils::GetAllPrimsOfType(
	const pxr::UsdPrim& StartPrim,
	const pxr::TfType& SchemaType,
	const TArray<TUsdStore<pxr::TfType>>& ExcludeSchemaTypes
)
{
	return GetAllPrimsOfType(
		StartPrim,
		SchemaType,
		[](const pxr::UsdPrim&)
		{
			return false;
		},
		ExcludeSchemaTypes
	);
}

TArray<TUsdStore<pxr::UsdPrim>> UsdUtils::GetAllPrimsOfType(
	const pxr::UsdPrim& StartPrim,
	const pxr::TfType& SchemaType,
	TFunction<bool(const pxr::UsdPrim&)> PruneChildren,
	const TArray<TUsdStore<pxr::TfType>>& ExcludeSchemaTypes
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UsdUtils::GetAllPrimsOfType);

	TArray<TUsdStore<pxr::UsdPrim>> Result;

	pxr::UsdPrimRange PrimRange(StartPrim, pxr::UsdTraverseInstanceProxies());

	for (pxr::UsdPrimRange::iterator PrimRangeIt = PrimRange.begin(); PrimRangeIt != PrimRange.end(); ++PrimRangeIt)
	{
		bool bIsExcluded = false;

		for (const TUsdStore<pxr::TfType>& SchemaToExclude : ExcludeSchemaTypes)
		{
			if (PrimRangeIt->IsA(SchemaToExclude.Get()))
			{
				bIsExcluded = true;
				break;
			}
		}

		if (!bIsExcluded && PrimRangeIt->IsA(SchemaType))
		{
			Result.Add(*PrimRangeIt);
		}

		if (bIsExcluded || PruneChildren(*PrimRangeIt))
		{
			PrimRangeIt.PruneChildren();
		}
	}

	return Result;
}

FString UsdUtils::GetAssetPathFromPrimPath(const FString& RootContentPath, const pxr::UsdPrim& Prim)
{
	FString FinalPath;

	auto GetEnclosingModelPrim = [](const pxr::UsdPrim& Prim) -> pxr::UsdPrim
	{
		pxr::UsdPrim ModelPrim = Prim.GetParent();

		while (ModelPrim)
		{
			if (IUsdPrim::IsKindChildOf(ModelPrim, "model"))
			{
				break;
			}
			else
			{
				ModelPrim = ModelPrim.GetParent();
			}
		}

		return ModelPrim.IsValid() ? ModelPrim : Prim;
	};

	const pxr::UsdPrim& ModelPrim = GetEnclosingModelPrim(Prim);

	const FString RawPrimName = UsdToUnreal::ConvertString(Prim.GetName());

	pxr::UsdModelAPI ModelApi = pxr::UsdModelAPI(ModelPrim);

	std::string RawAssetName;
	ModelApi.GetAssetName(&RawAssetName);

	FString AssetName = UsdToUnreal::ConvertString(RawAssetName);
	FString MeshName = UsdUnreal::ObjectUtils::SanitizeObjectName(RawPrimName);

	FString USDPath = UsdToUnreal::ConvertString(Prim.GetPrimPath().GetString().c_str());

	pxr::SdfAssetPath AssetPath;
	if (ModelApi.GetAssetIdentifier(&AssetPath))
	{
		std::string AssetIdentifier = AssetPath.GetAssetPath();
		USDPath = UsdToUnreal::ConvertString(AssetIdentifier.c_str());

		USDPath = FPaths::ConvertRelativePathToFull(RootContentPath, USDPath);

		FPackageName::TryConvertFilenameToLongPackageName(USDPath, USDPath);
		USDPath.RemoveFromEnd(AssetName);
	}

	FString VariantName;

	if (ModelPrim.HasVariantSets())
	{
		pxr::UsdVariantSet ModelVariantSet = ModelPrim.GetVariantSet("modelingVariant");
		if (ModelVariantSet.IsValid())
		{
			std::string VariantSelection = ModelVariantSet.GetVariantSelection();

			if (VariantSelection.length() > 0)
			{
				VariantName = UsdToUnreal::ConvertString(VariantSelection.c_str());
			}
		}
	}

	if (!VariantName.IsEmpty())
	{
		USDPath = USDPath / VariantName;
	}

	USDPath.RemoveFromStart(TEXT("/"));
	USDPath.RemoveFromEnd(RawPrimName);
	FinalPath /= (USDPath / MeshName);

	return FinalPath;
}
#endif	  // #if USE_USD_SDK

FString UsdUtils::AddHashToPrimPath(const FString& PrimPath)
{
	// Pad to exactly 10 digits, so it's easy to strip
	// Adding the underscore makes it so that we can use this even on the pseudoroot path "/", without
	// making an invalid SdfPath in the process (as something like /6154761547 is not a valid prim path)
	return FString::Printf(TEXT("%s_%010u"), *PrimPath, UE::HashStringFNV1a32(MakeStringView(PrimPath)));
}

FString UsdUtils::RemoveHashFromPrimPath(const FString& HashedPrimPath)
{
	// Make sure this matches the number of digits on the Printf format above (+ underscore)!
	return HashedPrimPath.LeftChop(11);
}

TArray<UE::FUsdPrim> UsdUtils::GetAllPrimsOfType(const UE::FUsdPrim& StartPrim, const TCHAR* SchemaName)
{
	return GetAllPrimsOfType(
		StartPrim,
		SchemaName,
		[](const UE::FUsdPrim&)
		{
			return false;
		}
	);
}

TArray<UE::FUsdPrim> UsdUtils::GetAllPrimsOfType(
	const UE::FUsdPrim& StartPrim,
	const TCHAR* SchemaName,
	TFunction<bool(const UE::FUsdPrim&)> PruneChildren,
	const TArray<const TCHAR*>& ExcludeSchemaNames
)
{
	TArray<UE::FUsdPrim> Result;

#if USE_USD_SDK
	const pxr::TfType SchemaType = pxr::TfType::FindByName(TCHAR_TO_UTF8(SchemaName));

	TArray<TUsdStore<pxr::TfType>> ExcludeSchemaTypes;
	ExcludeSchemaTypes.Reserve(ExcludeSchemaNames.Num());
	for (const TCHAR* ExcludeSchemaName : ExcludeSchemaNames)
	{
		ExcludeSchemaTypes.Add(pxr::TfType(pxr::TfType::FindByName(TCHAR_TO_UTF8(ExcludeSchemaName))));
	}

	auto UsdPruneChildren = [&PruneChildren](const pxr::UsdPrim& ChildPrim) -> bool
	{
		return PruneChildren(UE::FUsdPrim(ChildPrim));
	};

	TArray<TUsdStore<pxr::UsdPrim>> UsdResult = GetAllPrimsOfType(StartPrim, SchemaType, UsdPruneChildren, ExcludeSchemaTypes);

	for (const TUsdStore<pxr::UsdPrim>& Prim : UsdResult)
	{
		Result.Emplace(Prim.Get());
	}
#endif	  // #if USE_USD_SDK

	return Result;
}

double UsdUtils::GetDefaultTimeCode()
{
#if USE_USD_SDK
	return pxr::UsdTimeCode::Default().GetValue();
#else
	return 0.0;
#endif
}

double UsdUtils::GetEarliestTimeCode()
{
#if USE_USD_SDK
	return pxr::UsdTimeCode::EarliestTime().GetValue();
#else
	return 0.0;
#endif
}

#if USE_USD_SDK
FString UsdUtils::GetAssetHashPrefix(const pxr::UsdPrim& PrimForAsset, bool bShareAssetsForIdenticalPrims)
{
	if (!PrimForAsset || bShareAssetsForIdenticalPrims)
	{
		return FString{};
	}

	FString PrimPath = *UsdToUnreal::ConvertPath(PrimForAsset.GetPrimPath());
	FString StageIdentifier = *UsdToUnreal::ConvertString(PrimForAsset.GetStage()->GetRootLayer()->GetIdentifier());

	FSHA1 SHA1;
	SHA1.UpdateWithString(*PrimPath, PrimPath.Len());
	SHA1.UpdateWithString(*StageIdentifier, StageIdentifier.Len());

	FSHAHash Hash;
	SHA1.Final();
	SHA1.GetHash(&Hash.Hash[0]);
	return Hash.ToString() + TEXT("_");
}
#endif	  // WITH_EDITOR

namespace UE::UsdConversionUtils::Private
{
#if USE_USD_SDK
	void HandleTypeNameAndAddReference(
		const pxr::UsdPrim& ReferencerPrim,
		pxr::SdfPrimSpecHandle TargetPrimSpec,
		TOptional<EReferencerTypeHandling> ReferencerTypeHandling,
		TFunction<void()> AddReferenceOrPayloadLambda
	)
	{
		if (!ReferencerPrim || !TargetPrimSpec || !AddReferenceOrPayloadLambda)
		{
			return;
		}

		FScopedUsdAllocs Allocs;

		pxr::TfToken ReferencerTypeName = ReferencerPrim.GetTypeName();
		pxr::TfToken TargetTypeName = TargetPrimSpec->GetTypeName();

		// Check if we need to do anything special
		bool bNeedHandling = false;
		bool bUnknownTargetType = false;
		if (!ReferencerTypeName.IsEmpty() && !TargetTypeName.IsEmpty())
		{
			pxr::TfType TargetPrimType = pxr::UsdSchemaRegistry::GetTypeFromName(TargetTypeName);
			if (TargetPrimType.IsUnknown())
			{
				bNeedHandling = true;
				bUnknownTargetType = true;
			}
			else if (!ReferencerPrim.IsA(TargetPrimType))
			{
				bNeedHandling = true;
			}
		}
		if (!bNeedHandling)
		{
			// The schemas already match just fine
			AddReferenceOrPayloadLambda();
			return;
		}

		// Get what we actually need to do
		EReferencerTypeHandling Handling = EReferencerTypeHandling::MatchReferencedType;
		if (ReferencerTypeHandling.IsSet())
		{
			Handling = ReferencerTypeHandling.GetValue();
		}
		else if (const UUsdProjectSettings* Settings = GetDefault<UUsdProjectSettings>())
		{
			Handling = Settings->ReferencerTypeHandling;
		}

		// Show the prompt and update 'Handling' to something else if we can
		if (Handling == EReferencerTypeHandling::ShowPrompt)
		{
#if WITH_EDITOR
			const FText DialogText = FText::Format(
				LOCTEXT(
					"MismatchedTypeNamesSubText",
					"Tried to add a reference or payload from prim '{0}' with type '{1}', to target prim '{2}' with type '{3}'.\n\nSince these types are not identical, it is possible that the composed prim will not have the intended behaviour.\n\nHow do you wish to proceed?"
				),
				FText::FromString(UsdToUnreal::ConvertPath(ReferencerPrim.GetPrimPath())),
				FText::FromString(UsdToUnreal::ConvertToken(ReferencerTypeName)),
				FText::FromString(UsdToUnreal::ConvertPath(TargetPrimSpec->GetPath())),
				FText::FromString(UsdToUnreal::ConvertToken(TargetTypeName))
			);

			FScopedUnrealAllocs UEAllocs;

			// Dialog has to be on another module as this one is RTTI enabled, which means Slate code won't compile on
			// some targets (Mac Arm64 for example)
			IUsdUtilitiesModule& UtilitiesModule = FModuleManager::Get().LoadModuleChecked<IUsdUtilitiesModule>(TEXT("UsdUtilities"));
			if (UtilitiesModule.OnReferenceHandlingDialog.IsBound())
			{
				EReferencerTypeHandling ChosenHandling = Handling;
				const bool bAccepted = UtilitiesModule.OnReferenceHandlingDialog.Execute(DialogText, ChosenHandling);
				if (bAccepted)
				{
					Handling = ChosenHandling;
				}
			}
#else
			Handling = EReferencerTypeHandling::ClearReferencerType;
#endif	  // WITH EDITOR
		}

		switch (Handling)
		{
			default:
			case EReferencerTypeHandling::Ignore:
			{
				AddReferenceOrPayloadLambda();
				break;
			}
			case EReferencerTypeHandling::MatchReferencedType:
			{
				pxr::SdfChangeBlock Block;

				if (bUnknownTargetType)
				{
					USD_LOG_USERWARNING(FText::Format(
						LOCTEXT(
							"MatchReferencerFail",
							"Failed to match the referenced type when adding a reference or payload to prim '{0}', as the target prim spec '{1}' has an unknown type '{2}'! The referencer type will be cleared instead."
						),
						FText::FromString(UsdToUnreal::ConvertPath(ReferencerPrim.GetPrimPath())),
						FText::FromString(UsdToUnreal::ConvertPath(TargetPrimSpec->GetPath())),
						FText::FromString(UsdToUnreal::ConvertToken(TargetTypeName))
					));
					ReferencerPrim.ClearTypeName();
				}
				else
				{
					ReferencerPrim.SetTypeName(TargetTypeName);
				}

				AddReferenceOrPayloadLambda();
				break;
			}
			case EReferencerTypeHandling::ClearReferencerType:
			{
				pxr::SdfChangeBlock Block;
				ReferencerPrim.ClearTypeName();
				AddReferenceOrPayloadLambda();
				break;
			}
			case EReferencerTypeHandling::ShowPrompt:
			{
				// We showed the dialog but didn't choose any handling --> Do nothing
				break;
			}
		}
	}

	void AddReferenceOrPayload(
		bool bIsReference,
		const UE::FUsdPrim& Prim,
		const TCHAR* AbsoluteFilePath,
		const UE::FSdfPath& TargetPrimPath,
		double TimeCodeOffset,
		double TimeCodeScale,
		TOptional<EReferencerTypeHandling> ReferencerTypeHandling
	)
	{
		if (!Prim || !AbsoluteFilePath)
		{
			return;
		}

		FString RelativePath = AbsoluteFilePath;
		if (RelativePath.IsEmpty() && !TargetPrimPath.IsPrimPath())
		{
			// Nothing to target
			return;
		}

		FScopedUsdAllocs UsdAllocs;

		pxr::UsdPrim UsdPrim(Prim);

		pxr::UsdStageRefPtr UsdStage = UsdPrim.GetStage();
		if (!UsdStage)
		{
			return;
		}

		// Turn our layer path into a relative one
		if (!RelativePath.IsEmpty())
		{
			pxr::SdfLayerHandle EditLayer = UsdPrim.GetStage()->GetEditTarget().GetLayer();

			std::string RepositoryPath = EditLayer->GetRepositoryPath().empty() ? EditLayer->GetRealPath() : EditLayer->GetRepositoryPath();

			// If we're editing an in-memory stage our root layer may not have a path yet
			// Giving an empty InRelativeTo to MakePathRelativeTo causes it to use the engine binary
			if (!RepositoryPath.empty())
			{
				FString LayerAbsolutePath = UsdToUnreal::ConvertString(RepositoryPath);
				FPaths::MakePathRelativeTo(RelativePath, *LayerAbsolutePath);
			}
		}

		// Get the target layer
		pxr::SdfLayerRefPtr TargetLayer;
		bool bIsInternalReference = false;
		if (RelativePath.IsEmpty())
		{
			TargetLayer = UsdStage->GetRootLayer();
			bIsInternalReference = true;
		}
		else
		{
			TargetLayer = pxr::SdfLayer::FindOrOpen(UnrealToUsd::ConvertString(AbsoluteFilePath).Get());
		}
		if (!TargetLayer)
		{
			return;
		}

		// Get the target prim spec we want to reference
		pxr::SdfPrimSpecHandle TargetPrimSpec = TargetLayer->GetPrimAtPath(TargetPrimPath);
		if ((TargetPrimPath.IsEmpty() || !TargetPrimSpec) && TargetLayer->HasDefaultPrim())
		{
			TargetPrimSpec = TargetLayer->GetPrimAtPath(pxr::SdfPath::AbsoluteRootPath().AppendChild(TargetLayer->GetDefaultPrim()));
		}

		// We want to output no path for the prim if we received it as such, even if we already know what the path to the
		// default prim is, so that the authored reference doesn't actually specify any prim name and just refers to the
		// default prim by default. Otherwise if the default prim of the layer changed, we wouldn't update to the new prim
		pxr::SdfPath FinalPrimPath = TargetPrimSpec && !TargetPrimPath.IsEmpty() ? TargetPrimSpec->GetPath() : pxr::SdfPath{};
		std::string RelativeLayerPath = bIsInternalReference ? std::string() : UnrealToUsd::ConvertString(*RelativePath).Get();

		TFunction<void()> AddReferenceOrPayloadLambda = nullptr;
		if (bIsReference)
		{
			AddReferenceOrPayloadLambda = [UsdPrim, &RelativeLayerPath, &FinalPrimPath, TimeCodeOffset, TimeCodeScale]()
			{
				pxr::UsdReferences References = UsdPrim.GetReferences();
				References.AddReference(RelativeLayerPath, FinalPrimPath, pxr::SdfLayerOffset{TimeCodeOffset, TimeCodeScale});
			};
		}
		else	// It's a payload instead
		{
			AddReferenceOrPayloadLambda = [UsdPrim, &RelativeLayerPath, &FinalPrimPath, TimeCodeOffset, TimeCodeScale]()
			{
				pxr::UsdPayloads Payloads = UsdPrim.GetPayloads();
				Payloads.AddPayload(RelativeLayerPath, FinalPrimPath, pxr::SdfLayerOffset{TimeCodeOffset, TimeCodeScale});
			};
		}

		HandleTypeNameAndAddReference(Prim, TargetPrimSpec, ReferencerTypeHandling, AddReferenceOrPayloadLambda);
	}
#endif	  // #if USE_USD_SDK
}	 // namespace UE::UsdConversionUtils::Private

void UsdUtils::AddReference(
	UE::FUsdPrim& Prim,
	const TCHAR* AbsoluteFilePath,
	const UE::FSdfPath& TargetPrimPath,
	double TimeCodeOffset,
	double TimeCodeScale
)
{
	const EReferencerTypeHandling Handling = EReferencerTypeHandling::MatchReferencedType;
	AddReference(Prim, AbsoluteFilePath, Handling, TargetPrimPath, TimeCodeOffset, TimeCodeScale);
}

void UsdUtils::AddReference(
	UE::FUsdPrim& Prim,
	const TCHAR* AbsoluteFilePath,
	TOptional<EReferencerTypeHandling> ReferencerTypeHandling,
	const UE::FSdfPath& TargetPrimPath,
	double TimeCodeOffset,
	double TimeCodeScale
)
{
#if USE_USD_SDK
	const bool bIsReference = true;
	UE::UsdConversionUtils::Private::AddReferenceOrPayload(
		bIsReference,
		Prim,
		AbsoluteFilePath,
		TargetPrimPath,
		TimeCodeOffset,
		TimeCodeScale,
		ReferencerTypeHandling
	);
#endif	  // #if USE_USD_SDK
}

bool UsdUtils::GetReferenceFilePath(const UE::FUsdPrim& Prim, const FString& FileExtension, FString& OutReferenceFilePath)
{
#if USE_USD_SDK
	FScopedUsdAllocs UsdAllocs;

	pxr::UsdPrimCompositionQuery PrimCompositionQuery = pxr::UsdPrimCompositionQuery::GetDirectReferences(Prim);
	for (const pxr::UsdPrimCompositionQueryArc& CompositionArc : PrimCompositionQuery.GetCompositionArcs())
	{
		if (CompositionArc.GetArcType() == pxr::PcpArcTypeReference)
		{
			pxr::SdfReferenceEditorProxy ReferenceEditor;
			pxr::SdfReference UsdReference;

			if (CompositionArc.GetIntroducingListEditor(&ReferenceEditor, &UsdReference))
			{
				FString AbsoluteFilePath = UsdToUnreal::ConvertString(UsdReference.GetAssetPath());

				FString Extension = FPaths::GetExtension(AbsoluteFilePath);
				if (Extension == FileExtension && FPaths::FileExists(AbsoluteFilePath))
				{
					OutReferenceFilePath = AbsoluteFilePath;
					return true;
				}
			}
		}
	}
#endif	  // #if USE_USD_SDK

	return false;
}

TArray<FString> UsdUtils::GetReferenceOrPayloadFilePaths(const UE::FUsdPrim& Prim)
{
	TArray<FString> FilePaths;

#if USE_USD_SDK
	pxr::UsdPrimCompositionQuery::Filter Filter;
	Filter.dependencyTypeFilter = pxr::UsdPrimCompositionQuery::DependencyTypeFilter::Direct;

	pxr::UsdPrimCompositionQuery PrimCompositionQuery = pxr::UsdPrimCompositionQuery(Prim, Filter);
	for (const pxr::UsdPrimCompositionQueryArc& CompositionArc : PrimCompositionQuery.GetCompositionArcs())
	{
		if (CompositionArc.GetArcType() == pxr::PcpArcTypeReference)
		{
			pxr::SdfReferenceEditorProxy ReferenceEditor;
			pxr::SdfReference UsdReference;

			if (CompositionArc.GetIntroducingListEditor(&ReferenceEditor, &UsdReference))
			{
				FString FilePath = UsdToUnreal::ConvertString(UsdReference.GetAssetPath());
				FilePaths.Add(FilePath);
			}
		}
		else if (CompositionArc.GetArcType() == pxr::PcpArcTypePayload)
		{
			pxr::SdfPayloadEditorProxy PayloadEditor;
			pxr::SdfPayload UsdPayload;

			if (CompositionArc.GetIntroducingListEditor(&PayloadEditor, &UsdPayload))
			{
				FString FilePath = UsdToUnreal::ConvertString(UsdPayload.GetAssetPath());
				FilePaths.Add(FilePath);
			}
		}
	}
#endif	  // #if USE_USD_SDK

	return FilePaths;
}

void UsdUtils::AddPayload(
	UE::FUsdPrim& Prim,
	const TCHAR* AbsoluteFilePath,
	const UE::FSdfPath& TargetPrimPath,
	double TimeCodeOffset,
	double TimeCodeScale
)
{
	const EReferencerTypeHandling Handling = EReferencerTypeHandling::MatchReferencedType;
	AddPayload(Prim, AbsoluteFilePath, Handling, TargetPrimPath, TimeCodeOffset, TimeCodeScale);
}

void UsdUtils::AddPayload(
	UE::FUsdPrim& Prim,
	const TCHAR* AbsoluteFilePath,
	TOptional<EReferencerTypeHandling> ReferencerTypeHandling,
	const UE::FSdfPath& TargetPrimPath,
	double TimeCodeOffset,
	double TimeCodeScale
)
{
#if USE_USD_SDK
	const bool bIsReference = false;
	UE::UsdConversionUtils::Private::AddReferenceOrPayload(
		bIsReference,
		Prim,
		AbsoluteFilePath,
		TargetPrimPath,
		TimeCodeOffset,
		TimeCodeScale,
		ReferencerTypeHandling
	);
#endif	  // #if USE_USD_SDK
}

bool UsdUtils::RenamePrim(UE::FUsdPrim& Prim, const TCHAR* NewPrimName)
{
#if USE_USD_SDK
	FScopedUsdAllocs UsdAllocs;

	if (!Prim || !NewPrimName)
	{
		return false;
	}

	if (Prim.GetName() == FName(NewPrimName))
	{
		return false;
	}

	pxr::UsdPrim PxrUsdPrim{Prim};
	pxr::UsdStageRefPtr PxrUsdStage{Prim.GetStage()};
	if (!PxrUsdStage)
	{
		return false;
	}

	pxr::TfToken NewNameToken = UnrealToUsd::ConvertToken(NewPrimName).Get();
	pxr::SdfPath TargetPath = PxrUsdPrim.GetPrimPath().ReplaceName(NewNameToken);

	std::unordered_set<std::string> LocalLayerIdentifiers;
	const bool bIncludeSessionLayers = true;
	for (const pxr::SdfLayerHandle& Handle : PxrUsdStage->GetLayerStack(bIncludeSessionLayers))
	{
		LocalLayerIdentifiers.insert(Handle->GetIdentifier());
	}

	std::vector<pxr::SdfPrimSpecHandle> SpecStack = PxrUsdPrim.GetPrimStack();
	TArray<TPair<pxr::SdfLayerRefPtr, pxr::SdfBatchNamespaceEdit>> Edits;

	// Check if we can apply this rename, and collect error messages if we can't
	// We will only rename if we can change all specs, or else we'd split the prim
	TArray<FString> ErrorMessages;
	pxr::SdfNamespaceEditDetailVector Details;
	int32 LastDetailsSize = 0;
	bool bCanApply = true;
	for (const pxr::SdfPrimSpecHandle& Spec : SpecStack)
	{
		if (!Spec)
		{
			continue;
		}

		pxr::SdfPath SpecPath = Spec->GetPath();
		if (!SpecPath.IsPrimPath())
		{
			// Especially when it comes to variants, we can have many different specs for a prim.
			// e.g. we can simultaneously have "/Prim{Varset=}", "/Prim{Varset=Var}" and "/Prim" in there, and
			// we will fail to do anything if these paths are not prim paths
			continue;
		}

		pxr::SdfLayerRefPtr SpecLayer = Spec->GetLayer();

		// We should only rename specs on layers that are in the stage's *local* layer stack (which will include root, sublayers and
		// session layers). We shouldn't rename any spec that is created due to references/payloads to other layers, because if we do
		// we'll end up renaming the prims within those layers too, which is not what we want: For reference/payloads it's as if
		// we're just consuming the *contents* of the referenced prim, but we don't want to affect it. Another more drastic example:
		// if we were to remove the referencer prim, we don't really want to delete the referenced prim within its layer
		if (LocalLayerIdentifiers.count(SpecLayer->GetIdentifier()) == 0)
		{
			continue;
		}

		pxr::SdfBatchNamespaceEdit BatchEdit;
		BatchEdit.Add(pxr::SdfNamespaceEdit::Rename(SpecPath, NewNameToken));

		int32 CurrentNumDetails = Details.size();
		if (SpecLayer->CanApply(BatchEdit, &Details) != pxr::SdfNamespaceEditDetail::Result::Okay)
		{
			FString LayerIdentifier = UsdToUnreal::ConvertString(SpecLayer->GetIdentifier());

			// This error pushed something new into the Details vector. Get it as an error message
			FString ErrorMessage;
			if (CurrentNumDetails != LastDetailsSize)
			{
				ErrorMessage = UsdToUnreal::ConvertString(Details[CurrentNumDetails - 1].reason);
			}

			ErrorMessages.Add(FString::Printf(TEXT("\t%s: %s"), *LayerIdentifier, *ErrorMessage));
			bCanApply = false;
			// Don't break so we can collect all error messages
		}

		LastDetailsSize = CurrentNumDetails;
		Edits.Add(TPair<pxr::SdfLayerRefPtr, pxr::SdfBatchNamespaceEdit>{SpecLayer, BatchEdit});
	}

	if (!bCanApply)
	{
		USD_LOG_ERROR(
			TEXT("Failed to rename prim with path '%s' to name '%s'. Errors:\n%s"),
			*Prim.GetPrimPath().GetString(),
			NewPrimName,
			*FString::Join(ErrorMessages, TEXT("\n"))
		);

		return false;
	}

	// Actually apply the renames
	{
		pxr::SdfChangeBlock Block;

		for (const TPair<pxr::SdfLayerRefPtr, pxr::SdfBatchNamespaceEdit>& Pair : Edits)
		{
			const pxr::SdfLayerRefPtr& Layer = Pair.Key;
			const pxr::SdfBatchNamespaceEdit& Edit = Pair.Value;

			// Make sure that if the renamed prim is the layer's default prim, we also update that to match the
			// prim's new name
			pxr::UsdPrim ParentPrim = PxrUsdPrim.GetParent();
			const bool bNeedToRenameDefaultPrim = ParentPrim && ParentPrim.IsPseudoRoot() && (PxrUsdPrim.GetName() == Layer->GetDefaultPrim());

			if (!Layer->Apply(Edit))
			{
				// This should not be happening since CanApply was true, so stop doing whatever it is we're doing
				USD_LOG_ERROR(
					TEXT("Failed to rename prim with path '%s' to name '%s' in layer '%s'"),
					*Prim.GetPrimPath().GetString(),
					NewPrimName,
					*UsdToUnreal::ConvertString(Layer->GetIdentifier())
				);

				return false;
			}

			if (bNeedToRenameDefaultPrim)
			{
				Layer->SetDefaultPrim(NewNameToken);
			}
		}
	}

	// For whatever reason, if the renamed prim is within a variant set it will be left inactive (i.e. effectively deleted) post-rename by USD.
	// Here we override that with a SetActive opinion on the session layer, which will also trigger a new resync of that prim.
	//
	// We must send a separate notice for this (which is why this function can't be inside a change block) for two reasons:
	// - In order to let the transactor know that this edit is done on the session layer (so that we can have our active=true opinion there and not
	// save it to disk);
	// - Because after we apply the rename, usd *needs* to responds to notices in order to make the target path valid again. Until
	//   it does so, we can't Get/Override/Define a prim at the target path at all, and so can't set it to active.
	//
	// We can't do this *before* we rename because if we already have a prim defined/overriden on "/Root/Target", then we
	// can't apply a rename from a prim onto "/Root/Target": Meaning we'd lose all extra data we have on the prim on the session layer.
	{
		pxr::UsdEditContext EditContext{PxrUsdStage, PxrUsdStage->GetSessionLayer()};

		if (pxr::UsdPrim PostRenamePrim = PxrUsdStage->OverridePrim(TargetPath))
		{
			// We need to toggle it back and forth because whenever we undo a rename we'll rename our spec on the session layer
			// back to the original path, and that spec *already* has an active=true opinion that we set during the first rename.
			// This means that just setting it to active here wouldn't send any notice (because it already is). We need a new notice
			// to update to the fact that the child prim is now active again (the rename notice is a resync, but it already comes with the prim set to
			// inactive)
			pxr::SdfChangeBlock Block;
			const bool bActive = true;
			PostRenamePrim.SetActive(!bActive);
			PostRenamePrim.SetActive(bActive);
		}
	}

	return true;
#else
	return false;
#endif	  // #if USE_USD_SDK
}

#if USE_USD_SDK
FString UsdUtils::GetValidChildName(FString InName, const pxr::UsdPrim& ParentPrim)
{
	if (!ParentPrim)
	{
		return {};
	}

	FScopedUsdAllocs Allocs;

	TSet<FString> UsedNames;
	for (const pxr::UsdPrim& Child : ParentPrim.GetChildren())
	{
		UsedNames.Add(UsdToUnreal::ConvertToken(Child.GetName()));
	}

	return UsdUnreal::ObjectUtils::GetUniqueName(SanitizeUsdIdentifier(*InName), UsedNames);
}
#endif	  // USE_USD_SDK

FString UsdUtils::SanitizeUsdIdentifier(const TCHAR* InIdentifier)
{
#if USE_USD_SDK
	FScopedUsdAllocs Allocs;

	// We used to use pxr::TfMakeValidIdentifier() here, but it seems that that sanitizes away UTF8
	// characters, which are otherwise supported.
	//
	// Now we implement something based on _IsValidIdentifier() from USD's sdf\path.cpp instead,
	// as we need to actually produce a valid identifier and not only check if it's valid or not

	std::string Identifier = UnrealToUsd::ConvertString(InIdentifier).Get();
	if (Identifier.empty())
	{
		Identifier = "_";
	}
	else
	{
		std::string Sanitized;
		Sanitized.reserve(Identifier.size());
		std::stringstream Stream{Sanitized};

		std::string_view View{Identifier};
		pxr::TfUtf8CodePointIterator Iter{std::cbegin(View), std::cend(View)};

		// First codepoint has a different rule
		const pxr::TfUtf8CodePoint& FirstCodePoint = *Iter;
		if (FirstCodePoint == pxr::TfUtf8CodePointFromAscii('_') || pxr::TfIsUtf8CodePointXidStart(FirstCodePoint))
		{
			Stream << FirstCodePoint;
		}
		else
		{
			Stream << '_';
		}

		// Sanitize remaining codepoints
		for (++Iter; Iter != pxr::TfUtf8CodePointIterator::PastTheEndSentinel{}; ++Iter)
		{
			const pxr::TfUtf8CodePoint& CodePoint = *Iter;
			if (TfIsUtf8CodePointXidContinue(CodePoint))
			{
				Stream << CodePoint;
			}
			else
			{
				Stream << '_';
			}
		}

		Identifier = Stream.str();
	}

	return UsdToUnreal::ConvertString(Identifier);
#else
	return InIdentifier;
#endif	  // USE_USD_SDK
}

void UsdUtils::MakeVisible(UE::FUsdPrim& Prim, double TimeCode)
{
#if USE_USD_SDK
	FScopedUsdAllocs Allocs;

	pxr::UsdPrim PxrUsdPrim{Prim};
	if (pxr::UsdGeomImageable Imageable{PxrUsdPrim})
	{
		Imageable.MakeVisible(TimeCode);
	}
#endif	  // USE_USD_SDK
}

void UsdUtils::MakeInvisible(UE::FUsdPrim& Prim, double TimeCode)
{
#if USE_USD_SDK
	FScopedUsdAllocs Allocs;

	pxr::UsdPrim PxrUsdPrim{Prim};
	if (pxr::UsdGeomImageable Imageable{PxrUsdPrim})
	{
		Imageable.MakeInvisible(TimeCode);
	}
#endif	  // USE_USD_SDK
}

bool UsdUtils::IsVisible(const UE::FUsdPrim& Prim, double TimeCode)
{
#if USE_USD_SDK
	FScopedUsdAllocs Allocs;

	pxr::UsdPrim PxrUsdPrim{Prim};
	if (pxr::UsdGeomImageable Imageable{PxrUsdPrim})
	{
		return Imageable.ComputeVisibility(TimeCode) == pxr::UsdGeomTokens->inherited;
	}

	return true;
#else
	return false;
#endif	  // USE_USD_SDK
}

bool UsdUtils::HasInheritedVisibility(const UE::FUsdPrim& Prim, double TimeCode)
{
#if USE_USD_SDK
	FScopedUsdAllocs Allocs;

	pxr::UsdPrim PxrUsdPrim{Prim};
	if (pxr::UsdGeomImageable Imageable{PxrUsdPrim})
	{
		if (pxr::UsdAttribute VisibilityAttr = Imageable.GetVisibilityAttr())
		{
			pxr::TfToken Visibility;
			if (!VisibilityAttr.Get<pxr::TfToken>(&Visibility, TimeCode))
			{
				return true;
			}

			return Visibility == pxr::UsdGeomTokens->inherited;
		}
	}

	// If it doesn't have the attribute the default is for it to be 'inherited'
	return true;
#else
	return false;
#endif	  // USE_USD_SDK
}

bool UsdUtils::HasInvisibleParent(const UE::FUsdPrim& Prim, const UE::FUsdPrim& RootPrim, double TimeCode)
{
#if USE_USD_SDK
	FScopedUsdAllocs Allocs;

	pxr::UsdPrim PxrUsdPrim{Prim};
	pxr::UsdPrim Parent = PxrUsdPrim.GetParent();

	while (Parent && Parent != RootPrim)
	{
		if (pxr::UsdGeomImageable Imageable{Parent})
		{
			if (pxr::UsdAttribute VisibilityAttr = Imageable.GetVisibilityAttr())
			{
				pxr::TfToken Visibility;
				if (VisibilityAttr.Get<pxr::TfToken>(&Visibility, TimeCode))
				{
					if (Visibility == pxr::UsdGeomTokens->invisible)
					{
						return true;
					}
				}
			}
		}

		Parent = Parent.GetParent();
	}
#endif	  // USE_USD_SDK

	return false;
}

TArray<UE::FUsdPrim> UsdUtils::GetVisibleChildren(const UE::FUsdPrim& Prim, EUsdPurpose AllowedPurposes)
{
	TArray<UE::FUsdPrim> VisiblePrims;

#if USE_USD_SDK
	FScopedUsdAllocs UsdAllocs;

	TFunction<void(const pxr::UsdPrim& Prim)> RecursivelyCollectVisibleMeshes;
	RecursivelyCollectVisibleMeshes = [&RecursivelyCollectVisibleMeshes, &VisiblePrims, AllowedPurposes](const pxr::UsdPrim& Prim)
	{
		if (!Prim)
		{
			return;
		}

		if (pxr::UsdGeomImageable UsdGeomImageable = pxr::UsdGeomImageable(Prim))
		{
			if (pxr::UsdAttribute VisibilityAttr = UsdGeomImageable.GetVisibilityAttr())
			{
				pxr::TfToken VisibilityToken;
				if (VisibilityAttr.Get(&VisibilityToken) && VisibilityToken == pxr::UsdGeomTokens->invisible)
				{
					// We don't propagate the (in)visibility token, we just flat out stop recursing instead
					return;
				}
			}
		}

		// We want to only collect meshes from visible purposes, but unlike for the visibility attribute itself,
		// we don't want to stop traversal due to a hidden purpose, as a child prim may have a different purpose
		// to its parent
		if (EnumHasAllFlags(AllowedPurposes, IUsdPrim::GetPurpose(Prim)))
		{
			VisiblePrims.Add(UE::FUsdPrim{Prim});
		}

		for (const pxr::UsdPrim& ChildPrim : Prim.GetFilteredChildren(pxr::UsdTraverseInstanceProxies()))
		{
			RecursivelyCollectVisibleMeshes(ChildPrim);
		}
	};
	RecursivelyCollectVisibleMeshes(Prim);
#endif	  // USE_USD_SDK

	return VisiblePrims;
}

UE::FSdfPath UsdUtils::GetPrimSpecPathForLayer(const UE::FUsdPrim& Prim, const UE::FSdfLayer& Layer)
{
	UE::FSdfPath Result;
#if USE_USD_SDK
	FScopedUsdAllocs Allocs;

	pxr::UsdPrim UsdPrim{Prim};
	pxr::SdfLayerRefPtr UsdLayer{Layer};
	if (!UsdPrim || !UsdLayer)
	{
		return Result;
	}

	// We may have multiple specs in the same layer if we're within a variant set (e.g "/Root/Parent/Child" and
	// "/Root{Varset=Var}Parent/Child{ChildSet=ChildVar}" and "/Root{Varset=Var}Parent/Child").
	// This function needs to return a prim path with all of its variant selections (i.e. the last example above)
	std::size_t LargestPathLength = 0;
	for (const pxr::SdfPrimSpecHandle& Spec : UsdPrim.GetPrimStack())
	{
		if (!Spec)
		{
			continue;
		}

		pxr::SdfPath SpecPath = Spec->GetPath();
		if (!SpecPath.IsPrimPath())
		{
			continue;
		}

		if (Spec->GetLayer() == UsdLayer)
		{
			const std::size_t NewPathLength = Spec->GetPath().GetString().length();
			if (NewPathLength > LargestPathLength)
			{
				Result = UE::FSdfPath{SpecPath};
			}
		}
	}

#endif	  // USE_USD_SDK
	return Result;
}

USDUTILITIES_API void UsdUtils::RemoveAllLocalPrimSpecs(const UE::FUsdPrim& Prim, const UE::FSdfLayer& Layer)
{
#if USE_USD_SDK
	FScopedUsdAllocs Allocs;

	pxr::UsdPrim UsdPrim{Prim};
	if (!UsdPrim)
	{
		return;
	}

	pxr::SdfLayerRefPtr UsdLayer{Layer};
	pxr::UsdStageRefPtr UsdStage = UsdPrim.GetStage();

	std::unordered_set<std::string> LocalLayerIdentifiers;

	// We'll want to remove specs from the entire stage. We need to be careful though to only remove specs from the
	// local layer stack. If a prim within the stage has a reference/payload to another layer and we remove the
	// referencer prim, we don't want to end up removing the referenced/payload prim within its own layer too.
	if (!UsdLayer)
	{
		const bool bIncludeSessionLayers = true;
		for (const pxr::SdfLayerHandle& Handle : UsdStage->GetLayerStack(bIncludeSessionLayers))
		{
			LocalLayerIdentifiers.insert(Handle->GetIdentifier());
		}
	}

	const pxr::SdfPath TargetPath = UsdPrim.GetPrimPath();

	for (const pxr::SdfPrimSpecHandle& Spec : UsdPrim.GetPrimStack())
	{
		// For whatever reason sometimes there are invalid specs in the layer stack, so we need to be careful
		if (!Spec)
		{
			continue;
		}

		pxr::SdfPath SpecPath = Spec->GetPath();

		// Filtering by the target path is important because if X references Y, we'll actually get Y's specs within
		// X.GetPrimStack(), and we don't want to remove the referenced specs when removing the referencer.
		// We strip variant selections here because when removing something inside the variant, SpecPath will contain
		// the variant selection and look like '/PrimWithVarSet{VarSet=SomeVar}ChildPrim', but our TargetPath will
		// just look like '/PrimWithVarSet/ChildPrim' instead. These do refer to the exact same prim on the stage
		// though (when SomeVar is active at least), so we do want to remove both
		if (!SpecPath.IsPrimPath() || SpecPath.StripAllVariantSelections() != TargetPath)
		{
			continue;
		}

		pxr::SdfLayerRefPtr SpecLayer = Spec->GetLayer();
		if (UsdLayer && SpecLayer != UsdLayer)
		{
			continue;
		}

		if (!UsdLayer && LocalLayerIdentifiers.count(SpecLayer->GetIdentifier()) == 0)
		{
			continue;
		}

		USD_LOG_INFO(
			TEXT("Removing prim spec '%s' from layer '%s'"),
			*UsdToUnreal::ConvertPath(SpecPath),
			*UsdToUnreal::ConvertString(SpecLayer->GetIdentifier())
		);
		pxr::UsdEditContext Context(UsdStage, SpecLayer);
		UsdStage->RemovePrim(SpecPath);
	}

#endif	  // USE_USD_SDK
}

bool UsdUtils::CutPrims(const TArray<UE::FUsdPrim>& Prims)
{
	bool bCopied = UsdUtils::CopyPrims(Prims);
	if (!bCopied)
	{
		return false;
	}

	for (const UE::FUsdPrim& Prim : Prims)
	{
		UsdUtils::RemoveAllLocalPrimSpecs(Prim);
	}

	return true;
}

bool UsdUtils::CopyPrims(const TArray<UE::FUsdPrim>& Prims)
{
	bool bCopiedSomething = false;

#if USE_USD_SDK
	FScopedUsdAllocs Allocs;

	pxr::UsdStageRefPtr UsdStage;
	for (const UE::FUsdPrim& Prim : Prims)
	{
		if (Prim)
		{
			UsdStage = pxr::UsdStageRefPtr{Prim.GetStage()};
			if (UsdStage)
			{
				break;
			}
		}
	}
	if (!UsdStage)
	{
		return false;
	}

	pxr::UsdStageRefPtr ClipboardStage = pxr::UsdStageRefPtr{UnrealUSDWrapper::GetClipboardStage()};
	if (!ClipboardStage)
	{
		return false;
	}

	pxr::SdfLayerHandle ClipboardRoot = ClipboardStage->GetRootLayer();
	if (!ClipboardRoot)
	{
		return false;
	}

	pxr::UsdStagePopulationMask Mask;
	for (const UE::FUsdPrim& Prim : Prims)
	{
		if (Prim)
		{
			Mask.Add(pxr::SdfPath{Prim.GetPrimPath()});
		}
	}
	if (Mask.IsEmpty())
	{
		return false;
	}

	pxr::UsdStageRefPtr TempStage = pxr::UsdStage::OpenMasked(UsdStage->GetRootLayer(), Mask);
	if (!TempStage)
	{
		return false;
	}

	// USD will retain instances and prototypes even when flattening, which is not what we want
	// so let's disable instancing on our temp stage before we ask it to flatten.
	// Note how we traverse the entire masked stage here, because we also need to handle the case
	// where the prim we're duplicating is not instanceable, but has instanceable children
	TArray<pxr::SdfPath> OldInstanceablePrims;
	if (TempStage->GetPrototypes().size() > 0)
	{
		pxr::UsdEditContext Context{TempStage, TempStage->GetSessionLayer()};

		pxr::UsdPrimRange PrimRange(TempStage->GetPseudoRoot());
		for (pxr::UsdPrimRange::iterator PrimRangeIt = PrimRange.begin(); PrimRangeIt != PrimRange.end(); ++PrimRangeIt)
		{
			if (PrimRangeIt->IsPseudoRoot())
			{
				continue;
			}

			if (PrimRangeIt->HasAuthoredInstanceable())
			{
				PrimRangeIt->SetInstanceable(false);
				OldInstanceablePrims.Add(PrimRangeIt->GetPrimPath());
			}
		}
	}

	const bool bAddSourceFileComment = false;
	pxr::SdfLayerRefPtr FlattenedLayer = TempStage->Flatten(bAddSourceFileComment);
	if (!FlattenedLayer)
	{
		return false;
	}

	// We may had to force instanceable=false on the prims we duplicated in order to get our session layer
	// opinion to disable instancing. We don't want those prims to come out with "instanceable=false" on the
	// flattened copy though, so here we clear that opinion
	for (const pxr::SdfPath& Path : OldInstanceablePrims)
	{
		if (pxr::SdfPrimSpecHandle Spec = FlattenedLayer->GetPrimAtPath(Path))
		{
			Spec->ClearInstanceable();
		}
	}

	ClipboardRoot->Clear();

	TSet<FString> UsedNames;

	for (const UE::FUsdPrim& Prim : Prims)
	{
		pxr::SdfPrimSpecHandle FlattenedPrim = FlattenedLayer->GetPrimAtPath(pxr::SdfPath{Prim.GetPrimPath()});
		if (!FlattenedPrim)
		{
			continue;
		}

		// Have to ensure the selected prims can coexist as siblings on the clipboard until being pasted.
		// Note how we don't use GetValidChildName here: That should work too, but it could fail if somebody ever
		// calls this function within a SdfChangeBlock, given that GetValidChildName relies on USD's GetChildren,
		// which could potentially yield stale results until USD actually emits the notices about these prims being
		// added.
		FString PrimName = Prim.GetName().ToString();
		FString UniqueName = UsdUnreal::ObjectUtils::GetUniqueName(SanitizeUsdIdentifier(*PrimName), UsedNames);
		UsedNames.Add(UniqueName);

		const bool bSuccess = pxr::SdfCopySpec(
			FlattenedLayer,
			FlattenedPrim->GetPath(),
			ClipboardRoot,
			pxr::SdfPath::AbsoluteRootPath().AppendChild(UnrealToUsd::ConvertToken(*UniqueName).Get())
		);
		if (!bSuccess)
		{
			continue;
		}

		bCopiedSomething = true;
		USD_LOG_INFO(TEXT("Copied prim '%s' into the clipboard"), *Prim.GetPrimPath().GetString());
	}
#endif	  // USE_USD_SDK

	return bCopiedSomething;
}

TArray<UE::FSdfPath> UsdUtils::PastePrims(const UE::FUsdPrim& ParentPrim)
{
	TArray<UE::FSdfPath> Result;

#if USE_USD_SDK
	FScopedUsdAllocs Allocs;

	pxr::UsdPrim UsdParentPrim{ParentPrim};
	if (!UsdParentPrim)
	{
		return Result;
	}

	pxr::UsdStageRefPtr UsdStage = UsdParentPrim.GetStage();
	if (!UsdStage)
	{
		return Result;
	}

	pxr::UsdStageRefPtr ClipboardStage = pxr::UsdStageRefPtr{UnrealUSDWrapper::GetClipboardStage(/* bCreateIfNeeded = */ false)};
	if (!ClipboardStage)
	{
		return Result;
	}

	pxr::SdfLayerHandle ClipboardRoot = ClipboardStage->GetRootLayer();
	if (!ClipboardRoot)
	{
		return Result;
	}

	pxr::UsdPrimSiblingRange PrimChildren = ClipboardStage->GetPseudoRoot().GetChildren();
	int32 NumPrimsToPaste = std::distance(PrimChildren.begin(), PrimChildren.end());

	TArray<pxr::UsdPrim> PrimsToPaste;
	PrimsToPaste.Reserve(NumPrimsToPaste);
	for (const pxr::UsdPrim& ClipboardPrim : ClipboardStage->GetPseudoRoot().GetChildren())
	{
		PrimsToPaste.Add(ClipboardPrim);
	}

	pxr::SdfLayerHandle EditTarget = UsdStage->GetEditTarget().GetLayer();
	if (!EditTarget)
	{
		return Result;
	}

	TSet<FString> UsedNames;
	for (const pxr::UsdPrim& Child : ParentPrim.GetChildren())
	{
		UsedNames.Add(UsdToUnreal::ConvertToken(Child.GetName()));
	}

	Result.SetNum(NumPrimsToPaste);
	for (int32 Index = 0; Index < NumPrimsToPaste; ++Index)
	{
		const pxr::UsdPrim& ClipboardPrim = PrimsToPaste[Index];
		if (!ClipboardPrim)
		{
			continue;
		}

		const FString OriginalName = UsdToUnreal::ConvertToken(ClipboardPrim.GetName());
		FString ValidName = UsdUnreal::ObjectUtils::GetUniqueName(SanitizeUsdIdentifier(*OriginalName), UsedNames);
		UsedNames.Add(ValidName);

		pxr::SdfPath TargetSpecPath = UsdParentPrim.GetPath().AppendChild(UnrealToUsd::ConvertToken(*ValidName).Get());

		// Ensure our parent prim spec exists, otherwise pxr::SdfCopySpec will fail
		if (!pxr::SdfCreatePrimInLayer(EditTarget, TargetSpecPath))
		{
			continue;
		}

		if (!pxr::SdfCopySpec(ClipboardRoot, ClipboardPrim.GetPath(), EditTarget, TargetSpecPath))
		{
			continue;
		}

		USD_LOG_INFO(
			TEXT("Pasted prim '%s' as a child of prim '%s' within the edit target '%s'"),
			*OriginalName,
			*UsdToUnreal::ConvertPath(UsdParentPrim.GetPath()),
			*UsdToUnreal::ConvertString(EditTarget->GetIdentifier())
		);
		Result[Index] = UE::FSdfPath{TargetSpecPath};
	}
#endif	  // USE_USD_SDK

	return Result;
}

bool UsdUtils::CanPastePrims()
{
#if USE_USD_SDK
	pxr::UsdStageRefPtr ClipboardStage = pxr::UsdStageRefPtr{UnrealUSDWrapper::GetClipboardStage(/* bCreateIfNeeded = */ false)};
	if (!ClipboardStage)
	{
		return false;
	}

	for (const pxr::UsdPrim& ClipboardPrim : ClipboardStage->GetPseudoRoot().GetChildren())
	{
		if (ClipboardPrim)
		{
			return true;
		}
	}
#endif	  // USE_USD_SDK

	return false;
}

void UsdUtils::ClearPrimClipboard()
{
#if USE_USD_SDK
	pxr::UsdStageRefPtr ClipboardStage = pxr::UsdStageRefPtr{UnrealUSDWrapper::GetClipboardStage(/* bCreateIfNeeded = */ false)};
	if (!ClipboardStage)
	{
		return;
	}

	pxr::SdfLayerHandle ClipboardRoot = ClipboardStage->GetRootLayer();
	if (!ClipboardRoot)
	{
		return;
	}

	ClipboardRoot->Clear();
#endif	  // USE_USD_SDK
}

TArray<UE::FSdfPath> UsdUtils::DuplicatePrims(const TArray<UE::FUsdPrim>& Prims, EUsdDuplicateType DuplicateType, const UE::FSdfLayer& TargetLayer)
{
	TArray<UE::FSdfPath> Result;
	Result.SetNum(Prims.Num());

#if USE_USD_SDK
	FScopedUsdAllocs Allocs;

	pxr::UsdStageRefPtr UsdStage;
	for (const UE::FUsdPrim& Prim : Prims)
	{
		if (Prim)
		{
			UsdStage = pxr::UsdStageRefPtr{Prim.GetStage()};
			if (UsdStage)
			{
				break;
			}
		}
	}
	if (!UsdStage)
	{
		return Result;
	}

	pxr::SdfLayerRefPtr UsdLayer{TargetLayer};

	// Figure out which layers we'll modify
	std::unordered_set<pxr::SdfLayerHandle, pxr::TfHash> LayersThatCanBeAffected;
	switch (DuplicateType)
	{
		case EUsdDuplicateType::FlattenComposedPrim:
		case EUsdDuplicateType::SingleLayerSpecs:
		{
			if (!UsdLayer)
			{
				return Result;
			}

			LayersThatCanBeAffected.insert(UsdLayer);
			break;
		}
		case EUsdDuplicateType::AllLocalLayerSpecs:
		{
			const bool bIncludeSessionLayers = true;
			for (const pxr::SdfLayerHandle& Handle : UsdStage->GetLayerStack(bIncludeSessionLayers))
			{
				LayersThatCanBeAffected.insert(Handle);
			}

			// If any of our prims has specs on layers that are used by the stage but are not within the local layer
			// stack, then warn the user that some of these specs will not be duplicated
			{
				TArray<UE::FUsdPrim> PrimsWithExternalSpecs;
				for (const UE::FUsdPrim& Prim : Prims)
				{
					pxr::UsdPrim UsdPrim{Prim};
					if (!UsdPrim)
					{
						continue;
					}

					for (const pxr::SdfPrimSpecHandle& Spec : UsdPrim.GetPrimStack())
					{
						if (Spec && LayersThatCanBeAffected.count(Spec->GetLayer()) == 0)
						{
							PrimsWithExternalSpecs.Add(Prim);
							break;
						}
					}
				}
				USDConversionUtilsImpl::NotifySpecsWontBeDuplicated(PrimsWithExternalSpecs);
			}
			break;
		}
	}

	// If we're going to need to flatten, just flatten the stage once for all prims we'll duplicate
	pxr::SdfLayerRefPtr FlattenedLayer = nullptr;
	if (DuplicateType == EUsdDuplicateType::FlattenComposedPrim)
	{
		pxr::UsdStagePopulationMask Mask;
		for (int32 Index = 0; Index < Prims.Num(); ++Index)
		{
			pxr::UsdPrim UsdPrim{Prims[Index]};
			if (UsdPrim)
			{
				Mask.Add(UsdPrim.GetPath());
			}
		}

		pxr::UsdStageRefPtr TempStage = pxr::UsdStage::OpenMasked(UsdStage->GetRootLayer(), Mask);
		if (!TempStage)
		{
			return Result;
		}

		// USD will retain instances and prototypes even when flattening, which is not what we want
		// so let's disable instancing on our temp stage before we ask it to flatten.
		// Note how we travere the entire masked stage here, because we also need to handle the case
		// where the prim we're duplicating is not instanceable, but has instanceable children
		TArray<pxr::SdfPath> OldInstanceablePrims;
		if (TempStage->GetPrototypes().size() > 0)
		{
			pxr::UsdEditContext Context{TempStage, TempStage->GetSessionLayer()};

			pxr::UsdPrimRange PrimRange(TempStage->GetPseudoRoot());
			for (pxr::UsdPrimRange::iterator PrimRangeIt = PrimRange.begin(); PrimRangeIt != PrimRange.end(); ++PrimRangeIt)
			{
				if (PrimRangeIt->IsPseudoRoot())
				{
					continue;
				}

				if (PrimRangeIt->HasAuthoredInstanceable())
				{
					PrimRangeIt->SetInstanceable(false);
					OldInstanceablePrims.Add(PrimRangeIt->GetPrimPath());
				}
			}
		}

		const bool bAddSourceFileComment = false;
		FlattenedLayer = TempStage->Flatten(bAddSourceFileComment);
		if (!FlattenedLayer)
		{
			return Result;
		}

		// We may had to force instanceable=false on the prims we duplicated in order to get our session layer
		// opinion to disable instancing. We don't want those prims to come out with "instanceable=false" on the
		// flattened copy though, so here we clear that opinion
		for (const pxr::SdfPath& Path : OldInstanceablePrims)
		{
			if (pxr::SdfPrimSpecHandle Spec = FlattenedLayer->GetPrimAtPath(Path))
			{
				Spec->ClearInstanceable();
			}
		}
	}

	for (int32 Index = 0; Index < Prims.Num(); ++Index)
	{
		pxr::UsdPrim UsdPrim{Prims[Index]};
		if (!UsdPrim)
		{
			continue;
		}

		std::vector<pxr::SdfPrimSpecHandle> PrimSpecs = UsdPrim.GetPrimStack();

		// Note: We won't actually use these in case we're flattening, but it makes the code a bit simpler to also
		// do this while we're collecting LayersThatWillBeAffected below
		std::vector<pxr::SdfPrimSpecHandle> SpecsToDuplicate;
		SpecsToDuplicate.reserve(PrimSpecs.size());

		std::unordered_set<pxr::SdfLayerHandle, pxr::TfHash> LayersThatWillBeAffected;
		LayersThatWillBeAffected.reserve(PrimSpecs.size());

		pxr::SdfPath TargetPath = UsdPrim.GetPrimPath();
		for (int32 SpecIndex = PrimSpecs.size() - 1; SpecIndex >= 0; --SpecIndex)
		{
			const pxr::SdfPrimSpecHandle& Spec = PrimSpecs[SpecIndex];

			// For whatever reason sometimes there are invalid specs in the layer stack, so we need to be careful
			if (!Spec)
			{
				continue;
			}

			pxr::SdfPath SpecPath = Spec->GetPath();

			// Skip specs that have a different path than the actual prim path. The only way this could happen
			// is if the prim is referencing this particular path, and if we were to duplicate this spec
			// we'd essentially end up flattening the referenced prim over the new duplicate prim, which
			// is not what we want. We'll already get the fact that "prim references this other prim" by copying
			// the spec at the actual TargetPath however
			if (!SpecPath.IsPrimPath() || SpecPath.StripAllVariantSelections() != TargetPath)
			{
				continue;
			}

			pxr::SdfLayerHandle SpecLayerHandle = Spec->GetLayer();
			if (!SpecLayerHandle || LayersThatCanBeAffected.count(SpecLayerHandle) == 0)
			{
				continue;
			}

			SpecsToDuplicate.push_back(Spec);
			LayersThatWillBeAffected.insert(SpecLayerHandle);
		}

		// Find a usable name for the new duplicate prim
		pxr::SdfPath NewSpecPath;
		{
			const std::string SourcePrimName = UsdPrim.GetName().GetString();
			const pxr::SdfPath ParentPath = UsdPrim.GetPath().GetParentPath();

			int32 Suffix = -1;
			while (true)
			{
				NewSpecPath = ParentPath.AppendElementString(SourcePrimName + "_" + std::to_string(++Suffix));

				// We want to make sure our new duplicate prim is unique across the entire composed stage, as opposed
				// to silently overriding another prim that is only defined in an obscure layer somewhere
				pxr::UsdPrim ExistingPrim = UsdStage->GetPrimAtPath(NewSpecPath);
				if (!ExistingPrim)
				{
					break;
				}
			}
		}

		// Actually do the duplication operation we chose
		if (DuplicateType == EUsdDuplicateType::FlattenComposedPrim && FlattenedLayer)
		{
			pxr::SdfPrimSpecHandle FlattenedPrim = FlattenedLayer->GetPrimAtPath(UsdPrim.GetPath());
			if (!FlattenedPrim)
			{
				return Result;
			}

			if (!pxr::SdfJustCreatePrimInLayer(UsdLayer, NewSpecPath))
			{
				USD_LOG_WARNING(
					TEXT("Failed to create prim and parent specs for path '%s' within layer '%s'"),
					*UsdToUnreal::ConvertPath(NewSpecPath),
					*UsdToUnreal::ConvertString(UsdLayer->GetIdentifier())
				);
				return Result;
			}

			if (!pxr::SdfCopySpec(FlattenedLayer, FlattenedPrim->GetPath(), UsdLayer, NewSpecPath))
			{
				USD_LOG_WARNING(
					TEXT("Failed to copy flattened prim spec from '%s' onto path '%s' within layer '%s'"),
					*UsdToUnreal::ConvertPath(UsdPrim.GetPath()),
					*UsdToUnreal::ConvertPath(NewSpecPath),
					*UsdToUnreal::ConvertString(UsdLayer->GetIdentifier())
				);
				return Result;
			}

			USD_LOG_INFO(
				TEXT("Flattened prim '%s' onto spec '%s' at layer '%s'"),
				*UsdToUnreal::ConvertPath(UsdPrim.GetPath()),
				*UsdToUnreal::ConvertPath(NewSpecPath),
				*UsdToUnreal::ConvertString(UsdLayer->GetIdentifier())
			);
		}
		else
		{
			for (const pxr::SdfPrimSpecHandle& Spec : SpecsToDuplicate)
			{
				pxr::SdfPath SpecPath = Spec->GetPath();
				pxr::SdfLayerHandle SpecLayerHandle = Spec->GetLayer();

				USD_LOG_INFO(
					TEXT("Duplicating prim spec '%s' within layer '%s'"),
					*UsdToUnreal::ConvertPath(SpecPath),
					*UsdToUnreal::ConvertString(SpecLayerHandle->GetIdentifier())
				);

				// Technically we shouldn't need to do this since we'll already do our changes on the Sdf level, however the
				// USDTransactor will record these notices as belonging to the current edit target, and if that is not in sync
				// with the layer that is actually changing, we won't be able to undo/redo the duplicate operation
				pxr::UsdEditContext Context{UsdStage, SpecLayerHandle};

				// Since we're duplicating a prim essentially as a sibling, parent specs should always exist.
				// Let's ensure that though, just in case
				if (!pxr::SdfJustCreatePrimInLayer(SpecLayerHandle, NewSpecPath))
				{
					USD_LOG_WARNING(
						TEXT("Failed to create prim and parent specs for path '%s' within layer '%s'"),
						*UsdToUnreal::ConvertPath(NewSpecPath),
						*UsdToUnreal::ConvertString(SpecLayerHandle->GetIdentifier())
					);
					continue;
				}

				pxr::SdfShouldCopyValueFn ShouldCopyValue = [](pxr::SdfSpecType SpecType,
															   const pxr::TfToken& Field,
															   const pxr::SdfLayerHandle& SrcLayer,
															   const pxr::SdfPath& SrcPath,
															   bool FieldInSrc,
															   const pxr::SdfLayerHandle& DstLayer,
															   const pxr::SdfPath& DstPath,
															   bool FieldInDst,
															   std::optional<pxr::VtValue>* ValueToCopy) -> bool
				{
					// Only copy a field over if it has a value. Otherwise it seems to clear the destination spec
					// for nothing
					return FieldInSrc;
				};

				pxr::SdfShouldCopyChildrenFn ShouldCopyChildren = [](const pxr::TfToken& ChildrenField,
																	 const pxr::SdfLayerHandle& SrcLayer,
																	 const pxr::SdfPath& SrcPath,
																	 bool FieldInSrc,
																	 const pxr::SdfLayerHandle& DstLayer,
																	 const pxr::SdfPath& DstPath,
																	 bool FieldInDst,
																	 std::optional<pxr::VtValue>* SrcChildren,
																	 std::optional<pxr::VtValue>* DstChildren) -> bool
				{
					return true;
				};

				// We use the advanced version of SdfCopySpec here as otherwise the default behavior is to fully clear
				// the destination spec before copying stuff, and we may want to copy multiple specs overwriting each other
				if (!pxr::SdfCopySpec(SpecLayerHandle, SpecPath, SpecLayerHandle, NewSpecPath, ShouldCopyValue, ShouldCopyChildren))
				{
					USD_LOG_WARNING(
						TEXT("Failed to copy spec from path '%s' onto path '%s' within layer '%s'"),
						*UsdToUnreal::ConvertPath(SpecPath),
						*UsdToUnreal::ConvertPath(NewSpecPath),
						*UsdToUnreal::ConvertString(SpecLayerHandle->GetIdentifier())
					);
				}
			}
		}

		Result[Index] = UE::FSdfPath{NewSpecPath};
	}
#endif	  // USE_USD_SDK

	return Result;
}

void UsdUtils::SetPrimAssetInfo(UE::FUsdPrim& Prim, const FUsdUnrealAssetInfo& Info)
{
#if USE_USD_SDK
	FScopedUsdAllocs Allocs;

	pxr::UsdPrim UsdPrim{Prim};
	if (!UsdPrim)
	{
		return;
	}

	// Just fetch the dictionary already since we'll add custom keys anyway
	pxr::VtDictionary AssetInfoDict = UsdPrim.GetAssetInfo();

	if (!Info.Name.IsEmpty())
	{
		AssetInfoDict.SetValueAtPath(pxr::UsdModelAPIAssetInfoKeys->name, pxr::VtValue{UnrealToUsd::ConvertString(*Info.Name).Get()});
	}

	if (!Info.Identifier.IsEmpty())
	{
		AssetInfoDict.SetValueAtPath(
			pxr::UsdModelAPIAssetInfoKeys->identifier,
			pxr::VtValue{pxr::SdfAssetPath{UnrealToUsd::ConvertString(*Info.Identifier).Get()}}
		);
	}

	if (!Info.Version.IsEmpty())
	{
		AssetInfoDict.SetValueAtPath(pxr::UsdModelAPIAssetInfoKeys->version, pxr::VtValue{UnrealToUsd::ConvertString(*Info.Version).Get()});
	}

	if (!Info.UnrealContentPath.IsEmpty())
	{
		AssetInfoDict.SetValueAtPath(UnrealIdentifiers::UnrealContentPath, pxr::VtValue{UnrealToUsd::ConvertString(*Info.UnrealContentPath).Get()});
	}

	if (!Info.UnrealAssetType.IsEmpty())
	{
		AssetInfoDict.SetValueAtPath(UnrealIdentifiers::UnrealAssetType, pxr::VtValue{UnrealToUsd::ConvertString(*Info.UnrealAssetType).Get()});
	}

	if (!Info.UnrealExportTime.IsEmpty())
	{
		AssetInfoDict.SetValueAtPath(UnrealIdentifiers::UnrealExportTime, pxr::VtValue{UnrealToUsd::ConvertString(*Info.UnrealExportTime).Get()});
	}

	if (!Info.UnrealEngineVersion.IsEmpty())
	{
		AssetInfoDict.SetValueAtPath(
			UnrealIdentifiers::UnrealEngineVersion,
			pxr::VtValue{UnrealToUsd::ConvertString(*Info.UnrealEngineVersion).Get()}
		);
	}

	UsdPrim.SetAssetInfo(AssetInfoDict);
#endif	  // USE_USD_SDK
}

FUsdUnrealAssetInfo UsdUtils::GetPrimAssetInfo(const UE::FUsdPrim& Prim)
{
	FUsdUnrealAssetInfo Result;

#if USE_USD_SDK
	FScopedUsdAllocs Allocs;

	pxr::UsdPrim UsdPrim{Prim};
	if (!UsdPrim)
	{
		return Result;
	}

	// Just fetch the dictionary already since we'll fetch custom keys anyway
	pxr::VtDictionary AssetInfoDict = UsdPrim.GetAssetInfo();

	if (const pxr::VtValue* Value = AssetInfoDict.GetValueAtPath(pxr::UsdModelAPIAssetInfoKeys->name))
	{
		if (Value->IsHolding<std::string>())
		{
			Result.Name = UsdToUnreal::ConvertString(Value->Get<std::string>());
		}
	}

	if (const pxr::VtValue* Value = AssetInfoDict.GetValueAtPath(pxr::UsdModelAPIAssetInfoKeys->identifier))
	{
		if (Value->IsHolding<pxr::SdfAssetPath>())
		{
			Result.Identifier = UsdToUnreal::ConvertString(Value->Get<pxr::SdfAssetPath>().GetAssetPath());
		}
	}

	if (const pxr::VtValue* Value = AssetInfoDict.GetValueAtPath(pxr::UsdModelAPIAssetInfoKeys->version))
	{
		if (Value->IsHolding<std::string>())
		{
			Result.Version = UsdToUnreal::ConvertString(Value->Get<std::string>());
		}
	}

	if (const pxr::VtValue* Value = AssetInfoDict.GetValueAtPath(UnrealIdentifiers::UnrealContentPath))
	{
		if (Value->IsHolding<std::string>())
		{
			Result.UnrealContentPath = UsdToUnreal::ConvertString(Value->Get<std::string>());
		}
	}

	if (const pxr::VtValue* Value = AssetInfoDict.GetValueAtPath(UnrealIdentifiers::UnrealAssetType))
	{
		if (Value->IsHolding<std::string>())
		{
			Result.UnrealAssetType = UsdToUnreal::ConvertString(Value->Get<std::string>());
		}
	}

	if (const pxr::VtValue* Value = AssetInfoDict.GetValueAtPath(UnrealIdentifiers::UnrealExportTime))
	{
		if (Value->IsHolding<std::string>())
		{
			Result.UnrealExportTime = UsdToUnreal::ConvertString(Value->Get<std::string>());
		}
	}

	if (const pxr::VtValue* Value = AssetInfoDict.GetValueAtPath(UnrealIdentifiers::UnrealEngineVersion))
	{
		if (Value->IsHolding<std::string>())
		{
			Result.UnrealEngineVersion = UsdToUnreal::ConvertString(Value->Get<std::string>());
		}
	}
#endif	  // USE_USD_SDK

	return Result;
}

#if USE_USD_SDK
bool UsdUtils::ClearNonEssentialPrimMetadata(const pxr::UsdPrim& Prim)
{
	FScopedUsdAllocs Allocs;

	pxr::SdfChangeBlock ChangeBlock;

	// Note: This only returns top-level fields, and won't have a separate entry for values inside VtDictionaries
	// or anything like that. This means this likely won't be that expensive, and we don't have to care about order
	std::map<pxr::TfToken, pxr::VtValue, pxr::TfDictionaryLessThan> MetadataMap = Prim.GetAllAuthoredMetadata();

	for (std::map<pxr::TfToken, pxr::VtValue, pxr::TfDictionaryLessThan>::const_iterator MetadataIter = MetadataMap.begin();
		 MetadataIter != MetadataMap.end();
		 ++MetadataIter)
	{
		const pxr::TfToken& FieldName = MetadataIter->first;

		// We consider those "essential metadata", as removing them will mess with the prim definition
		const static std::unordered_set<pxr::TfToken, pxr::TfHash> FieldsToSkip = {pxr::SdfFieldKeys->Specifier, pxr::SdfFieldKeys->TypeName};
		if (FieldsToSkip.count(FieldName) > 0)
		{
			continue;
		}

		const bool bSuccess = Prim.ClearMetadata(FieldName);

		if (!bSuccess)
		{
			USD_LOG_WARNING(
				TEXT("Failed to clear metadata field '%s' from prim '%s'"),
				*UsdToUnreal::ConvertToken(FieldName),
				*UsdToUnreal::ConvertPath(Prim.GetPrimPath())
			);
			return false;
		}
	}

	return true;
}
#endif	  // USE_USD_SDK

void UsdUtils::CollectSchemaAnalytics(const UE::FUsdStage& Stage, TArray<FAnalyticsEventAttribute>& OutAttributes)
{
#if USE_USD_SDK
	if (!Stage)
	{
		return;
	}

	// Last collected at cl 48758811
	const static TSet<FString> NativeSchemaNames{
		"AccessibilityAPI",
		"AssetPreviewsAPI",
		"Backdrop",
		"BasisCurves",
		"BendingWingAPI",
		"BlendShape",
		"Camera",
		"Capsule",
		"Capsule_1",
		"ChaosFabricAPI",
		"ChaosSolverPropertiesAPI",
		"ClipsAPI",
		"CloFabricAPI",
		"CloSolverPropertiesAPI",
		"ClothRootAPI",
		"ClothSolverAPI",
		"CollapsingAPI",
		"CollectionAPI",
		"ColorSpaceAPI",
		"ColorSpaceDefinitionAPI",
		"Cone",
		"ConnectableAPI",
		"ControlRigAPI",
		"CoordSysAPI",
		"Cube",
		"Cylinder",
		"CylinderLight",
		"Cylinder_1",
		"DiskLight",
		"DistantLight",
		"DomeLight",
		"DomeLight_1",
		"Field3DAsset",
		"GenerativeProcedural",
		"GeomModelAPI",
		"GeomSubset",
		"GeometryLight",
		"GroomAPI",
		"GroomBindingAPI",
		"HermiteCurves",
		"HydraGenerativeProceduralAPI",
		"LightAPI",
		"LightFilter",
		"LightListAPI",
		"ListAPI",
		"LiveLinkAPI",
		"LodSubtreeAPI",
		"Material",
		"MaterialBindingAPI",
		"MaterialXConfigAPI",
		"Mesh",
		"MeshLightAPI",
		"ModelAPI",
		"MotionAPI",
		"NaniteAssemblyExternalRefAPI",
		"NaniteAssemblyRootAPI",
		"NaniteAssemblySkelBindingAPI",
		"NodeDefAPI",
		"NodeGraph",
		"NodeGraphNodeAPI",
		"NurbsCurves",
		"NurbsPatch",
		"OpenVDBAsset",
		"PhysicsArticulationRootAPI",
		"PhysicsCollisionAPI",
		"PhysicsCollisionGroup",
		"PhysicsDistanceJoint",
		"PhysicsDriveAPI",
		"PhysicsFilteredPairsAPI",
		"PhysicsFixedJoint",
		"PhysicsJoint",
		"PhysicsLimitAPI",
		"PhysicsMassAPI",
		"PhysicsMaterialAPI",
		"PhysicsMeshCollisionAPI",
		"PhysicsPrismaticJoint",
		"PhysicsRevoluteJoint",
		"PhysicsRigidBodyAPI",
		"PhysicsScene",
		"PhysicsSphericalJoint",
		"PinAPI",
		"Plane",
		"PluginLight",
		"PluginLightFilter",
		"PointInstancer",
		"Points",
		"PortalLight",
		"PrimvarsAPI",
		"RectLight",
		"RenderPass",
		"RenderPatternAPI",
		"RenderProduct",
		"RenderSettings",
		"RenderVar",
		"RiMaterialAPI",
		"RiSplineAPI",
		"SceneGraphPrimAPI",
		"Scope",
		"SemanticsLabelsAPI",
		"SewingAPI",
		"Shader",
		"ShadowAPI",
		"ShapingAPI",
		"SimMeshDataAPI",
		"SimPatternAPI",
		"SkelAnimation",
		"SkelBindingAPI",
		"SkelRoot",
		"Skeleton",
		"SparseVolumeTextureAPI",
		"SpatialAudio",
		"Sphere",
		"SphereLight",
		"SpringAPI",
		"StatementsAPI",
		"SubdivisionAPI",
		"TackSpringAPI",
		"TetMesh",
		"VisibilityAPI",
		"Volume",
		"VolumeLightAPI",
		"Xform",
		"XformCommonAPI"
	};

	// clang-format off
	// "Interesting" here means either something we don't support, or something that would be interesting
	// to check the usage of, like for our Groom/ControlRig/LiveLink support, or whether the new support
	// for OpenVDB is used, etc.
	TMap<FString, int32> InterestingNativeSchemaCounts{
		{"AssetPreviewsAPI", 0},
		{"Backdrop", 0},
		{"BasisCurves", 0},
		{"Capsule_1", 0},
		{"ClipsAPI", 0},
		{"CollapsingAPI", 0},
		{"CollectionAPI", 0},
		{"ControlRigAPI", 0},
		{"CoordSysAPI", 0},
		{"Cylinder_1", 0},
		{"CylinderLight", 0},
		{"Field3DAsset", 0},
		{"GenerativeProcedural", 0},
		{"GeometryLight", 0},
		{"GroomAPI", 0},
		{"GroomBindingAPI", 0},
		{"HermiteCurves", 0},
		{"HydraGenerativeProceduralAPI", 0},
		{"LightFilter", 0},
		{"LightListAPI", 0},
		{"ListAPI", 0},
		{"LiveLinkAPI", 0},
		{"LodSubtreeAPI", 0},
		{"MeshLightAPI", 0},
		{"MotionAPI", 0},
		{"NaniteAssemblyExternalRefAPI", 0},
		{"NaniteAssemblyRootAPI", 0},
		{"NaniteAssemblySkelBindingAPI", 0},
		{"NurbsCurves", 0},
		{"NurbsPatch", 0},
		{"OpenVDBAsset", 0},
		{"PhysicsArticulationRootAPI", 0},
		{"PhysicsCollisionAPI", 0},
		{"PhysicsCollisionGroup", 0},
		{"PhysicsDistanceJoint", 0},
		{"PhysicsDriveAPI", 0},
		{"PhysicsFilteredPairsAPI", 0},
		{"PhysicsFixedJoint", 0},
		{"PhysicsJoint", 0},
		{"PhysicsLimitAPI", 0},
		{"PhysicsMassAPI", 0},
		{"PhysicsMaterialAPI", 0},
		{"PhysicsMeshCollisionAPI", 0},
		{"PhysicsPrismaticJoint", 0},
		{"PhysicsRevoluteJoint", 0},
		{"PhysicsRigidBodyAPI", 0},
		{"PhysicsScene", 0},
		{"PhysicsSphericalJoint", 0},
		{"PluginLight", 0},
		{"PluginLightFilter", 0},
		{"Points", 0},
		{"PortalLight", 0},
		{"RenderDenoisePass", 0},
		{"RenderPass", 0},
		{"RenderProduct", 0},
		{"RenderSettings", 0},
		{"RenderVar", 0},
		{"RiMaterialAPI", 0},
		{"RiRenderPassAPI", 0},
		{"RiSplineAPI", 0},
		{"SceneGraphPrimAPI", 0},
		{"ShadowAPI", 0},
		{"SparseVolumeTextureAPI", 0},
		{"SpatialAudio", 0},
		{"StatementsAPI", 0},
		{"SubdivisionAPI", 0},
		{"TetMesh", 0},
		{"VisibilityAPI", 0},
		{"Volume", 0},
		{"VolumeLightAPI", 0}
	};
	// clang-format on

	TSet<FString> SeenSchemas;

	int32 CustomSchemaCount = 0;
	{
		FScopedUsdAllocs Allocs;

		pxr::UsdPrimRange PrimRange{Stage.GetPseudoRoot(), pxr::UsdTraverseInstanceProxies()};
		for (pxr::UsdPrimRange::iterator PrimRangeIt = ++PrimRange.begin(); PrimRangeIt != PrimRange.end(); ++PrimRangeIt)
		{
			// It's perfectly fine to have a typeless prim (e.g. "def 'Cube'")
			if (PrimRangeIt->HasAuthoredTypeName())
			{
				const pxr::TfToken& TypeName = PrimRangeIt->GetTypeName();
				const FString TypeNameStr = UsdToUnreal::ConvertToken(TypeName);

				if (!TypeNameStr.IsEmpty())
				{
					SeenSchemas.Add(TypeNameStr);

					if (int32* InterestingSchemaCount = InterestingNativeSchemaCounts.Find(TypeNameStr))
					{
						*InterestingSchemaCount = *InterestingSchemaCount + 1;
					}
				}
			}

			const pxr::UsdPrimTypeInfo& PrimTypeInfo = PrimRangeIt->GetPrimTypeInfo();
			for (const pxr::TfToken& AppliedSchema : PrimTypeInfo.GetAppliedAPISchemas())
			{
				std::pair<pxr::TfToken, pxr::TfToken> Pair = pxr::UsdSchemaRegistry::GetTypeNameAndInstance(AppliedSchema);
				FString TypeName = UsdToUnreal::ConvertToken(Pair.first);

				// These applied schema names shouldn't ever end up as the empty string... but we don't really want to pop
				// an ensure or show a warning when analytics fails
				if (!TypeName.IsEmpty())
				{
					SeenSchemas.Add(TypeName);

					if (int32* InterestingSchemaCount = InterestingNativeSchemaCounts.Find(TypeName))
					{
						*InterestingSchemaCount = *InterestingSchemaCount + 1;
					}
				}
			}
		}
	}

	for (const FString& SchemaName : SeenSchemas)
	{
		// We only care about non-native schemas
		if (!NativeSchemaNames.Contains(SchemaName))
		{
			++CustomSchemaCount;
		}
	}

	if (CustomSchemaCount > 0)
	{
		OutAttributes.Emplace(TEXT("CustomSchemas"), CustomSchemaCount);
	}

	for (const TPair<FString, int32>& Pair : InterestingNativeSchemaCounts)
	{
		const FString& InterestingNativeSchemaName = Pair.Key;
		int32 Count = Pair.Value;
		if (Count > 0)
		{
			OutAttributes.Emplace(InterestingNativeSchemaName, Count);
		}
	}
#endif	  // USE_USD_SDK
}

void UsdUtils::ReadStageMetaData(UE::FUsdStage Stage, TMap<FString, FString>& OutMetaDataMap)
{
#if USE_USD_SDK
	using namespace pxr;

	const FUsdStageInfo StageInfo(Stage);
	OutMetaDataMap.Add(TEXT("Meters Per Unit"), FString::SanitizeFloat(StageInfo.MetersPerUnit));
	if (StageInfo.UpAxis == EUsdUpAxis::ZAxis)
	{
		OutMetaDataMap.Add(TEXT("Up Axis"), TEXT("Z"));
	}
	else if (StageInfo.UpAxis == EUsdUpAxis::YAxis)
	{
		OutMetaDataMap.Add(TEXT("Up Axis"), TEXT("Y"));
	}

	// DefaultPrim
	UE::FVtValue DefaultPrim;
	if (Stage.GetMetadata(*UsdToUnreal::ConvertToken(SdfFieldKeys->DefaultPrim), DefaultPrim))
	{
		VtValue UsdValue = DefaultPrim.GetUsdValue();

		if (UsdValue.IsHolding<TfToken>())
		{
			TfToken Value = UsdValue.UncheckedGet<TfToken>();
			FString DefaultPrimString = UsdToUnreal::ConvertToken(Value);

			if (DefaultPrimString.Len() > 0)
			{
				OutMetaDataMap.Add(TEXT("Default Prim"), DefaultPrimString);
			}
		}
	}

	// double typed potential meta data handling:
	auto ProcessDoubleTypedMetadata = [&OutMetaDataMap, &Stage](const FString& ExtraInformationKey, const FString& TokenStringified)
	{
		UE::FVtValue UEValue;
		if (Stage.GetMetadata(*TokenStringified, UEValue))
		{
			VtValue UsdValue = UEValue.GetUsdValue();

			if (UsdValue.IsHolding<double>())
			{
				double Value = UsdValue.UncheckedGet<double>();
				OutMetaDataMap.Add(ExtraInformationKey, FString::SanitizeFloat(Value));
			}
		}
	};
	ProcessDoubleTypedMetadata(TEXT("Time Codes Per Second"), UsdToUnreal::ConvertToken(SdfFieldKeys->TimeCodesPerSecond));
	ProcessDoubleTypedMetadata(TEXT("Frames Per Second"), UsdToUnreal::ConvertToken(SdfFieldKeys->FramesPerSecond));
	ProcessDoubleTypedMetadata(TEXT("Start Time Code"), UsdToUnreal::ConvertToken(SdfFieldKeys->StartTimeCode));
	ProcessDoubleTypedMetadata(TEXT("End Time Code"), UsdToUnreal::ConvertToken(SdfFieldKeys->EndTimeCode));

	// Documentation
	UE::FVtValue Documentation;
	if (Stage.GetMetadata(*UsdToUnreal::ConvertToken(SdfFieldKeys->Documentation), Documentation))
	{
		VtValue UsdValue = Documentation.GetUsdValue();

		if (UsdValue.IsHolding<std::string>())
		{
			std::string Value = UsdValue.UncheckedGet<std::string>();
			OutMetaDataMap.Add(TEXT("Documentation"), UsdToUnreal::ConvertString(Value.c_str()));
		}
	}

	// CustomLayerData
	UE::FVtValue CustomLayerData;
	if (Stage.GetMetadata(*UsdToUnreal::ConvertToken(SdfFieldKeys->CustomLayerData), CustomLayerData))
	{
		VtValue UsdValue = CustomLayerData.GetUsdValue();

		if (UsdValue.IsHolding<VtDictionary>())
		{
			const VtDictionary& UsdDictionary = UsdValue.UncheckedGet<VtDictionary>();

			for (VtDictionary::const_iterator ValueIter = UsdDictionary.begin(); ValueIter != UsdDictionary.end(); ++ValueIter)
			{
				const std::string& DictFieldName = ValueIter->first;
				const VtValue& DictFieldValue = ValueIter->second;

				FString StringifiedKey = UsdToUnreal::ConvertString(DictFieldName);
				FString StringifiedValue = UsdUtils::Stringify(DictFieldValue);

				OutMetaDataMap.Add(StringifiedKey, StringifiedValue);
			}
		}
	}
#endif
}

bool UsdUtils::IsInstancingAwareTranslationEnabled()
{
#if USE_USD_SDK
	return GInstancingAwareTranslation;
#else
	return true;
#endif
}

UE::FUsdPrim UsdUtils::GetPrototypePrim(const UE::FUsdPrim& Prim)
{
	if (!IsInstancingAwareTranslationEnabled())
	{
		return Prim;
	}
	if (Prim.IsInstance())
	{
		return Prim.GetPrototype();
	}
	else if (Prim.IsInstanceProxy())
	{
		return Prim.GetPrimInPrototype();
	}
	return Prim;
}

UE::FSdfPath UsdUtils::GetPrototypePrimPath(const UE::FUsdPrim& Prim)
{
	return GetPrototypePrim(Prim).GetPrimPath();
}

#if USE_USD_SDK
#undef LOCTEXT_NAMESPACE
#endif
