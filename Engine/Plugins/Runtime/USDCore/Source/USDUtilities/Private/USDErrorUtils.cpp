// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDErrorUtils.h"

#include "USDLog.h"
#include "USDProjectSettings.h"

#include "HAL/IConsoleManager.h"
#include "Math/NumericLimits.h"
#include "Misc/ScopeLock.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"

#if WITH_EDITOR
#include "IMessageLogListing.h"
#include "MessageLogModule.h"
#endif	  // WITH_EDITOR

#if USE_USD_SDK
#include "USDMemory.h"
#include "USDTypesConversion.h"

#include "USDIncludesStart.h"
#include "pxr/base/tf/errorMark.h"
#include "USDIncludesEnd.h"
#endif	  // #if USE_USD_SDK

#define LOCTEXT_NAMESPACE "USDErrorUtils"

static bool GUsdUseMessageLog = true;
static FAutoConsoleVariableRef CVarUsdUseMessageLog(
	TEXT("USD.UseMessageLog"),
	GUsdUseMessageLog,
	TEXT(
		"Output user-facing messages from USD code and the USD-related Unreal plugins on the Message Log (in addition to the Output Log, which is always done). This can help bring issues to attention and it should make it easier to visualize the messages, but it can be slower and consume more memory."
	)
);

namespace UE::USDErrorUtils::Private
{
#if USE_USD_SDK
	// We need an extra level of indirection because TfErrorMark is noncopyable
	using MarkRef = TUsdStore<TSharedRef<pxr::TfErrorMark>>;
	TArray<MarkRef> ErrorMarkStack;
	TUniquePtr<class FUsdDiagnosticDelegate> DiagnosticDelegate;
	TUniquePtr<class FUsdCombinedMessages> CombinedMessages;
	int32 CombinedMessagesRefCount;
	FCriticalSection CombinedMessagesLock;

	FName GenerateMessageIdentifier(uint32 Number)
	{
		// This FName constructor takes an int32. Unfortunately, there are now some new validation 
		// on FNames that asserts if we try using negative integers, so here we mask off the top bit
		// to ensure the int32 becomes positive.
		//
		// See also FName::IsValidInternalNumber and NAME_NO_NUMBER_INTERNAL
		return FName(TEXT("USD_LOG_ID"), Number & 0x7FFFFFFFu);
	}

	void SendMessageToOutputLog(const FTokenizedMessage& Message)
	{
		if (Message.GetSeverity() == EMessageSeverity::Error)
		{
			UE_LOGF(LogUsd, Error, "%ls", *(Message.ToText().ToString()));
		}
		else if (Message.GetSeverity() == EMessageSeverity::Warning || Message.GetSeverity() == EMessageSeverity::PerformanceWarning)
		{
			UE_LOGF(LogUsd, Warning, "%ls", *(Message.ToText().ToString()));
		}
		else
		{
			UE_LOGF(LogUsd, Log, "%ls", *(Message.ToText().ToString()));
		}
	}

	void LogUsdDiagnosticMessage(const pxr::TfDiagnosticBase& DiagMessage)
	{
		FScopedUsdAllocs Allocs;

		std::string Msg = DiagMessage.GetDiagnosticCodeAsString();
		Msg += ": ";
		Msg += DiagMessage.GetCommentary();

		const size_t LineNumber = DiagMessage.GetSourceLineNumber();
		const std::string Filename = DiagMessage.GetSourceFileName();
		const pxr::TfEnum ErrorCode = DiagMessage.GetDiagnosticCode();

		EMessageSeverity::Type Severity = EMessageSeverity::Info;
		switch (ErrorCode.GetValueAsInt())
		{
			default:
			case pxr::TF_DIAGNOSTIC_STATUS_TYPE:
			{
				Severity = EMessageSeverity::Info;
				break;
			}
			case pxr::TF_DIAGNOSTIC_WARNING_TYPE:
			{
				Severity = EMessageSeverity::Warning;
				break;
			}
			case pxr::TF_DIAGNOSTIC_FATAL_CODING_ERROR_TYPE:
			case pxr::TF_DIAGNOSTIC_NONFATAL_ERROR_TYPE:
			case pxr::TF_DIAGNOSTIC_RUNTIME_ERROR_TYPE:
			case pxr::TF_DIAGNOSTIC_FATAL_ERROR_TYPE:
			case pxr::TF_DIAGNOSTIC_CODING_ERROR_TYPE:
			{
				Severity = EMessageSeverity::Error;
				break;
			}
		}

		FScopedUnrealAllocs UEAllocs;

		const uint32 Identifier = HashCombine(FCrc::StrCrc32(Filename.c_str()), GetTypeHash(LineNumber));

		// Note: Using an FText here causes these messages to be user-facing
		FUsdLogManager::Log(Severity, FText::FromString(UsdToUnreal::ConvertString(Msg)), Identifier);
	}

	void PushErrorMark()
	{
		FScopedUsdAllocs Allocs;

		TSharedRef<pxr::TfErrorMark> Mark = MakeShared<pxr::TfErrorMark>();
		Mark->SetMark();

		ErrorMarkStack.Emplace(Mark);
	}

	void PopErrorMark()
	{
		if (ErrorMarkStack.Num() == 0)
		{
			return;
		}

		FScopedUsdAllocs Allocs;

		MarkRef Store = ErrorMarkStack.Pop();
		pxr::TfErrorMark& Mark = Store.Get().Get();

		if (Mark.IsClean())
		{
			return;
		}

		for (pxr::TfErrorMark::Iterator ErrorIter = Mark.GetBegin(); ErrorIter != Mark.GetEnd(); ++ErrorIter)
		{
			LogUsdDiagnosticMessage(*ErrorIter);
		}

		Mark.Clear();
	}

	class FUsdDiagnosticDelegate : public pxr::TfDiagnosticMgr::Delegate
	{
	public:
		virtual ~FUsdDiagnosticDelegate() override = default;

		virtual void IssueError(const pxr::TfError& Message) override
		{
			LogUsdDiagnosticMessage(Message);
		}

		virtual void IssueFatalError(const pxr::TfCallContext& Context, const std::string& Message) override
		{
			FScopedUnrealAllocs UEAllocs;

			EMessageSeverity::Type Severity = EMessageSeverity::Error;
			const char* SourceFile = Context.GetFile();
			size_t LineNumber = Context.GetLine();

			uint32 Identifier = HashCombine(FCrc::StrCrc32(SourceFile), GetTypeHash(LineNumber));
			FUsdLogManager::Log(Severity, FText::FromString(UsdToUnreal::ConvertString(Message)), Identifier);
		}

		virtual void IssueStatus(const pxr::TfStatus& Message) override
		{
			LogUsdDiagnosticMessage(Message);
		}

		virtual void IssueWarning(const pxr::TfWarning& Message) override
		{
			LogUsdDiagnosticMessage(Message);
		}
	};

	class FUsdCombinedMessages
	{
	public:
		FUsdCombinedMessages(const FUsdCombinedMessages& Other) = delete;
		FUsdCombinedMessages(FUsdCombinedMessages&& Other) = default;
		FUsdCombinedMessages& operator=(const FUsdCombinedMessages& Other) = delete;
		FUsdCombinedMessages& operator=(FUsdCombinedMessages&& Other) = default;

		FUsdCombinedMessages(FUsdLogManager::FMessageDisplayCallback InCallback)
			: MessageDisplayCallback(InCallback)
		{
			const UUsdProjectSettings* ProjectSettings = GetDefault<UUsdProjectSettings>();
			bOptimizeUsdLog = ProjectSettings->bOptimizeUsdLog;
		}

		~FUsdCombinedMessages()
		{
			DisplayMessages();
		}

		void DisplayMessages()
		{
#if WITH_EDITOR
			if (bHasSkippedMessages)
			{
				const static TSharedRef<FTokenizedMessage> SkippedMessage = FTokenizedMessage::Create(
					EMessageSeverity::Info,
					LOCTEXT(
						"SkippedMEssages",
						"Some similar log messages were skipped during the previous USD operation. You can disable this behavior by unchecking 'Optimize Usd Log' on the Unreal project settings window."
					)
				);
				SendMessageToOutputLog(*SkippedMessage);
			}

			if (!GUsdUseMessageLog)
			{
				return;
			}

			// Move our messages to a simple array that the IMessageLogListing can consume
			TArray<TSharedRef<FTokenizedMessage>> TokenizedMessages;
			if (bOptimizeUsdLog)
			{
				TokenizedMessages.Reserve(LoggedMessagesByID.Num());

				for (const TPair<FName, FMessageInfo>& Pair : LoggedMessagesByID)
				{
					const FMessageInfo& MessageInfo = Pair.Value;
					if (!MessageInfo.bUserFacing)
					{
						continue;
					}

					TSharedPtr<FTokenizedMessage> Message = MessageInfo.Message;
					if (MessageInfo.Count > 1)
					{
						Message->AddText(FText::Format(LOCTEXT("InstancesTextEditor", " (and {0} similar messages)"), MessageInfo.Count));
					}

					TokenizedMessages.Add(Message.ToSharedRef());
				}
			}
			else
			{
				TokenizedMessages.Reserve(UncollapsedLoggedMessages.Num());

				for (const FMessageInfo& MessageInfo : UncollapsedLoggedMessages)
				{
					if (!MessageInfo.bUserFacing)
					{
						continue;
					}

					TokenizedMessages.Add(MessageInfo.Message.ToSharedRef());
				}
			}

			if (MessageDisplayCallback)
			{
				MessageDisplayCallback(TokenizedMessages);
			}
#endif	  // WITH_EDITOR
		}

		// Returns whether we already had a similar message previously added
		bool AppendMessage(const TSharedRef<FTokenizedMessage>& Message, bool bUserFacing)
		{
			const FName Identifier = Message->GetIdentifier();
			bool bFoundSimilarMessage = false;

			if (bOptimizeUsdLog)
			{
				// Use this instead of just FindOrAdd because it's probably more useful to retain the very first
				// version of the message, if we end up getting multiple
				if (FMessageInfo* FoundMessageInfo = LoggedMessagesByID.Find(Identifier))
				{
					bFoundSimilarMessage = true;
					bHasSkippedMessages = true;

					FoundMessageInfo->bUserFacing = FoundMessageInfo->bUserFacing || bUserFacing;
					FoundMessageInfo->Count += 1;
				}
				else
				{
					FMessageInfo MessageInfo;
					MessageInfo.Message = Message;
					MessageInfo.bUserFacing = MessageInfo.bUserFacing || bUserFacing;
					MessageInfo.Count += 1;

					LoggedMessagesByID.Add(Identifier, MessageInfo);
				}
			}
			else if (GUsdUseMessageLog)
			{
				FMessageInfo& MessageInfo = UncollapsedLoggedMessages.Emplace_GetRef();
				MessageInfo.Message = Message;
				MessageInfo.bUserFacing = bUserFacing;
			}

			return bFoundSimilarMessage;
		}

		bool HasErrorsOrWarnings()
		{
			for (const TPair<FName, FMessageInfo>& Pair : LoggedMessagesByID)
			{
				// Higher severity has lower numbers
				if ((int)Pair.Value.Message->GetSeverity() <= (int)EMessageSeverity::Warning)
				{
					return true;
				}
			}

			return false;
		}

	private:
		struct FMessageInfo
		{
			TSharedPtr<FTokenizedMessage> Message;
			bool bUserFacing = false;
			int32 Count = 0;
		};

		bool bOptimizeUsdLog = false;

		bool bHasSkippedMessages = false;
		TMap<FName, FMessageInfo> LoggedMessagesByID;
		TArray<FMessageInfo> UncollapsedLoggedMessages;
		FUsdLogManager::FMessageDisplayCallback MessageDisplayCallback;
	};

	void LogMessageInternal(const TSharedRef<FTokenizedMessage>& Message, bool bUserFacing)
	{
		bool bFoundSimilar = false;
		{
			FScopeLock Lock(&CombinedMessagesLock);
			if (CombinedMessages)
			{
				bFoundSimilar = CombinedMessages->AppendMessage(Message, bUserFacing);
			}
		}

		if (!bFoundSimilar)
		{
			// Log immediately, so that our messages are interleaved with any other non-USD logged messages
			// naturally, and are present on the output log already in case we crash after this.
			//
			// We'll skip messages here if they're similar, but if we do it's because we have a CombinedMessages,
			// and when that struct is destroyed it will output an additional message to the Output Log letting
			// the user know that some messages were skipped and what they can do about it
			SendMessageToOutputLog(*Message);
		}
	}
#endif	  // USE_USD_SDK
}	 // namespace UE::USDErrorUtils::Private

void FUsdLogManager::Log(EMessageSeverity::Type Severity, const FString& Message, uint32 MessageID)
{
#if USE_USD_SDK
	using namespace UE::USDErrorUtils::Private;

	TSharedRef<FTokenizedMessage> TokenizedMessage = FTokenizedMessage::Create(Severity, FText::FromString(Message));
	TokenizedMessage->SetIdentifier(GenerateMessageIdentifier(MessageID));

	const bool bUserFacing = false;
	LogMessageInternal(TokenizedMessage, bUserFacing);
#endif	  // USE_USD_SDK
}

void FUsdLogManager::Log(EMessageSeverity::Type Severity, const FText& Message, uint32 MessageID)
{
#if USE_USD_SDK
	using namespace UE::USDErrorUtils::Private;

	TSharedRef<FTokenizedMessage> TokenizedMessage = FTokenizedMessage::Create(Severity, Message);
	TokenizedMessage->SetIdentifier(GenerateMessageIdentifier(MessageID));

	const bool bUserFacing = true;
	LogMessageInternal(TokenizedMessage, bUserFacing);
#endif	  // USE_USD_SDK
}

bool FUsdLogManager::HasAccumulatedErrors()
{
#if USE_USD_SDK
	using namespace UE::USDErrorUtils::Private;

	FScopeLock Lock(&CombinedMessagesLock);

	if (CombinedMessages.IsValid())
	{
		return CombinedMessages->HasErrorsOrWarnings();
	}
#endif	  // USE_USD_SDK

	return false;
}

void FUsdLogManager::RegisterDiagnosticDelegate()
{
#if USE_USD_SDK
	using namespace UE::USDErrorUtils::Private;

	if (DiagnosticDelegate.IsValid())
	{
		UnregisterDiagnosticDelegate();
	}

	DiagnosticDelegate = MakeUnique<FUsdDiagnosticDelegate>();

	pxr::TfDiagnosticMgr& DiagMgr = pxr::TfDiagnosticMgr::GetInstance();
	DiagMgr.AddDelegate(DiagnosticDelegate.Get());
#endif	  // USE_USD_SDK
}

void FUsdLogManager::UnregisterDiagnosticDelegate()
{
#if USE_USD_SDK
	using namespace UE::USDErrorUtils::Private;

	if (!DiagnosticDelegate.IsValid())
	{
		return;
	}

	pxr::TfDiagnosticMgr& DiagMgr = pxr::TfDiagnosticMgr::GetInstance();
	DiagMgr.RemoveDelegate(DiagnosticDelegate.Get());

	DiagnosticDelegate = nullptr;
#endif	  // USE_USD_SDK
}

void FUsdLogManager::DisplayMessagesOnMessageLog(const TArray<TSharedRef<FTokenizedMessage>>& Messages)
{
#if WITH_EDITOR
	if (Messages.Num() == 0)
	{
		return;
	}

	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	TSharedRef<IMessageLogListing> LogListing = MessageLogModule.GetLogListing(TEXT("USD"));

	// We'll output the messages to the output log ourselves from LogMessageInternal()
	const bool bMirrorToOutputLog = false;
	LogListing->AddMessages(Messages, bMirrorToOutputLog);

	// Force display even if all we have are info level messages.
	// This is also handy because it also outputs them on the output log for us
	const bool bForce = true;
	LogListing->NotifyIfAnyMessages(	//
		LOCTEXT("Log", "The previous USD operation produced some log messages."),
		EMessageSeverity::Info,
		bForce
	);
#endif	  // WITH_EDITOR
}

FScopedUsdMessageLog::FScopedUsdMessageLog(FUsdLogManager::FMessageDisplayCallback InCallback)
{
#if USE_USD_SDK
	using namespace UE::USDErrorUtils::Private;

	FScopeLock Lock(&CombinedMessagesLock);

	if (++CombinedMessagesRefCount == 1)
	{
		CombinedMessages = MakeUnique<FUsdCombinedMessages>(InCallback);
		PushErrorMark();
	}

	check(CombinedMessagesRefCount < MAX_int32);
#endif	  // USE_USD_SDK
}

FScopedUsdMessageLog::~FScopedUsdMessageLog()
{
#if USE_USD_SDK
	using namespace UE::USDErrorUtils::Private;

	FScopeLock Lock(&CombinedMessagesLock);

	if (--CombinedMessagesRefCount == 0)
	{
		PopErrorMark();
		CombinedMessages.Reset();
	}
#endif	  // USE_USD_SDK
}

// Deprecated
void UsdUtils::StartMonitoringErrors()
{
}

// Deprecated
TArray<FString> UsdUtils::GetErrorsAndStopMonitoring()
{
	return {};
}

// Deprecated
bool UsdUtils::ShowErrorsAndStopMonitoring(const FText& ToastMessage)
{
	return false;
}

// Deprecated
void FUsdLogManager::LogMessage(EMessageSeverity::Type Severity, const FText& Message)
{
#if USE_USD_SDK
	Log(Severity, Message, 0);
#endif	  // USE_USD_SDK
}

// Deprecated
void FUsdLogManager::LogMessage(const TSharedRef<FTokenizedMessage>& Message)
{
#if USE_USD_SDK
	using namespace UE::USDErrorUtils::Private;

	const bool bUserFacing = true;
	LogMessageInternal(Message, bUserFacing);
#endif	  // USE_USD_SDK
}

// Deprecated
void FUsdLogManager::EnableMessageLog()
{
}

// Deprecated
void FUsdLogManager::DisableMessageLog()
{
}

#undef LOCTEXT_NAMESPACE
