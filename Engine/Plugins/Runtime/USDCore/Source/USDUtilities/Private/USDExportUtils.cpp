// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDExportUtils.h"

#include "Misc/Paths.h"

namespace UE::USDExportUtils::Private
{
	static int32 UniquePathScopeWhenNonZero = 0;
	static TSet<FString> UniquePathsForExport;
}

void UsdUnreal::ExportUtils::BeginUniquePathScope()
{
	using namespace UE::USDExportUtils::Private;

	++UniquePathScopeWhenNonZero;
}

void UsdUnreal::ExportUtils::EndUniquePathScope()
{
	using namespace UE::USDExportUtils::Private;

	--UniquePathScopeWhenNonZero;

	if (UniquePathScopeWhenNonZero == 0)
	{
		UniquePathsForExport.Reset();
	}
}

UsdUnreal::ExportUtils::FUniquePathScope::FUniquePathScope()
{
	BeginUniquePathScope();
}

UsdUnreal::ExportUtils::FUniquePathScope::~FUniquePathScope()
{
	EndUniquePathScope();
}

FString UsdUnreal::ExportUtils::GetUniqueFilePathForExport(const FString& DesiredPathWithExtension)
{
	using namespace UE::USDExportUtils::Private;

	FString SanitizedPath = DesiredPathWithExtension;
	SanitizeFilePath(SanitizedPath);

	if (UniquePathScopeWhenNonZero == 0)
	{
		// Not in a unique path scope --> Just return the path directly
		return SanitizedPath;
	}

	if (!UniquePathsForExport.Contains(SanitizedPath))
	{
		UniquePathsForExport.Add(SanitizedPath);
		return SanitizedPath;
	}

	FString Path;
	FString FileName;
	FString Extension;
	FPaths::Split(SanitizedPath, Path, FileName, Extension);
	FString Prefix = FPaths::Combine(Path, FileName);

	int32 Index = 0;
	FString Result = FString::Printf(TEXT("%s_%d.%s"), *Prefix, Index, *Extension);
	while (UniquePathsForExport.Contains(Result))
	{
		Result = FString::Printf(TEXT("%s_%d.%s"), *Prefix, ++Index, *Extension);
	}

	UniquePathsForExport.Add(Result);
	return Result;
}

void UsdUnreal::ExportUtils::SanitizeFilePath(FString& Path)
{
	FPaths::NormalizeFilename(Path);
	FPaths::RemoveDuplicateSlashes(Path);
	FPaths::CollapseRelativeDirectories(Path);
}
