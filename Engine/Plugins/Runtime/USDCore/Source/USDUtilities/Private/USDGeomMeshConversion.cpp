// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "USDGeomMeshConversion.h"

#include "UnrealUSDWrapper.h"
#include "USDAttributeUtils.h"
#include "USDClassesModule.h"
#include "USDConversionUtils.h"
#include "USDDrawModeComponent.h"
#include "USDErrorUtils.h"
#include "USDLayerUtils.h"
#include "USDMemory.h"
#include "USDObjectUtils.h"
#include "USDPrimConversion.h"
#include "USDProjectSettings.h"
#include "USDShadeConversion.h"
#include "USDSkeletalDataConversion.h"
#include "USDTypesConversion.h"

#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdAttribute.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdStage.h"

#include "Animation/AnimSequence.h"
#include "Animation/SkeletalMeshActor.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "GeometryCache.h"
#include "GeometryCacheHelpers.h"
#include "GeometryCacheMeshData.h"
#include "GeometryCacheTrack.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "MeshDescription.h"
#include "Misc/Paths.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "SkeletalMeshAttributes.h"
#include "SkeletalRenderPublic.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"
#include "StaticMeshResources.h"

#if WITH_EDITOR
#include "MaterialEditingLibrary.h"
#endif	  // WITH_EDITOR

#include "USDIncludesStart.h"
#include "opensubdiv/far/primvarRefiner.h"
#include "opensubdiv/far/topologyRefiner.h"
#include "pxr/imaging/geomUtil/capsuleMeshGenerator.h"
#include "pxr/imaging/geomUtil/cylinderMeshGenerator.h"
#include "pxr/imaging/pxOsd/meshTopology.h"
#include "pxr/imaging/pxOsd/refinerFactory.h"
#include "pxr/usd/sdf/layerUtils.h"
#include "pxr/usd/sdf/types.h"
#include "pxr/usd/usd/editContext.h"
#include "pxr/usd/usd/prim.h"
#include "pxr/usd/usd/primRange.h"
#include "pxr/usd/usdGeom/capsule.h"
#include "pxr/usd/usdGeom/capsule_1.h"
#include "pxr/usd/usdGeom/cone.h"
#include "pxr/usd/usdGeom/cube.h"
#include "pxr/usd/usdGeom/cylinder.h"
#include "pxr/usd/usdGeom/cylinder_1.h"
#include "pxr/usd/usdGeom/gprim.h"
#include "pxr/usd/usdGeom/mesh.h"
#include "pxr/usd/usdGeom/plane.h"
#include "pxr/usd/usdGeom/pointInstancer.h"
#include "pxr/usd/usdGeom/primvarsAPI.h"
#include "pxr/usd/usdGeom/sphere.h"
#include "pxr/usd/usdGeom/subset.h"
#include "pxr/usd/usdGeom/tokens.h"
#include "pxr/usd/usdPhysics/collisionAPI.h"
#include "pxr/usd/usdPhysics/meshCollisionAPI.h"
#include "pxr/usd/usdPhysics/tokens.h"
#include "pxr/usd/usdShade/material.h"
#include "pxr/usd/usdShade/materialBindingAPI.h"
#include "pxr/usd/usdShade/tokens.h"
#include "pxr/usd/usdSkel/bindingAPI.h"
#include "pxr/usd/usdSkel/root.h"
#include "pxr/usd/usdSkel/skeletonQuery.h"
#include "pxr/usd/usdSkel/skinningQuery.h"
#include "pxr/usd/usdSkel/utils.h"
#include "pxr/usd/usdVol/volume.h"
#include "pxr/usdImaging/usdImaging/implicitSurfaceMeshUtils.h"
#include "USDIncludesEnd.h"

#define LOCTEXT_NAMESPACE "USDGeomMeshConversion"

static int32 GMaxInstancesPerPointInstancer = -1;
static FAutoConsoleVariableRef CVarMaxInstancesPerPointInstancer(
	TEXT("USD.MaxInstancesPerPointInstancer"),
	GMaxInstancesPerPointInstancer,
	TEXT("We will only parse up to this many instances from any point instancer when reading from USD to UE. Set this to -1 to disable this limit.")
);

static bool GExportNaniteSourceMeshData = true;
static FAutoConsoleVariableRef CVarExportNaniteSourceMeshData(
	TEXT("USD.ExportNaniteSourceMeshData"),
	GExportNaniteSourceMeshData,
	TEXT(
		"Try using Nanite hi-res MeshDescription data when exporting static meshes with the bExportStaticMeshSourceData option enabled. If false, it means we will always just use the StaticMesh's LOD source MeshDescription data instead."
	)
);

static bool GIgnoreNormalsWhenSubdividing = true;
static FAutoConsoleVariableRef CVarIgnoreNormalsWhenSubdividing(
	TEXT("USD.Subdiv.IgnoreNormalsWhenSubdividing"),
	GIgnoreNormalsWhenSubdividing,
	TEXT("This being true means that whenever we subdivide a mesh we fully ignore the authored normals (if any) and recompute new normals. If this "
		 "is false we will try interpolating the normals during subdivision like a regular primvar")
);

static const FString MaxUsdSubdivLevelCvarName = TEXT("USD.Subdiv.MaxSubdivLevel");
static int32 GMaxSubdivLevel = 6;
static FAutoConsoleVariableRef CVarMaxSubdivLevel(
	*MaxUsdSubdivLevelCvarName,
	GMaxSubdivLevel,
	TEXT("Maximum allowed level of subdivision (1 means a single iteration of subdivision)")
);

static float GMeshNormalRepairThreshold = 0.05f;
static FAutoConsoleVariableRef CVarMeshNormalRepairThreshold(
	TEXT("USD.MeshNormalRepairThreshold"),
	GMeshNormalRepairThreshold,
	TEXT("We will try repairing up to this fraction of a Mesh's normals when invalid. If a Mesh has more invalid normals than this, we will "
		 "recompute all of them. Defaults to 0.05 (5% of all normals).")
);

static bool GSkipMeshTangentComputation = false;
static FAutoConsoleVariableRef CVarSkipMeshTangentComputation(
	TEXT("USD.SkipMeshTangentComputation"),
	GSkipMeshTangentComputation,
	TEXT("Skip computing tangents for meshes. With meshes with a huge numer of vertices, it can take a very long time to compute them.")
);

static bool GTriangulateQuadsAndNgons = true;
static FAutoConsoleVariableRef CVarTriangulateQuadsAndNgons(
	TEXT("USD.TriangulateQuadsAndNgons"),
	GTriangulateQuadsAndNgons,
	TEXT("Triangulate quad and ngon faces when generating static and skeletal meshes.")
);

namespace UE::UsdGeomMeshConversion::Private
{
	static const FString DisplayColorID = TEXT("!DisplayColor");

	// Like with the other primitive shapes we'll be making a single "default capsule" that we scale around with a transform
	// to represent capsules of different radii and heights. This lets us animate these attributes on the LevelSequence using a static
	// mesh, but could lead to distorted shapes for aggressive transforms. Because of that, we use these defaults that specify
	// a very thin capsule with not many cap segments, as that visually minimizes the distortions we get under these transforms
	static const double DefaultCapsuleMeshRadius = 0.125;
	static const double DefaultCapsuleMeshHeight = 0.75;
	static const double DefaultCapsuleNumRadialSegments = 10;
	static const double DefaultCapsuleNumCapAxialSegments = 2;

	static const double DefaultConeMeshRadius = 0.5;
	static const double DefaultConeMeshHeight = 1.0;

	static const double DefaultCubeMeshSize = 1.0;

	static const double DefaultCylinderMeshRadius = 0.50;
	static const double DefaultCylinderMeshHeight = 1.00;
	static const double DefaultCylinderNumRadialSegments = 10;

	static const double DefaultSphereMeshRadius = 0.5;

	static const double DefaultPlaneMeshWidth = 1.0;
	static const double DefaultPlaneMeshLength = 1.0;

	void WarnIfAnimated(pxr::UsdAttribute Attr)
	{
		if (Attr.ValueMightBeTimeVarying())
		{
			USD_LOG_USERWARNING(FText::Format(
				LOCTEXT("AnimatedAttributeWarning", "Animation of the '{0}' attribute for prim '{1}' is not supported!"),
				FText::FromString(UsdToUnreal::ConvertString(Attr.GetName())),
				FText::FromString(UsdToUnreal::ConvertPath(Attr.GetPrim().GetPrimPath()))
			));
		}
	}

	pxr::TfToken GetAxisWithValidation(pxr::UsdAttribute Attr)
	{
		WarnIfAnimated(Attr);

		pxr::TfToken Axis = UsdUtils::GetUsdValue<pxr::TfToken>(Attr, pxr::UsdTimeCode::Default());
		if (Axis != pxr::UsdGeomTokens->x && Axis != pxr::UsdGeomTokens->y && Axis != pxr::UsdGeomTokens->z)
		{
			USD_LOG_USERWARNING(FText::Format(
				LOCTEXT(
					"UnsupportedAxis",
					"Attribute '{0}' has unsupported value '{1}'! Supported values are \"{2}\",\"{3}\" and \"{4}\""
				),
				FText::FromString(UsdToUnreal::ConvertPath(Attr.GetPath())),
				FText::FromString(UsdToUnreal::ConvertToken(Axis)),
				FText::FromString(UsdToUnreal::ConvertToken(pxr::UsdGeomTokens->x)),
				FText::FromString(UsdToUnreal::ConvertToken(pxr::UsdGeomTokens->y)),
				FText::FromString(UsdToUnreal::ConvertToken(pxr::UsdGeomTokens->z))
			));
		}
		return Axis;
	}

	/**
	 * Tries fetching the default opinion for the attribute if one is authored, otherwise fetches the EarliestTime opinion.
	 *
	 * This is useful because sometimes we want to explicitly fetch the default opinion to place it on the level / use it as a base
	 * case (like for the primitives), but otherwise check the regular timeSample/spline opinions instead of using the schema fallback values.
	 */
	template<typename T>
	T GetDefaultOrEarliest(const pxr::UsdAttribute& Attr)
	{
		pxr::UsdTimeCode TimeCodeToUse = pxr::UsdTimeCode::EarliestTime();

		pxr::UsdResolveInfo Info = Attr.GetResolveInfo(pxr::UsdTimeCode::Default());
		if (Info.GetSource() == pxr::UsdResolveInfoSourceDefault)
		{
			TimeCodeToUse = pxr::UsdTimeCode::Default();
		}

		return UsdUtils::GetUsdValue<T>(Attr, TimeCodeToUse);
	};

	int32 GetPrimValueIndex(const pxr::TfToken& InterpType, const int32 VertexIndex, const int32 VertexInstanceIndex, const int32 PolygonIndex)
	{
		if (InterpType == pxr::UsdGeomTokens->vertex)
		{
			return VertexIndex;
		}
		else if (InterpType == pxr::UsdGeomTokens->varying)
		{
			return VertexIndex;
		}
		else if (InterpType == pxr::UsdGeomTokens->faceVarying)
		{
			return VertexInstanceIndex;
		}
		else if (InterpType == pxr::UsdGeomTokens->uniform)
		{
			return PolygonIndex;
		}
		else			 /* if ( InterpType == pxr::UsdGeomTokens->constant ) */
		{
			return 0;	 // return index 0 for constant or any other unsupported cases
		}
	}

	pxr::TfToken GetAttrInterpolation(const pxr::UsdAttribute& Attr, const pxr::TfToken& DefaultValue = pxr::UsdGeomTokens->constant)
	{
		if (!Attr)
		{
			return DefaultValue;
		}

		pxr::TfToken RetrievedValue;
		const bool bGotInterpolationValue = Attr.GetMetadata(pxr::UsdGeomTokens->interpolation, &RetrievedValue);

		// If we have an authored value just go ahead and use that
		if (Attr.HasAuthoredMetadata(pxr::UsdGeomTokens->interpolation) && bGotInterpolationValue)
		{
			return RetrievedValue;
		}

		// Otherwise if our attribute describes an array with a single element and has no authored interpolation assume "constant", as
		// it's impossible for any other interpolation type to be valid. usdview does this too.
		// Note we try our best to get anything here and also check timeSampled values in case our default time Get() fails
		pxr::VtValue TypeErasedValue;
		if (Attr.Get(&TypeErasedValue) || Attr.Get(&TypeErasedValue, pxr::UsdTimeCode::EarliestTime()))
		{
			if (TypeErasedValue.IsArrayValued() && TypeErasedValue.GetArraySize() == 1)
			{
				return pxr::UsdGeomTokens->constant;
			}
		}
		// If we couldn't get any actual value for the attribute whatsoever then pretend it doesn't have a valid value for interpolation
		// either. We need use this because if SubdivideMeshData sees that an attribute has e.g. "vertex" interpolation, it will allocate
		// and try generating one value for it for every vertex... if we don't have any value to begin with we'll just end up with a zero
		// value for each instead
		else
		{
			return {};
		}

		// Otherwise if we don't have an authored value but did manage to get value for interpolation somehow
		// (maybe as a fallback?) then return that
		if (bGotInterpolationValue)
		{
			return RetrievedValue;
		}

		return DefaultValue;
	}

	pxr::TfToken GetGprimOrientation(const pxr::UsdGeomGprim& Gprim, pxr::UsdTimeCode TimeCode)
	{
		if (pxr::UsdAttribute Attr = Gprim.GetOrientationAttr())
		{
			pxr::TfToken Orientation;
			if (Attr.Get(&Orientation, TimeCode))
			{
				return Orientation;
			}
		}
		return pxr::UsdGeomTokens->rightHanded;
	}

	pxr::VtArray<int> GetFaceVertexCounts(const pxr::UsdPrim& UsdPrim, pxr::UsdTimeCode TimeCode)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GetFaceVertexCounts);

		if (pxr::UsdGeomMesh Mesh = pxr::UsdGeomMesh{UsdPrim})
		{
			pxr::UsdAttribute Attr = Mesh.GetFaceVertexCountsAttr();

			pxr::VtArray<int> Result;
			if (Attr && Attr.Get(&Result, TimeCode))
			{
				return Result;
			}
		}

		const pxr::PxOsdMeshTopology* Topology = nullptr;
		TOptional<pxr::PxOsdMeshTopology> CreatedTopology;

		if (pxr::UsdGeomCapsule Capsule = pxr::UsdGeomCapsule{UsdPrim})
		{
			Topology = &pxr::UsdImagingGetCapsuleMeshTopology();
		}
		else if (pxr::UsdGeomCapsule_1 Capsule_1 = pxr::UsdGeomCapsule_1{UsdPrim})
		{
			CreatedTopology = pxr::GeomUtilCapsuleMeshGenerator::GenerateTopology(
				DefaultCapsuleNumRadialSegments,
				DefaultCapsuleNumCapAxialSegments
			);
			Topology = CreatedTopology.GetPtrOrNull();
		}
		else if (pxr::UsdGeomCone Cone = pxr::UsdGeomCone{UsdPrim})
		{
			Topology = &pxr::UsdImagingGetUnitConeMeshTopology();
		}
		else if (pxr::UsdGeomCube Cube = pxr::UsdGeomCube{UsdPrim})
		{
			Topology = &pxr::UsdImagingGetUnitCubeMeshTopology();
		}
		else if (pxr::UsdGeomCylinder Cylinder = pxr::UsdGeomCylinder{UsdPrim})
		{
			Topology = &pxr::UsdImagingGetUnitCylinderMeshTopology();
		}
		else if (pxr::UsdGeomCylinder_1 Cylinder_1 = pxr::UsdGeomCylinder_1{UsdPrim})
		{
			CreatedTopology = pxr::GeomUtilCylinderMeshGenerator::GenerateTopology(DefaultCylinderNumRadialSegments);
			Topology = CreatedTopology.GetPtrOrNull();
		}
		else if (pxr::UsdGeomSphere Sphere = pxr::UsdGeomSphere{UsdPrim})
		{
			Topology = &pxr::UsdImagingGetUnitSphereMeshTopology();
		}
		else if (pxr::UsdGeomPlane Plane = pxr::UsdGeomPlane{UsdPrim})
		{
			Topology = &pxr::UsdImagingGetPlaneTopology();
		}
		if (Topology)
		{
			return Topology->GetFaceVertexCounts();
		}

		return {};
	}

	pxr::VtArray<pxr::GfVec3f> GetTwoRadiiCapsuleMeshPoints(pxr::TfToken Axis, float RadiusBottom, float RadiusTop, float Height)
	{
		TOptional<pxr::GfMatrix4d> Transform;
		if (Axis == pxr::UsdGeomTokens->x)
		{
			Transform = pxr::GfMatrix4d{0.0, 1.0, 0.0, 0.0, 0.0, 0.0, 1.0, 0.0, 1.0, 0.0, 0.0, 0.0, 0.0, 0.0, 0.0, 1.0};
		}
		else if (Axis == pxr::UsdGeomTokens->y)
		{
			Transform = pxr::GfMatrix4d{0.0, 0.0, 1.0, 0.0, 1.0, 0.0, 0.0, 0.0, 0.0, 1.0, 0.0, 0.0, 0.0, 0.0, 0.0, 1.0};
		}

		const size_t NumPoints = pxr::GeomUtilCapsuleMeshGenerator::ComputeNumPoints(
			DefaultCapsuleNumRadialSegments,
			DefaultCapsuleNumCapAxialSegments
		);
		pxr::VtArray<pxr::GfVec3f> Points{NumPoints};

		pxr::GeomUtilCapsuleMeshGenerator::GeneratePoints(
			Points.begin(),
			DefaultCapsuleNumRadialSegments,
			DefaultCapsuleNumCapAxialSegments,
			RadiusBottom,
			RadiusTop,
			Height,
			Transform.GetPtrOrNull()
		);

		return Points;
	}

	pxr::VtArray<pxr::GfVec3f> GetUnitCylinderMeshPoints(pxr::TfToken Axis)
	{
		if (Axis == pxr::UsdGeomTokens->x)
		{
			static const pxr::VtArray<pxr::GfVec3f> XCylinder = []()
			{
				// The USD cylinder is aligned to the z axis by default
				pxr::VtArray<pxr::GfVec3f> Points = pxr::UsdImagingGetUnitCylinderMeshPoints();

				pxr::GfMatrix4d ZToXAxis{0.0, 1.0, 0.0, 0.0, 0.0, 0.0, 1.0, 0.0, 1.0, 0.0, 0.0, 0.0, 0.0, 0.0, 0.0, 1.0};

				for (pxr::GfVec3f& Point : Points)
				{
					Point = pxr::GfVec3f(ZToXAxis.Transform(Point));
				}

				return Points;
			}();
			return XCylinder;
		}
		else if (Axis == pxr::UsdGeomTokens->y)
		{
			static const pxr::VtArray<pxr::GfVec3f> YCylinder = []()
			{
				// The USD cylinder is aligned to the z axis by default
				pxr::VtArray<pxr::GfVec3f> Points = pxr::UsdImagingGetUnitCylinderMeshPoints();

				pxr::GfMatrix4d ZToYAxis{0.0, 0.0, 1.0, 0.0, 1.0, 0.0, 0.0, 0.0, 0.0, 1.0, 0.0, 0.0, 0.0, 0.0, 0.0, 1.0};

				for (pxr::GfVec3f& Point : Points)
				{
					Point = pxr::GfVec3f(ZToYAxis.Transform(Point));
				}

				return Points;
			}();
			return YCylinder;
		}
		else
		{
			return pxr::UsdImagingGetUnitCylinderMeshPoints();
		}
	}

	pxr::VtArray<pxr::GfVec3f> GetTwoRadiiCylinderMeshPoints(pxr::TfToken Axis, float RadiusBottom, float RadiusTop, float Height)
	{
		TOptional<pxr::GfMatrix4d> Transform;
		if (Axis == pxr::UsdGeomTokens->x)
		{
			Transform = pxr::GfMatrix4d{0.0, 1.0, 0.0, 0.0, 0.0, 0.0, 1.0, 0.0, 1.0, 0.0, 0.0, 0.0, 0.0, 0.0, 0.0, 1.0};
		}
		else if (Axis == pxr::UsdGeomTokens->y)
		{
			Transform = pxr::GfMatrix4d{0.0, 0.0, 1.0, 0.0, 1.0, 0.0, 0.0, 0.0, 0.0, 1.0, 0.0, 0.0, 0.0, 0.0, 0.0, 1.0};
		}

		const size_t NumPoints = pxr::GeomUtilCylinderMeshGenerator::ComputeNumPoints(DefaultCylinderNumRadialSegments);
		pxr::VtArray<pxr::GfVec3f> Points{NumPoints};

		pxr::GeomUtilCylinderMeshGenerator::GeneratePoints(
			Points.begin(),
			DefaultCylinderNumRadialSegments,
			RadiusBottom,
			RadiusTop,
			Height,
			Transform.GetPtrOrNull()
		);

		return Points;
	}

	pxr::VtArray<pxr::GfVec3f> GetUnitConeMeshPoints(pxr::TfToken Axis)
	{
		if (Axis == pxr::UsdGeomTokens->x)
		{
			static const pxr::VtArray<pxr::GfVec3f> XCylinder = []()
			{
				// The USD cone is aligned to the z axis by default
				pxr::VtArray<pxr::GfVec3f> Points = pxr::UsdImagingGetUnitConeMeshPoints();

				pxr::GfMatrix4d ZToXAxis{0.0, 1.0, 0.0, 0.0, 0.0, 0.0, 1.0, 0.0, 1.0, 0.0, 0.0, 0.0, 0.0, 0.0, 0.0, 1.0};

				for (pxr::GfVec3f& Point : Points)
				{
					Point = pxr::GfVec3f(ZToXAxis.Transform(Point));
				}

				return Points;
			}();
			return XCylinder;
		}
		else if (Axis == pxr::UsdGeomTokens->y)
		{
			static const pxr::VtArray<pxr::GfVec3f> YCylinder = []()
			{
				// The USD cone is aligned to the z axis by default
				pxr::VtArray<pxr::GfVec3f> Points = pxr::UsdImagingGetUnitConeMeshPoints();

				pxr::GfMatrix4d ZToYAxis{0.0, 0.0, 1.0, 0.0, 1.0, 0.0, 0.0, 0.0, 0.0, 1.0, 0.0, 0.0, 0.0, 0.0, 0.0, 1.0};

				for (pxr::GfVec3f& Point : Points)
				{
					Point = pxr::GfVec3f(ZToYAxis.Transform(Point));
				}

				return Points;
			}();
			return YCylinder;
		}
		else
		{
			return pxr::UsdImagingGetUnitConeMeshPoints();
		}
	}

	int32 GetLODIndexFromName(const std::string& Name)
	{
		const std::string LODString = UnrealIdentifiers::LOD.GetString();

		// True if Name does not start with "LOD"
		if (Name.rfind(LODString, 0) != 0)
		{
			return INDEX_NONE;
		}

		// After LODString there should be only numbers
		if (Name.find_first_not_of("0123456789", LODString.size()) != std::string::npos)
		{
			return INDEX_NONE;
		}

		const int Base = 10;
		char** EndPtr = nullptr;
		return std::strtol(Name.c_str() + LODString.size(), EndPtr, Base);
	}

	bool ConvertStaticMeshLOD(
		const FStaticMeshLODResources& LODRenderMesh,
		pxr::UsdGeomMesh& UsdMesh,
		const pxr::UsdTimeCode TimeCode,
		const TArray<FString>& MaterialAssignments,
		pxr::UsdPrim PrimToReceiveMaterialAssignments
	)
	{
		pxr::UsdPrim MeshPrim = UsdMesh.GetPrim();
		pxr::UsdStageRefPtr Stage = MeshPrim.GetStage();
		if (!Stage)
		{
			return false;
		}
		const FUsdStageInfo StageInfo{Stage};

		// Vertices
		{
			const int32 VertexCount = LODRenderMesh.VertexBuffers.StaticMeshVertexBuffer.GetNumVertices();

			// Points
			{
				pxr::UsdAttribute Points = UsdMesh.CreatePointsAttr();
				if (Points)
				{
					pxr::VtArray<pxr::GfVec3f> PointsArray;
					PointsArray.reserve(VertexCount);

					for (int32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
					{
						FVector VertexPosition = (FVector)LODRenderMesh.VertexBuffers.PositionVertexBuffer.VertexPosition(VertexIndex);
						PointsArray.push_back(UnrealToUsd::ConvertVectorFloat(StageInfo, VertexPosition));
					}

					Points.Set(PointsArray, TimeCode);
				}
			}

			// Normals
			{
				// We need to emit this if we're writing normals (which we always are) because any DCC that can
				// actually subdivide (like usdview) will just discard authored normals and fully recompute them
				// on-demand in case they have a valid subdivision scheme (which is the default state).
				// Reference: https://graphics.pixar.com/usd/release/api/class_usd_geom_mesh.html#UsdGeom_Mesh_Normals
				if (pxr::UsdAttribute SubdivisionAttr = UsdMesh.CreateSubdivisionSchemeAttr())
				{
					ensure(SubdivisionAttr.Set(pxr::UsdGeomTokens->none));
				}

				pxr::UsdAttribute NormalsAttribute = UsdMesh.CreateNormalsAttr();
				if (NormalsAttribute)
				{
					pxr::VtArray<pxr::GfVec3f> Normals;
					Normals.reserve(VertexCount);

					for (int32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
					{
						FVector VertexNormal = (FVector4)LODRenderMesh.VertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(VertexIndex);
						Normals.push_back(UnrealToUsd::ConvertVectorFloat(StageInfo, VertexNormal));
					}

					NormalsAttribute.Set(Normals, TimeCode);
				}
			}

			// UVs
			{
				const int32 TexCoordSourceCount = LODRenderMesh.VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords();

				for (int32 TexCoordSourceIndex = 0; TexCoordSourceIndex < TexCoordSourceCount; ++TexCoordSourceIndex)
				{
					pxr::TfToken UsdUVSetName = UsdUtils::GetUVSetName(TexCoordSourceIndex).Get();

					pxr::UsdGeomPrimvar PrimvarST = pxr::UsdGeomPrimvarsAPI(MeshPrim).CreatePrimvar(
						UsdUVSetName,
						pxr::SdfValueTypeNames->TexCoord2fArray,
						pxr::UsdGeomTokens->vertex
					);

					if (PrimvarST)
					{
						pxr::VtVec2fArray UVs;

						for (int32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
						{
							FVector2D TexCoord = FVector2D(
								LODRenderMesh.VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(VertexIndex, TexCoordSourceIndex)
							);
							TexCoord[1] = 1.f - TexCoord[1];

							UVs.push_back(UnrealToUsd::ConvertVectorFloat(TexCoord));
						}

						PrimvarST.Set(UVs, TimeCode);
					}
				}
			}

			// Vertex colors
			if (LODRenderMesh.bHasColorVertexData)
			{
				pxr::UsdGeomPrimvar DisplayColorPrimvar = UsdMesh.CreateDisplayColorPrimvar(pxr::UsdGeomTokens->vertex);
				pxr::UsdGeomPrimvar DisplayOpacityPrimvar = UsdMesh.CreateDisplayOpacityPrimvar(pxr::UsdGeomTokens->vertex);

				if (DisplayColorPrimvar)
				{
					pxr::VtArray<pxr::GfVec3f> DisplayColors;
					DisplayColors.reserve(VertexCount);

					pxr::VtArray<float> DisplayOpacities;
					DisplayOpacities.reserve(VertexCount);

					for (int32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
					{
						const FColor& VertexColor = LODRenderMesh.VertexBuffers.ColorVertexBuffer.VertexColor(VertexIndex);

						pxr::GfVec4f Color = UnrealToUsd::ConvertColor(VertexColor);
						DisplayColors.push_back(pxr::GfVec3f(Color[0], Color[1], Color[2]));
						DisplayOpacities.push_back(Color[3]);
					}

					DisplayColorPrimvar.Set(DisplayColors, TimeCode);
					DisplayOpacityPrimvar.Set(DisplayOpacities, TimeCode);
				}
			}
		}

		// Faces
		{
			const int32 FaceCount = LODRenderMesh.GetNumTriangles();

			// Face Vertex Counts
			{
				pxr::UsdAttribute FaceCountsAttribute = UsdMesh.CreateFaceVertexCountsAttr();

				if (FaceCountsAttribute)
				{
					pxr::VtArray<int> FaceVertexCounts;
					FaceVertexCounts.reserve(FaceCount);

					for (int32 FaceIndex = 0; FaceIndex < FaceCount; ++FaceIndex)
					{
						FaceVertexCounts.push_back(3);
					}

					FaceCountsAttribute.Set(FaceVertexCounts, TimeCode);
				}
			}

			// Face Vertex Indices
			{
				pxr::UsdAttribute FaceVertexIndicesAttribute = UsdMesh.GetFaceVertexIndicesAttr();

				if (FaceVertexIndicesAttribute)
				{
					FIndexArrayView Indices = LODRenderMesh.IndexBuffer.GetArrayView();
					ensure(Indices.Num() == FaceCount * 3);

					pxr::VtArray<int> FaceVertexIndices;
					FaceVertexIndices.reserve(FaceCount * 3);

					for (int32 Index = 0; Index < FaceCount * 3; ++Index)
					{
						FaceVertexIndices.push_back(Indices[Index]);
					}

					FaceVertexIndicesAttribute.Set(FaceVertexIndices, TimeCode);
				}
			}
		}

		// Material assignments
		{
			bool bHasUEMaterialAssignements = false;

			TArray<FString> UnrealMaterialsForLOD;
			for (const FStaticMeshSection& Section : LODRenderMesh.Sections)
			{
				if (MaterialAssignments.IsValidIndex(Section.MaterialIndex))
				{
					UnrealMaterialsForLOD.Add(MaterialAssignments[Section.MaterialIndex]);
					bHasUEMaterialAssignements = true;
				}
				else
				{
					// Keep unrealMaterials with the same number of elements as our MaterialIndices expect
					UnrealMaterialsForLOD.Add(TEXT(""));
				}
			}

			// This LOD has a single material assignment, just create/bind an UnrealMaterial child prim directly
			if (bHasUEMaterialAssignements && UnrealMaterialsForLOD.Num() == 1)
			{
				UsdUtils::AuthorUnrealMaterialBinding(PrimToReceiveMaterialAssignments, UnrealMaterialsForLOD[0]);
			}
			// Multiple material assignments to the same LOD (and so the same mesh prim). Need to create a GeomSubset for each UE mesh section
			else if (UnrealMaterialsForLOD.Num() > 1)
			{
				// Need to fetch all triangles of a section, and add their indices
				for (int32 SectionIndex = 0; SectionIndex < LODRenderMesh.Sections.Num(); ++SectionIndex)
				{
					const FStaticMeshSection& Section = LODRenderMesh.Sections[SectionIndex];

					// Note that we will continue authoring the GeomSubsets on even if we later find out we have no material assignment (just
					// "") for this section, so as to satisfy the "partition" family condition (below)
					pxr::UsdPrim GeomSubsetPrim = Stage->DefinePrim(
						MeshPrim.GetPath().AppendPath(pxr::SdfPath("Section" + std::to_string(SectionIndex))),
						UnrealToUsd::ConvertToken(TEXT("GeomSubset")).Get()
					);

					// MaterialPrim may be in another stage, so we may need another GeomSubset there
					pxr::UsdPrim MaterialGeomSubsetPrim = GeomSubsetPrim;
					if (PrimToReceiveMaterialAssignments.GetStage() != MeshPrim.GetStage())
					{
						MaterialGeomSubsetPrim = PrimToReceiveMaterialAssignments.GetStage()->OverridePrim(
							PrimToReceiveMaterialAssignments.GetPath().AppendPath(pxr::SdfPath("Section" + std::to_string(SectionIndex)))
						);
					}

					pxr::UsdGeomSubset GeomSubsetSchema{GeomSubsetPrim};

					// Element type attribute
					pxr::UsdAttribute ElementTypeAttr = GeomSubsetSchema.CreateElementTypeAttr();
					ElementTypeAttr.Set(pxr::UsdGeomTokens->face, TimeCode);

					// Indices attribute
					const uint32 TriangleCount = Section.NumTriangles;
					const uint32 FirstTriangleIndex = Section.FirstIndex / 3;	 // FirstIndex is the first *vertex* instance index
					pxr::VtArray<int> IndicesAttrValue;
					for (uint32 TriangleIndex = FirstTriangleIndex; TriangleIndex - FirstTriangleIndex < TriangleCount; ++TriangleIndex)
					{
						// Note that we add VertexInstances in sequence to the usda file for the faceVertexInstances attribute, which
						// also constitutes our triangle order
						IndicesAttrValue.push_back(static_cast<int>(TriangleIndex));
					}

					pxr::UsdAttribute IndicesAttr = GeomSubsetSchema.CreateIndicesAttr();
					IndicesAttr.Set(IndicesAttrValue, TimeCode);

					// Family name attribute
					pxr::UsdAttribute FamilyNameAttr = GeomSubsetSchema.CreateFamilyNameAttr();
					FamilyNameAttr.Set(pxr::UsdShadeTokens->materialBind, TimeCode);

					// Family type
					pxr::UsdGeomSubset::SetFamilyType(UsdMesh, pxr::UsdShadeTokens->materialBind, pxr::UsdGeomTokens->partition);

					// material:binding relationship
					UsdUtils::AuthorUnrealMaterialBinding(MaterialGeomSubsetPrim, UnrealMaterialsForLOD[SectionIndex]);
				}
			}
		}

		return true;
	}

	bool ConvertMeshDescription(
		const FMeshDescription& MeshDescription,
		pxr::UsdGeomMesh& UsdMesh,
		const FMatrix& AdditionalTransform,
		const pxr::UsdTimeCode TimeCode,
		const TArray<FString>* MaterialIndexToContentPath = nullptr,
		const TMap<FName, int32>* ImportedMaterialSlotNameToIndex = nullptr,
		pxr::UsdPrim* PrimToReceiveMaterialAssignments = nullptr
	)
	{
		pxr::UsdPrim MeshPrim = UsdMesh.GetPrim();
		pxr::UsdStageRefPtr Stage = MeshPrim.GetStage();
		if (!Stage)
		{
			return false;
		}
		const FUsdStageInfo StageInfo{Stage};

		FStaticMeshConstAttributes Attributes(MeshDescription);
		TVertexAttributesConstRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();
		TPolygonGroupAttributesConstRef<FName> PolygonGroupImportedMaterialSlotNames = Attributes.GetPolygonGroupMaterialSlotNames();
		TVertexInstanceAttributesConstRef<FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals();
		TVertexInstanceAttributesConstRef<FVector4f> VertexInstanceColors = Attributes.GetVertexInstanceColors();
		TVertexInstanceAttributesConstRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();

		const int32 VertexCount = VertexPositions.GetNumElements();
		const int32 VertexInstanceCount = VertexInstanceNormals.GetNumElements();
		const int32 FaceCount = MeshDescription.Polygons().Num();

		//
		// Prepare temp arrays to receive the converted data
		//

		pxr::VtArray<pxr::GfVec3f> PointsArray;
		PointsArray.reserve(VertexCount);

		pxr::VtArray<int> FaceVertexCounts;
		FaceVertexCounts.reserve(FaceCount);

		pxr::VtArray<int> FaceVertexIndices;
		FaceVertexIndices.reserve(VertexInstanceCount);

		pxr::VtArray<pxr::GfVec3f> Normals;
		Normals.reserve(VertexInstanceCount);

		pxr::VtArray<pxr::GfVec3f> DisplayColors;
		pxr::VtArray<float> DisplayOpacities;

		// Check if we'll need face-varying displayColors/opacity or not
		// This is useful because *every* MeshDescription will have one VertexInstanceColor element for each instance
		// with white opaque color, even if the actual source data didn't have anything. We shouldn't emit thousands of "(1, 1, 1)"
		// to the USD file for no reason
		bool bUseConstantColor = true;
		FVector4f FirstColor{1.0f, 1.0f, 1.0f, 1.0f};
		if (VertexInstanceColors.GetNumElements() > 1)
		{
			FirstColor = VertexInstanceColors[FVertexInstanceID(0)];
			for (const FVertexInstanceID InstanceID : MeshDescription.VertexInstances().GetElementIDs())
			{
				const FVector4f& OtherColor = VertexInstanceColors[InstanceID];
				if (OtherColor != FirstColor)
				{
					bUseConstantColor = false;
					break;
				}
			}
		}
		if (bUseConstantColor)
		{
			pxr::GfVec4f Color = UnrealToUsd::ConvertColor(FLinearColor(FirstColor));
			DisplayColors.push_back(pxr::GfVec3f(Color[0], Color[1], Color[2]));
			DisplayOpacities.push_back(Color[3]);
		}
		else
		{
			DisplayColors.reserve(VertexInstanceCount);
			DisplayOpacities.reserve(VertexInstanceCount);
		}

		const int32 NumUVs = VertexInstanceUVs.GetNumChannels();
		TArray<pxr::VtVec2fArray> UVs;
		UVs.SetNum(NumUVs);
		for (int32 UVIndex = 0; UVIndex < NumUVs; ++UVIndex)
		{
			UVs[UVIndex].reserve(VertexInstanceCount);
		}

		// Convert points
		for (const FVertexID VertexID : MeshDescription.Vertices().GetElementIDs())
		{
			FVector UEPosition = AdditionalTransform.TransformPosition((FVector)VertexPositions[VertexID]);
			PointsArray.push_back(UnrealToUsd::ConvertVectorFloat(StageInfo, UEPosition));
		}

		// Convert all vertex instance attributes in one go
		//
		// It's important to emit the polygons exactly in the order that they are in MeshDescription.Polygons() here,
		// because down below when emitting material assignment and UsdGeomSubsets, we'll fetch the indices of these
		// polygons when iterating over the polygon groups, and they are also meant to match the polygon order within
		// MeshDescription.Polygons()
		for (FPolygonID PolygonID : MeshDescription.Polygons().GetElementIDs())
		{
			TArray<FVertexInstanceID> PolygonVertexInstances = MeshDescription.GetPolygonVertexInstances(PolygonID);
			FaceVertexCounts.push_back(static_cast<int>(PolygonVertexInstances.Num()));

			for (const FVertexInstanceID& VertexInstanceID : PolygonVertexInstances)
			{
				int32 VertexIndex = MeshDescription.GetVertexInstanceVertex(VertexInstanceID).GetValue();
				FaceVertexIndices.push_back(static_cast<int>(VertexIndex));

				FVector UENormal = (FVector)VertexInstanceNormals[VertexInstanceID].GetSafeNormal();
				Normals.push_back(UnrealToUsd::ConvertVectorFloat(StageInfo, UENormal));

				if (!bUseConstantColor)
				{
					pxr::GfVec4f Color = UnrealToUsd::ConvertColor(FLinearColor(VertexInstanceColors[VertexInstanceID]));
					DisplayColors.push_back(pxr::GfVec3f(Color[0], Color[1], Color[2]));
					DisplayOpacities.push_back(Color[3]);
				}

				for (int32 UVIndex = 0; UVIndex < NumUVs; ++UVIndex)
				{
					FVector2D UV = FVector2D(VertexInstanceUVs.Get(VertexInstanceID, UVIndex));
					UV[1] = 1.f - UV[1];
					UVs[UVIndex].push_back(UnrealToUsd::ConvertVectorFloat(UV));
				}
			}
		}

		// Create attributes and set converted data into USD
		{
			pxr::UsdAttribute Points = UsdMesh.CreatePointsAttr();
			Points.Set(PointsArray, TimeCode);

			pxr::UsdAttribute FaceCountsAttribute = UsdMesh.CreateFaceVertexCountsAttr();
			FaceCountsAttribute.Set(FaceVertexCounts, TimeCode);

			pxr::UsdAttribute FaceVertexIndicesAttribute = UsdMesh.GetFaceVertexIndicesAttr();
			FaceVertexIndicesAttribute.Set(FaceVertexIndices, TimeCode);

			// We need to emit this if we're writing normals (which we always are) because any DCC that can
			// actually subdivide (like usdview) will just discard authored normals and fully recompute them
			// on-demand in case they have a valid subdivision scheme (which is the default state).
			// Reference: https://graphics.pixar.com/usd/release/api/class_usd_geom_mesh.html#UsdGeom_Mesh_Normals
			if (pxr::UsdAttribute SubdivisionAttr = UsdMesh.CreateSubdivisionSchemeAttr())
			{
				ensure(SubdivisionAttr.Set(pxr::UsdGeomTokens->none));
			}
			pxr::UsdAttribute NormalsAttribute = UsdMesh.CreateNormalsAttr();
			NormalsAttribute.Set(Normals, TimeCode);
			UsdMesh.SetNormalsInterpolation(pxr::UsdGeomTokens->faceVarying);

			pxr::UsdGeomPrimvar DisplayColorPrimvar = UsdMesh.CreateDisplayColorPrimvar(
				bUseConstantColor ? pxr::UsdGeomTokens->constant : pxr::UsdGeomTokens->faceVarying
			);
			DisplayColorPrimvar.Set(DisplayColors, TimeCode);

			pxr::UsdGeomPrimvar DisplayOpacityPrimvar = UsdMesh.CreateDisplayOpacityPrimvar(
				bUseConstantColor ? pxr::UsdGeomTokens->constant : pxr::UsdGeomTokens->faceVarying
			);
			DisplayOpacityPrimvar.Set(DisplayOpacities, TimeCode);

			for (int32 UVIndex = 0; UVIndex < NumUVs; ++UVIndex)
			{
				pxr::TfToken UsdUVSetName = UsdUtils::GetUVSetName(UVIndex).Get();

				pxr::UsdGeomPrimvar PrimvarST = pxr::UsdGeomPrimvarsAPI(MeshPrim)
													.CreatePrimvar(UsdUVSetName, pxr::SdfValueTypeNames->TexCoord2fArray, pxr::UsdGeomTokens->vertex);

				PrimvarST.Set(UVs[UVIndex], TimeCode);
				PrimvarST.SetInterpolation(pxr::UsdGeomTokens->faceVarying);
			}
		}

		// Handle material assignments in another pass
		if (MaterialIndexToContentPath && ImportedMaterialSlotNameToIndex && PrimToReceiveMaterialAssignments)
		{
			const int32 NumPolygonGroups = MeshDescription.PolygonGroups().Num();
			const bool bCreateSubset = NumPolygonGroups > 1;

			// It's important that we're traversing the polygon groups in the same order the order used when emitting the faces,
			// as we may need to emit triangle indices that match that data if we're emitting UsdGeomSubsets
			for (const FPolygonGroupID& PolygonGroupID : MeshDescription.PolygonGroups().GetElementIDs())
			{
				const FName& PolygonGroupImportedSlotName = PolygonGroupImportedMaterialSlotNames[PolygonGroupID];

				int32 MaterialIndex = PolygonGroupID.GetValue();
				if (const int32* FoundIndex = ImportedMaterialSlotNameToIndex->Find(PolygonGroupImportedSlotName))
				{
					MaterialIndex = *FoundIndex;
				}

				const FString* ContentPath = nullptr;
				if (MaterialIndexToContentPath->IsValidIndex(MaterialIndex))
				{
					ContentPath = &((*MaterialIndexToContentPath)[MaterialIndex]);
				}

				// Create the triangles
				if (bCreateSubset)
				{
					// Create an UsdGeomSubset for this polygon group, as we have multiple assignments (and sections) in this mesh to write out.
					// Note that the subsets need to be a valid partition of the mesh, so we must create one even if we failed to find an
					// actual material assignment to use for it

					const int32 SectionIndex = PolygonGroupID.GetValue();

					pxr::UsdPrim GeomSubsetPrim = Stage->DefinePrim(
						MeshPrim.GetPath().AppendPath(pxr::SdfPath("Section" + std::to_string(SectionIndex))),
						UnrealToUsd::ConvertToken(TEXT("GeomSubset")).Get()
					);

					// MaterialPrim may be in another stage, so we may need another GeomSubset there
					pxr::UsdPrim MaterialGeomSubsetPrim = GeomSubsetPrim;
					if (PrimToReceiveMaterialAssignments->GetStage() != MeshPrim.GetStage())
					{
						MaterialGeomSubsetPrim = PrimToReceiveMaterialAssignments->GetStage()->OverridePrim(
							PrimToReceiveMaterialAssignments->GetPath().AppendPath(pxr::SdfPath("Section" + std::to_string(SectionIndex)))
						);
					}

					pxr::UsdGeomSubset GeomSubsetSchema{GeomSubsetPrim};

					// Element type attribute
					pxr::UsdAttribute ElementTypeAttr = GeomSubsetSchema.CreateElementTypeAttr();
					ElementTypeAttr.Set(pxr::UsdGeomTokens->face, TimeCode);

					// Indices attribute
					pxr::VtArray<int> IndicesAttrValue;
					for (const FPolygonID& PolygonID : MeshDescription.GetPolygonGroupPolygonIDs(PolygonGroupID))
					{
						IndicesAttrValue.push_back(static_cast<int>(PolygonID.GetValue()));
					}

					pxr::UsdAttribute IndicesAttr = GeomSubsetSchema.CreateIndicesAttr();
					IndicesAttr.Set(IndicesAttrValue, TimeCode);

					// Family name attribute
					pxr::UsdAttribute FamilyNameAttr = GeomSubsetSchema.CreateFamilyNameAttr();
					FamilyNameAttr.Set(pxr::UsdShadeTokens->materialBind, TimeCode);

					// Family type
					pxr::UsdGeomSubset::SetFamilyType(UsdMesh, pxr::UsdShadeTokens->materialBind, pxr::UsdGeomTokens->partition);

					// material:binding relationship
					if (ContentPath)
					{
						UsdUtils::AuthorUnrealMaterialBinding(MaterialGeomSubsetPrim, *ContentPath);
					}
				}
				else if (ContentPath)
				{
					// This will be the single assignment for this entire mesh: Just create the assignment directly on the Mesh
					UsdUtils::AuthorUnrealMaterialBinding(*PrimToReceiveMaterialAssignments, *ContentPath);
				}
			}
		}

		return true;
	}

	bool RecursivelyCollapseChildMeshes(
		const pxr::UsdPrim& Prim,
		FMeshDescription& OutMeshDescription,
		UsdUtils::FUsdPrimMaterialAssignmentInfo& OutMaterialAssignments,
		UsdToUnreal::FUsdMeshConversionOptions& Options,
		bool bSkipTransform,
		bool bSkipVisibility,
		bool bIsInsideSkelRoot
	)
	{
		if (!bSkipVisibility)
		{
			// Ignore invisible child meshes.
			//
			// We used to compute visibility here and flat out ignore any invisible meshes. However, it could be that this mesh is invisible
			// due to the first prim (the parentmost prim of the recursive calls) being invisible. If the first is invisible but animated
			// then its possible it will become visible later, so if the child meshes are all invisible due to that fact alone then we should still
			// consider them. If the first is invisible but *not* animated then we should still consider it in the same way, because that's sort
			// of what you'd expect by calling ConvertGeomMeshHierarchy: We shouldn't just return nothing if the prim happens to be invisible.
			// Besides, it could be that first is invisible due to itself having a parent that is invisible but has visibility animations:
			// In that case we'd also want to generate meshes even if first is effectively invisible, since those parents can become visible
			// later as well. The only case left is if first is invisible due having parents that are invisible and not animated: Checking for
			// this would involve checking visibility and animations of all of its parents though, which is probably a bit too much, and like
			// in the case where first itself is invisible and not animated, the caller may still expect to receive a valid mesh even if the
			// prim's parents are invisible.
			//
			// The only case in which we'll truly discard invisible submeshes now is if they're invisible *by themselves*. If we're collapsing
			// them then we know they're not animated either, so they will basically never be visible at all, at any time code.
			//
			// Note that if we were to ever manually set any of these back to visible again via the editor, the visibility changes are
			// now resyncs and we'll reparse this entire asset, which will give us the chance to add them back to the collapsed mesh.
			if (pxr::UsdGeomImageable UsdGeomImageable = pxr::UsdGeomImageable(Prim))
			{
				if (pxr::UsdAttribute VisibilityAttr = UsdGeomImageable.GetVisibilityAttr())
				{
					pxr::TfToken VisibilityToken;
					if (VisibilityAttr.Get(&VisibilityToken) && VisibilityToken == pxr::UsdGeomTokens->invisible)
					{
						return true;
					}
				}
			}
		}

		FTransform ChildTransform = Options.AdditionalTransform;
		if (!bSkipTransform)
		{
			if (pxr::UsdGeomXformable Xformable = pxr::UsdGeomXformable(Prim))
			{
				FTransform LocalChildTransform;
				UsdToUnreal::ConvertXformable(Prim.GetStage(), Xformable, LocalChildTransform, Options.TimeCode.GetValue());

				ChildTransform = LocalChildTransform * Options.AdditionalTransform;
			}
		}

		bool bSuccess = true;
		bool bTraverseChildren = true;

		// Since ConvertGeomMesh and ConvertPointInstancerToMesh take the Options object by const ref and we traverse
		// children afterwards, its fine to overwrite Options.AdditionalTransform. We do have to put it back to our
		// original value after we're done though, as calls to sibling prims that would run after this call would need
		// the original AdditionalTransform in place. The alternative is to copy the entire options object...
		TGuardValue<FTransform> Guard{Options.AdditionalTransform, ChildTransform};

		// Ignore meshes from disabled purposes, but keep traversing regardless.
		// This because a child purposes can differ from a parent purposes (i.e. we could want to skip a parent mesh, but get a child mesh)
		const bool bPurposeIsVisible = EnumHasAllFlags(Options.PurposesToLoad, IUsdPrim::GetPurpose(Prim));

		if (pxr::UsdGeomMesh Mesh = pxr::UsdGeomMesh(Prim))
		{
			// We never want to glob up *skinned* meshes inside SkelRoots, as those presumably will be handled by the
			// UsdSkelSkeletonTranslator and the skeletal data code path.
			if (bPurposeIsVisible && (!GIsEditor || !bIsInsideSkelRoot || !Prim.HasAPI<pxr::UsdSkelBindingAPI>()))
			{
				bSuccess = UsdToUnreal::ConvertGeomMesh(Mesh, OutMeshDescription, OutMaterialAssignments, Options);
			}
		}
		// Check for primitive types directly (don't just check for Gprim here because if we get something else like a UsdVolVolume or
		// UsdGeomBasisCurves ConvertGeomPrimitive will return false and we will dump our MeshDescription)
		else if (Prim.IsA<pxr::UsdGeomCapsule>() ||
				 Prim.IsA<pxr::UsdGeomCapsule_1>() ||
				 Prim.IsA<pxr::UsdGeomCone>() ||
				 Prim.IsA<pxr::UsdGeomCube>() ||
				 Prim.IsA<pxr::UsdGeomCylinder>() ||
				 Prim.IsA<pxr::UsdGeomCylinder_1>() ||
				 Prim.IsA<pxr::UsdGeomPlane>() ||
				 Prim.IsA<pxr::UsdGeomSphere>())
		{
			if (bPurposeIsVisible)
			{
				bSuccess = UsdToUnreal::ConvertGeomPrimitive(Prim, OutMeshDescription, OutMaterialAssignments, Options);
			}
		}
		else if (pxr::UsdGeomPointInstancer PointInstancer = pxr::UsdGeomPointInstancer{Prim})
		{
			if (bPurposeIsVisible)
			{
				bSuccess = UsdToUnreal::ConvertPointInstancerToMesh(PointInstancer, OutMeshDescription, OutMaterialAssignments, Options);
			}

			// We never want to step into point instancers when fetching prims for drawing
			bTraverseChildren = false;
		}

		if (bTraverseChildren)
		{
			for (const pxr::UsdPrim& ChildPrim : Prim.GetFilteredChildren(pxr::UsdTraverseInstanceProxies()))
			{
				if (!bSuccess)
				{
					break;
				}

				const bool bSkipTransformOrVisibility = false;

				bSuccess &= RecursivelyCollapseChildMeshes(
					ChildPrim,
					OutMeshDescription,
					OutMaterialAssignments,
					Options,
					bSkipTransformOrVisibility,
					bSkipTransformOrVisibility,
					bIsInsideSkelRoot || ChildPrim.IsA<pxr::UsdSkelRoot>()
				);
			}
		}

		return bSuccess;
	}

	void RecursivelyCollectPrimvars(
		const pxr::UsdPrim& Prim,
		const UsdToUnreal::FUsdMeshConversionOptions& Options,
		TSet<FString>& InOutAllPrimvars,
		TSet<FString>& InOutPreferredPrimvars,
		bool bIsFirstPrim
	)
	{
		FScopedUsdAllocs Allocs;

		// This should always replicate the same traversal pattern of RecursivelyCollapseChildMeshes

		if (!bIsFirstPrim)
		{
			if (pxr::UsdGeomImageable UsdGeomImageable = pxr::UsdGeomImageable(Prim))
			{
				if (pxr::UsdAttribute VisibilityAttr = UsdGeomImageable.GetVisibilityAttr())
				{
					pxr::TfToken VisibilityToken;
					if (VisibilityAttr.Get(&VisibilityToken) && VisibilityToken == pxr::UsdGeomTokens->invisible)
					{
						return;
					}
				}
			}
		}

		const bool bPurposeIsVisible = EnumHasAllFlags(Options.PurposesToLoad, IUsdPrim::GetPurpose(Prim));

		bool bTraverseChildren = true;

		if (pxr::UsdGeomPointInstancer PointInstancer = pxr::UsdGeomPointInstancer{Prim})
		{
			pxr::SdfPathVector PrototypePaths;
			if (!PointInstancer.GetPrototypesRel().GetTargets(&PrototypePaths))
			{
				return;
			}

			pxr::UsdStageRefPtr Stage = Prim.GetStage();
			for (const pxr::SdfPath& PrototypePath : PrototypePaths)
			{
				pxr::UsdPrim PrototypeUsdPrim = Stage->GetPrimAtPath(PrototypePath);
				if (!PrototypeUsdPrim)
				{
					continue;
				}

				const bool bChildIsFirstPrim = false;
				RecursivelyCollectPrimvars(PrototypeUsdPrim, Options, InOutAllPrimvars, InOutPreferredPrimvars, bChildIsFirstPrim);
			}

			// We never want to step into point instancers when fetching prims for drawing
			bTraverseChildren = false;
		}

		if (Prim && bPurposeIsVisible)
		{
			TArray<TUsdStore<pxr::UsdGeomPrimvar>> Primvars = UsdUtils::GetUVSetPrimvars(Prim, TNumericLimits<int32>::Max());

			for (const TUsdStore<pxr::UsdGeomPrimvar>& Primvar : Primvars)
			{
				FString PrimvarName = UsdToUnreal::ConvertToken(Primvar.Get().GetName());
				PrimvarName.RemoveFromStart(TEXT("primvars:"));

				InOutAllPrimvars.Add(PrimvarName);

				// Keep track of which primvars are texCoord2f as we always want to prefer these over other float2s
				if (Primvar.Get().GetTypeName().GetRole() == pxr::SdfValueTypeNames->TexCoord2f.GetRole())
				{
					InOutPreferredPrimvars.Add(PrimvarName);
				}
			}
		}

		if (bTraverseChildren)
		{
			for (const pxr::UsdPrim& ChildPrim : Prim.GetFilteredChildren(pxr::UsdTraverseInstanceProxies()))
			{
				const bool bChildIsFirstPrim = false;
				RecursivelyCollectPrimvars(ChildPrim, Options, InOutAllPrimvars, InOutPreferredPrimvars, bChildIsFirstPrim);
			}
		}
	}

	/**
	 * Returns the set of primvar names that can be used for each UV index for a mesh collapsed from the subtree
	 * starting at RootPrim.
	 */
	TMap<FString, int32> CollectSubtreePrimvars(
		const pxr::UsdPrim& RootPrim,
		const UsdToUnreal::FUsdMeshConversionOptions& Options,
		bool bIsFirstPrim
	)
	{
		TSet<FString> AllPrimvars;
		TSet<FString> PreferredPrimvars;

		RecursivelyCollectPrimvars(RootPrim, Options, AllPrimvars, PreferredPrimvars, bIsFirstPrim);

		return UsdUtils::CombinePrimvarsIntoUVSets(AllPrimvars, PreferredPrimvars);
	}

	struct FUsdRawBoneInfluence	   // just weight, vertex, and Bone, sorted in CopySkinWeightsToAttribute
	{
		float Weight;
		int32 VertexIndex;
		int32 BoneIndex;
	};

	// Unconverted, raw USD mesh data to convert into a MeshDescription
	struct FUsdMeshData
	{
		// So that we can reference the prim on error messages
		FString SourcePrimPath;

		pxr::TfToken Orientation = pxr::UsdGeomTokens->rightHanded;

		pxr::VtArray<int> FaceVertexCounts;
		pxr::VtArray<int> FaceVertexIndices;

		// Main attributes, which could have come from primvars
		pxr::VtArray<pxr::GfVec3f> Points;
		pxr::VtArray<pxr::GfVec3f> Normals;
		pxr::VtArray<pxr::GfVec3f> Velocities;
		pxr::VtArray<pxr::GfVec3f> DisplayColors;
		pxr::VtArray<float> DisplayOpacities;
		TArray<pxr::VtArray<pxr::GfVec2f>> UVSets;

		// Attributes used for skinned meshes
		pxr::VtArray<int> JointIndices;
		pxr::VtArray<float> JointWeights;
		uint32 NumInfluencesPerComponent;

		// Custom attributes, that may come from primvars (used essentially for baking textures in Interchange)
		struct FPrimvar
		{
			using FPrimvarType = TVariant<
				pxr::VtArray<bool>,
				pxr::VtArray<int32>,
				pxr::VtArray<float>,
				pxr::VtArray<pxr::GfVec2f>,
				pxr::VtArray<pxr::GfVec3f>,
				pxr::VtArray<pxr::GfVec4f>>;

			FPrimvarType Data;
			pxr::VtArray<int> Indices;
			pxr::TfToken Interpolation;
		};

		TMap<FString, FPrimvar> Primvars;

		pxr::TfToken PointInterpolation;
		pxr::TfToken NormalInterpolation;
		pxr::TfToken VelocityInterpolation;
		pxr::TfToken DisplayColorInterpolation;
		pxr::TfToken DisplayOpacityInterpolation;
		TArray<pxr::TfToken> UVSetInterpolations;
		pxr::TfToken JointIndexInterpolation;
		pxr::TfToken JointWeightInterpolation;

		// In case those are indexed primvars, these will contain the indices
		// Note: Velocities is not a primvar, so it can't have indices
		pxr::VtArray<int> PointIndices;
		pxr::VtArray<int> NormalIndices;
		pxr::VtArray<int> DisplayColorIndices;
		pxr::VtArray<int> DisplayOpacityIndices;
		TArray<pxr::VtArray<int>> UVSetIndices;
		pxr::VtArray<int> JointIndexIndices;
		pxr::VtArray<int> JointWeightIndices;

		// Attributes used for subdivision
		TOptional<int32> SubdivisionLevelOverride;                    // Authored directly on the prim with the SubdivisionAPI schema
		pxr::TfToken SubdivScheme = pxr::UsdGeomTokens->catmullClark; // Default to something usable so we can subdivide e.g. UsdGeom primitives even if they don't have anything authored
		pxr::TfToken InterpolateBoundary;
		pxr::TfToken FaceVaryingInterpolation;
		pxr::TfToken TriangleSubdivision;
		pxr::VtArray<int> CornerIndices;
		pxr::VtArray<float> CornerSharpnesses;
		pxr::VtArray<int> CreaseIndices;
		pxr::VtArray<int> CreaseLengths;
		pxr::VtArray<float> CreaseSharpnesses;
		pxr::TfToken CreaseMethod = pxr::PxOsdOpenSubdivTokens->uniform;
		pxr::VtArray<int> HoleIndices;

		UsdUtils::FUsdPrimMaterialAssignmentInfo LocalMaterialInfo;

		TOptional<int32> ProvidedNumUVSets;
		int32 MaterialIndexOffset = 0;
	};

	// https://en.cppreference.com/w/cpp/utility/variant/visit2
	// Helper type for the visitor
	template<class... Ts>
	struct TOverloaded : Ts...
	{
		using Ts::operator()...;
	};
	// Explicit deduction guide (not needed as of C++20
	template<class... Ts>
	TOverloaded(Ts...) -> TOverloaded<Ts...>;

	namespace USDSkeletalDataConversion
	{
		bool HasMultipleRootBones(const pxr::UsdSkelSkeletonQuery& SkeletonQuery)
		{
			const pxr::UsdSkelTopology& SkelTopology = SkeletonQuery.GetTopology();
			const pxr::VtArray<int>& JointParentIndices = SkelTopology.GetParentIndices();
			bool bFoundRoot = false;
			for (int ParentIndex : JointParentIndices)
			{
				if (ParentIndex == INDEX_NONE)
				{
					if (bFoundRoot)
					{
						return true;
					}
					bFoundRoot = true;
				}
			}

			return false;
		}

		// Based on SkeletalMeshLODImporterData.cpp 's CopySkinWeightsToAttribute.
		void CopySkinWeightsToAttribute(
			const TArray<FUsdRawBoneInfluence>& InInfluences,
			const TArray<FVertexID>& InVertexIDMap,
			FSkinWeightsVertexAttributesRef OutSkinWeightsAttribute
		)
		{
			using namespace UE::AnimationCore;

			// The weights are stored with links back to the vertices, rather than being compact.
			// Make a copy of the weights, sort them by vertex id and go by equal vertex-id strides.
			// We could do an indirection but the traversal + setup cost is probably not worth it.
			TArray<FUsdRawBoneInfluence> SortedInfluences(InInfluences);
			SortedInfluences.Sort(
				[](const FUsdRawBoneInfluence& A, const FUsdRawBoneInfluence& B)
				{
					return A.VertexIndex < B.VertexIndex;
				}
			);

			// Do the base skin weights first. We do the alternate skin weights later, since they may require geometric remapping.
			TArray<FBoneWeight> BoneWeights;
			for (int32 StartStride = 0, EndStride = 0; EndStride != SortedInfluences.Num(); StartStride = EndStride)
			{
				const int32 VertexIndex = SortedInfluences[StartStride].VertexIndex;

				// There exist meshes where the influence map got auto-filled with 100% weight on root in by using the wedge count of the raw mesh,
				// due to missing weights (e.g. static mesh imported as a skeletal mesh), and so may refer to vertices that don't exist.
				// We just stop when we get to the broken set and ignore the rest.
				if (VertexIndex >= InVertexIDMap.Num())
				{
					break;
				}

				EndStride = StartStride + 1;
				while (EndStride < SortedInfluences.Num() && VertexIndex == SortedInfluences[EndStride].VertexIndex)
				{
					EndStride++;
				}

				BoneWeights.Reset(0);
				for (int32 Idx = StartStride; Idx < EndStride; Idx++)
				{
					const FUsdRawBoneInfluence& RawInfluence = SortedInfluences[Idx];
					int32 BoneIndex = RawInfluence.BoneIndex;

					FBoneWeight BoneWeight(static_cast<FBoneIndexType>(BoneIndex), RawInfluence.Weight);
					BoneWeights.Add(BoneWeight);
				}

				if (BoneWeights.IsEmpty())
				{
					const FBoneWeight RootBoneWeight(0, 1.0f);
					BoneWeights.Add(RootBoneWeight);
				}

				OutSkinWeightsAttribute.Set(InVertexIDMap[VertexIndex], BoneWeights);
			}
		}
	}	 // namespace USDSkeletalDataConversion

	void CollectPrimvarData(const pxr::UsdPrim& Prim, const UsdToUnreal::FUsdMeshConversionOptions& Options, FUsdMeshData& InOutMeshData)
	{
		using namespace UsdToUnreal;
		const pxr::TfToken DefaultInterpolation = pxr::UsdGeomTokens->varying;

		pxr::UsdGeomPrimvarsAPI PrimvarAPI{Prim};
		std::vector<pxr::UsdGeomPrimvar> Primvars = PrimvarAPI.GetPrimvarsWithAuthoredValues();
		for (const pxr::UsdGeomPrimvar& Primvar : Primvars)
		{
			std::string PrimvarName = Primvar.GetPrimvarName().GetString();
			if (PrimvarName == "displayColor" || PrimvarName == "displayOpacity" || PrimvarName == "st"
				|| (Options.ImportPrimvars == FUsdMeshConversionOptions::EImportPrimvar::Bake
					&& !Options.PrimvarNames.Contains(UTF8_TO_TCHAR(PrimvarName.c_str()))))
			{
				continue;
			}

			std::string PrimvarType = Primvar.GetTypeName().GetAsToken().GetString();
			FUsdMeshData::FPrimvar CustomPrimvar;

			if (pxr::VtArray<float> PrimvarFloat; PrimvarType == pxr::SdfValueTypeNames->FloatArray && Primvar.Get(&(PrimvarFloat), Options.TimeCode))
			{
				CustomPrimvar.Data.Set<pxr::VtArray<float>>(PrimvarFloat);
			}
			else if (pxr::VtArray<int32> PrimvarInt; PrimvarType == pxr::SdfValueTypeNames->IntArray && Primvar.Get(&(PrimvarInt), Options.TimeCode))
			{
				CustomPrimvar.Data.Set<pxr::VtArray<int32>>(PrimvarInt);
			}
			else if (pxr::VtArray<bool> PrimvarBool;
					 PrimvarType == pxr::SdfValueTypeNames->BoolArray && Primvar.Get(&(PrimvarBool), Options.TimeCode))
			{
				CustomPrimvar.Data.Set<pxr::VtArray<bool>>(PrimvarBool);
			}
			else if (pxr::VtArray<pxr::GfVec2f> PrimvarFloat2;
					 PrimvarType == pxr::SdfValueTypeNames->Float2Array && Primvar.Get(&(PrimvarFloat2), Options.TimeCode))
			{
				CustomPrimvar.Data.Set<pxr::VtArray<pxr::GfVec2f>>(PrimvarFloat2);
			}
			else if (pxr::VtArray<pxr::GfVec3f> PrimvarFloat3;
					 PrimvarType == pxr::SdfValueTypeNames->Float3Array && Primvar.Get(&(PrimvarFloat3), Options.TimeCode))
			{
				CustomPrimvar.Data.Set<pxr::VtArray<pxr::GfVec3f>>(PrimvarFloat3);
			}
			else if (pxr::VtArray<pxr::GfVec4f> PrimvarFloat4;
					 PrimvarType == pxr::SdfValueTypeNames->Float4Array && Primvar.Get(&(PrimvarFloat4), Options.TimeCode))
			{
				CustomPrimvar.Data.Set<pxr::VtArray<pxr::GfVec4f>>(PrimvarFloat4);
			}
			else
			{
				USD_LOG_WARNING(TEXT("Storage of primvar type '%s' not yet supported."), *UsdToUnreal::ConvertString(PrimvarType));
			}

			bool bDataCollected = false;
			Visit(
				[&bDataCollected](auto&& Value)
				{
					bDataCollected = !Value.empty();
				},
				CustomPrimvar.Data
			);

			// only add if we collected the data
			if (bDataCollected)
			{
				CustomPrimvar.Interpolation = GetAttrInterpolation(Primvar, DefaultInterpolation);
				Primvar.GetIndices(&CustomPrimvar.Indices, Options.TimeCode);
				InOutMeshData.Primvars.FindOrAdd(PrimvarName.c_str(), MoveTemp(CustomPrimvar));
			}
		}
	}

	template<typename T>
	void GetOrEarliest(const T& AttributeOrPrimvar, auto& OutputArray, pxr::UsdTimeCode TimeCode)
	{
		const bool bResult = AttributeOrPrimvar.Get(&OutputArray, TimeCode);

		// Some DCCs seem to output all their mesh data in a single time sample sometimes, as opposed to on the default time code.
		// When importing, by default we'll exclusively sample the Default timecode, which means we won't find anything in that
		// case, and so Get() and the mesh parsing as a whole will likely fail.
		// Here we use a fallback to also check the earliest time code in that case.
		if (!bResult && TimeCode.IsDefault() && AttributeOrPrimvar.HasAuthoredValue())
		{
			if (AttributeOrPrimvar.Get(&OutputArray, pxr::UsdTimeCode::EarliestTime()))
			{
				pxr::UsdPrim Prim;
				if constexpr (std::is_same_v<T, pxr::UsdAttribute>)
				{
					Prim = AttributeOrPrimvar.GetPrim();
				}
				else
				{
					Prim = AttributeOrPrimvar.GetAttr().GetPrim();
				}

				USD_LOG_WARNING(
					TEXT(
						"Mesh data attribute or primvar '%s' of prim '%s' exclusively has time samples but was sampled at the default time code, which failed. The importer will fallback to sampling the earliest time code instead."
					),
					*UsdToUnreal::ConvertToken(AttributeOrPrimvar.GetName()),
					*UsdToUnreal::ConvertPath(Prim.GetPrimPath())
				);
			}
		}
	}

	bool CollectMeshData(
		const pxr::UsdPrim& Prim,
		const UsdToUnreal::FUsdMeshConversionOptions& Options,
		FUsdMeshData& InOutMeshData,
		UsdUtils::FUsdPrimMaterialAssignmentInfo& InOutCombinedMaterialAssignments
	)
	{
		FScopedUsdAllocs Allocs;

		pxr::UsdGeomGprim Gprim{Prim};
		if (!Gprim)
		{
			return false;
		}

		auto GetIndicesOrEarliest = [&Prim](const auto& AttributeOrPrimvar, auto& OutputArray, pxr::UsdTimeCode TimeCode)
		{
			const bool bResult = AttributeOrPrimvar.GetIndices(&OutputArray, TimeCode);
			if (!bResult && TimeCode.IsDefault() && AttributeOrPrimvar.HasAuthoredValue())
			{
				AttributeOrPrimvar.GetIndices(&OutputArray, pxr::UsdTimeCode::EarliestTime());
			}
		};

		// All pointsBased/Gprim attributes we'll retrieve happen to have default varying interpolation
		const pxr::TfToken DefaultInterpolation = pxr::UsdGeomTokens->varying;

		InOutMeshData.SourcePrimPath = UsdToUnreal::ConvertPath(Prim.GetPrimPath());

		InOutMeshData.Orientation = GetGprimOrientation(Gprim, Options.TimeCode);

		// DisplayColors
		if (pxr::UsdGeomPrimvar DisplayColorsPrimvar = Gprim.GetDisplayColorPrimvar())
		{
			GetOrEarliest(DisplayColorsPrimvar, InOutMeshData.DisplayColors, Options.TimeCode);
			GetIndicesOrEarliest(DisplayColorsPrimvar, InOutMeshData.DisplayColorIndices, Options.TimeCode);
			InOutMeshData.DisplayColorInterpolation = GetAttrInterpolation(DisplayColorsPrimvar, DefaultInterpolation);
		}

		// DisplayOpacities
		if (pxr::UsdGeomPrimvar DisplayOpacitiesPrimvar = Gprim.GetDisplayOpacityPrimvar())
		{
			GetOrEarliest(DisplayOpacitiesPrimvar, InOutMeshData.DisplayOpacities, Options.TimeCode);
			GetIndicesOrEarliest(DisplayOpacitiesPrimvar, InOutMeshData.DisplayOpacityIndices, Options.TimeCode);
			InOutMeshData.DisplayOpacityInterpolation = GetAttrInterpolation(DisplayOpacitiesPrimvar, DefaultInterpolation);
		}

		// Per-prim subdivision level (outside the UsdGeomMesh section so it also works for primitives)
		if (Prim.HasAPI(UnrealIdentifiers::SubdivisionAPI))
		{
			int32 SubdivisionLevelOverride = 0;
			pxr::UsdAttribute Attribute = Prim.GetAttribute(UnrealIdentifiers::SubdivisionLevel);
			if (Attribute && Attribute.HasAuthoredValue() && Attribute.Get<int32>(&SubdivisionLevelOverride))
			{
				InOutMeshData.SubdivisionLevelOverride = SubdivisionLevelOverride;
			}
		}

		if (pxr::UsdGeomMesh UsdMesh{Prim})
		{
			// Faces
			if (pxr::UsdAttribute FaceVertexCountsAttr = UsdMesh.GetFaceVertexCountsAttr())  
			{
				GetOrEarliest(FaceVertexCountsAttr, InOutMeshData.FaceVertexCounts, Options.TimeCode);
			}

			// Vertex indices
			if (pxr::UsdAttribute FaceVertexIndicesAttr = UsdMesh.GetFaceVertexIndicesAttr())
			{
				GetOrEarliest(FaceVertexIndicesAttr, InOutMeshData.FaceVertexIndices, Options.TimeCode);
			}

			// Points
			if (pxr::UsdGeomPrimvar PointsPrimvar = pxr::UsdGeomPrimvar(Prim.GetAttribute(UnrealIdentifiers::PrimvarsPoints)))
			{
				// Should points always have "vertex" interpolation? Having "varying" forces it to just tessellate instead,
				// and all OpenSubdiv tutorials use the vertex interpolation type for it
				GetOrEarliest(PointsPrimvar, InOutMeshData.Points, Options.TimeCode);
				GetIndicesOrEarliest(PointsPrimvar, InOutMeshData.PointIndices, Options.TimeCode);
				InOutMeshData.PointInterpolation = GetAttrInterpolation(PointsPrimvar, pxr::UsdGeomTokens->vertex);
			}
			else if (pxr::UsdAttribute PointsAttr = UsdMesh.GetPointsAttr())
			{
				GetOrEarliest(PointsAttr, InOutMeshData.Points, Options.TimeCode);
				InOutMeshData.PointInterpolation = GetAttrInterpolation(PointsAttr, pxr::UsdGeomTokens->vertex);
			}

			// Normals
			if (pxr::UsdGeomPrimvar NormalsPrimvar = pxr::UsdGeomPrimvar(Prim.GetAttribute(UnrealIdentifiers::PrimvarsNormals)))
			{
				GetOrEarliest(NormalsPrimvar, InOutMeshData.Normals, Options.TimeCode);
				GetIndicesOrEarliest(NormalsPrimvar, InOutMeshData.NormalIndices, Options.TimeCode);
				InOutMeshData.NormalInterpolation = GetAttrInterpolation(NormalsPrimvar, DefaultInterpolation);
			}
			else if (pxr::UsdAttribute NormalsAttr = UsdMesh.GetNormalsAttr())
			{
				GetOrEarliest(NormalsAttr, InOutMeshData.Normals, Options.TimeCode);
				InOutMeshData.NormalInterpolation = GetAttrInterpolation(NormalsAttr, DefaultInterpolation);
			}

			// Velocities
			if (pxr::UsdAttribute VelocitiesAttr = UsdMesh.GetVelocitiesAttr())
			{
				GetOrEarliest(VelocitiesAttr, InOutMeshData.Velocities, Options.TimeCode);
				InOutMeshData.VelocityInterpolation = GetAttrInterpolation(VelocitiesAttr, DefaultInterpolation);
			}

			// Custom primvars
			if (Options.ImportPrimvars != UsdToUnreal::FUsdMeshConversionOptions::EImportPrimvar::Standard)
			{
				CollectPrimvarData(Prim, Options, InOutMeshData);
			}

			// Collect the subdivision attributes only if we plan on subdividing
			if (InOutMeshData.SubdivisionLevelOverride.Get(Options.SubdivisionLevel) > 0)
			{
				if (pxr::UsdAttribute SubdivSchemeAttr = UsdMesh.GetSubdivisionSchemeAttr())
				{
					GetOrEarliest(SubdivSchemeAttr, InOutMeshData.SubdivScheme, Options.TimeCode);
				}

				if (InOutMeshData.SubdivScheme != pxr::UsdGeomTokens->none)
				{
					if (pxr::UsdAttribute InterpolateBoundaryAttr = UsdMesh.GetInterpolateBoundaryAttr())
					{
						GetOrEarliest(InterpolateBoundaryAttr, InOutMeshData.InterpolateBoundary, Options.TimeCode);
					}

					if (pxr::UsdAttribute FaceVaryingInterpolationAttr = UsdMesh.GetFaceVaryingLinearInterpolationAttr())
					{
						GetOrEarliest(FaceVaryingInterpolationAttr, InOutMeshData.FaceVaryingInterpolation, Options.TimeCode);
					}

					if (pxr::UsdAttribute TriangleSubdivisionAttr = UsdMesh.GetTriangleSubdivisionRuleAttr())
					{
						GetOrEarliest(TriangleSubdivisionAttr, InOutMeshData.TriangleSubdivision, Options.TimeCode);
					}

					if (pxr::UsdAttribute CornerIndicesAttr = UsdMesh.GetCornerIndicesAttr())
					{
						GetOrEarliest(CornerIndicesAttr, InOutMeshData.CornerIndices, Options.TimeCode);
					}

					if (pxr::UsdAttribute CornerSharpnessesAttr = UsdMesh.GetCornerSharpnessesAttr())
					{
						GetOrEarliest(CornerSharpnessesAttr, InOutMeshData.CornerSharpnesses, Options.TimeCode);
					}

					if (pxr::UsdAttribute CreaseIndicesAttr = UsdMesh.GetCreaseIndicesAttr())
					{
						GetOrEarliest(CreaseIndicesAttr, InOutMeshData.CreaseIndices, Options.TimeCode);
					}

					if (pxr::UsdAttribute CreaseLengthsAttr = UsdMesh.GetCreaseLengthsAttr())
					{
						GetOrEarliest(CreaseLengthsAttr, InOutMeshData.CreaseLengths, Options.TimeCode);
					}

					if (pxr::UsdAttribute CreaseSharpnessesAttr = UsdMesh.GetCreaseSharpnessesAttr())
					{
						GetOrEarliest(CreaseSharpnessesAttr, InOutMeshData.CreaseSharpnesses, Options.TimeCode);
					}

					if (pxr::UsdAttribute HoleIndicesAttr = UsdMesh.GetHoleIndicesAttr())
					{
						GetOrEarliest(HoleIndicesAttr, InOutMeshData.HoleIndices, Options.TimeCode);
					}

					// For some reason this is not part of the USD schema so just pick the first valid token.
					// See UsdImagingMeshAdapter::GetSubdivTags
					InOutMeshData.CreaseMethod = pxr::PxOsdOpenSubdivTokens->uniform;
				}
			}
		}

		// UVs
		{
			TArray<TUsdStore<pxr::UsdGeomPrimvar>> PrimvarsToUse;

			// If we already have a primvar to UV index assignment, let's just use that.
			// When collapsing, we'll do a pre-pass on all meshes to translate and determine this beforehand.
			if (InOutCombinedMaterialAssignments.PrimvarToUVIndex.Num() > 0)
			{
				int32 HighestProvidedUVIndex = 0;
				for (const TPair<FString, int32>& Pair : InOutCombinedMaterialAssignments.PrimvarToUVIndex)
				{
					HighestProvidedUVIndex = FMath::Max(HighestProvidedUVIndex, Pair.Value);
				}
				InOutMeshData.ProvidedNumUVSets = HighestProvidedUVIndex + 1;

				TArray<TUsdStore<pxr::UsdGeomPrimvar>> AllMeshUVPrimvars = UsdUtils::GetUVSetPrimvars(Prim, TNumericLimits<int32>::Max());
				PrimvarsToUse = UsdUtils::AssemblePrimvarsIntoUVSets(AllMeshUVPrimvars, InOutCombinedMaterialAssignments.PrimvarToUVIndex);
			}
			// Let's use the best primvar assignment for this particular mesh instead
			else
			{
				PrimvarsToUse = UsdUtils::GetUVSetPrimvars(Prim);
				InOutCombinedMaterialAssignments.PrimvarToUVIndex = UsdUtils::AssemblePrimvarsIntoPrimvarToUVIndexMap(PrimvarsToUse);
			}

			// Unpack the primvars we'll be using into simple arrays so that if we want to subdivide this mesh
			// we can just update those arrays with new data
			InOutMeshData.UVSets.Reset(PrimvarsToUse.Num());
			InOutMeshData.UVSetIndices.Reset(PrimvarsToUse.Num());
			InOutMeshData.UVSetInterpolations.Reset(PrimvarsToUse.Num());
			for (const TUsdStore<pxr::UsdGeomPrimvar>& PrimvarPtr : PrimvarsToUse)
			{
				pxr::VtArray<pxr::GfVec2f>& UVs = InOutMeshData.UVSets.Emplace_GetRef();
				pxr::VtArray<int>& Indices = InOutMeshData.UVSetIndices.Emplace_GetRef();
				pxr::TfToken& Interpolation = InOutMeshData.UVSetInterpolations.Emplace_GetRef();

				// There are some code paths where it's OK to end up with an invalid primvar here:
				// For example when collapsing two cubes and only one of them has the e.g. "st1" primvar:
				// We will allocate a UV index for it and try reading it on both cubes, and end up with
				// an invalid primvar in one of them, although it is important to retain the UV set
				// ordering between the cubes
				if (pxr::UsdGeomPrimvar Primvar = PrimvarPtr.Get())
				{
					GetOrEarliest(Primvar, UVs, Options.TimeCode);
					GetIndicesOrEarliest(Primvar, Indices, Options.TimeCode);
					Interpolation = GetAttrInterpolation(Primvar, DefaultInterpolation);
				}
			}
		}

		// Skeletal attributes
		{
			if (pxr::UsdGeomPrimvar JointIndicesPrimvar = pxr::UsdGeomPrimvar(Prim.GetAttribute(pxr::UsdSkelTokens->primvarsSkelJointIndices)))
			{
				GetOrEarliest(JointIndicesPrimvar, InOutMeshData.JointIndices, Options.TimeCode);
				GetIndicesOrEarliest(JointIndicesPrimvar, InOutMeshData.JointIndices, Options.TimeCode);
				InOutMeshData.JointIndexInterpolation = GetAttrInterpolation(JointIndicesPrimvar, pxr::UsdGeomTokens->vertex);

				InOutMeshData.NumInfluencesPerComponent = JointIndicesPrimvar.GetElementSize();
			}

			if (pxr::UsdGeomPrimvar JointWeightsPrimvar = pxr::UsdGeomPrimvar(Prim.GetAttribute(pxr::UsdSkelTokens->primvarsSkelJointWeights)))
			{
				GetOrEarliest(JointWeightsPrimvar, InOutMeshData.JointWeights, Options.TimeCode);
				GetIndicesOrEarliest(JointWeightsPrimvar, InOutMeshData.JointWeightIndices, Options.TimeCode);
				InOutMeshData.JointWeightInterpolation = GetAttrInterpolation(JointWeightsPrimvar, pxr::UsdGeomTokens->vertex);
			}
		}

		// Material assignments
		{
			const bool bProvideMaterialIndices = true;
			InOutMeshData.LocalMaterialInfo = UsdUtils::GetPrimMaterialAssignments(
				Prim,
				Options.TimeCode,
				bProvideMaterialIndices,
				Options.RenderContext,
				Options.MaterialPurpose,
				Options.bForceCheckUnrealMaterialAttribute,
				Options.bRequireMaterialsHaveProvidedRenderContext
			);

			InOutMeshData.MaterialIndexOffset = InOutCombinedMaterialAssignments.Slots.Num();
		}

		return true;
	}

	void ProcessSkinnedMeshData(
		const pxr::UsdPrim& Prim,
		const UsdToUnreal::FUsdMeshConversionOptions& Options,
		FUsdMeshData& InOutMeshData,
		const pxr::UsdSkelSkinningQuery* SkinningQuery,
		const pxr::UsdSkelSkeletonQuery* SkeletonQuery
	)
	{
		const bool bProcessSkeletalAttributes = SkeletonQuery && SkeletonQuery->IsValid() && SkinningQuery && SkinningQuery->IsValid();
		if (!bProcessSkeletalAttributes)
		{
			return;
		}

		// Collect GeomBindTransform if we have one
		bool bApplyGeomBindTransform = false;
		pxr::GfMatrix4d GeomBindTransform{1.0};
		pxr::GfMatrix4d InvTransposeGeomBindTransform{1.0};
		if (bProcessSkeletalAttributes && SkinningQuery)
		{
			GeomBindTransform = SkinningQuery->GetGeomBindTransform(Options.TimeCode);
			if (GeomBindTransform != pxr::GfMatrix4d(1.0))	  // This is USD's identity
			{
				if (GeomBindTransform.GetDeterminant() == 0.0)
				{
					// Can't invert, just use as-is
					USD_LOG_WARNING(TEXT("Failed to invert geomBindTransform for prim '%s': Normals may be incorrect"), *UsdToUnreal::ConvertPath(Prim.GetPrimPath()));
					InvTransposeGeomBindTransform = GeomBindTransform;
				}
				else
				{
					InvTransposeGeomBindTransform = GeomBindTransform.GetInverse().GetTranspose();
				}
				bApplyGeomBindTransform = true;
			}
		}

		// Apply the geomBindTransform if we have a relevant one
		if (bApplyGeomBindTransform)
		{
			for (pxr::GfVec3f& Point : InOutMeshData.Points)
			{
				Point = static_cast<pxr::GfVec3f>(GeomBindTransform.Transform(Point));
			}
			for (pxr::GfVec3f& Normal : InOutMeshData.Normals)
			{
				Normal = static_cast<pxr::GfVec3f>(InvTransposeGeomBindTransform.TransformDir(Normal));
			}
		}

		const size_t NumPoints = InOutMeshData.Points.size();

		// If our influences are marked as "constant", they won't necessarily have a value for each point, so here
		// we "flatten" them by copy pasting that value until they match the count of points.
		// See UsdSkelSkinningQuery::ComputeVaryingJointInfluences()
		const bool bIsRigidlyDeformed = InOutMeshData.JointIndexInterpolation == pxr::UsdGeomTokens->constant;
		if (bIsRigidlyDeformed)
		{
			pxr::UsdSkelExpandConstantInfluencesToVarying(&InOutMeshData.JointIndices, NumPoints);
			pxr::UsdSkelExpandConstantInfluencesToVarying(&InOutMeshData.JointWeights, NumPoints);
		}

		// Compensate for any additional "true root bone" we may add in case the bound skeleton has multiple root bones
		// We'll only ever set NumAdditionalBones to 1 or 0 (as we'll only either need a "true root bone" or
		// not), but naming it this way allows us to use it like an offset, which should make it easier to
		// understand whenever it is used
		// TODO: Can this be done simpler by just performing it after the joint remapping?
		uint32 NumAdditionalBones = (Options.bOffsetJointIndicesForMultipleRootBones && USDSkeletalDataConversion::HasMultipleRootBones(*SkeletonQuery)) ? 1 : 0;
		if (NumAdditionalBones > 0)
		{
			for (int32 Index = 0; Index < InOutMeshData.JointIndices.size(); ++Index)
			{
				InOutMeshData.JointIndices[Index] += NumAdditionalBones;
			}
		}

		// Remap the joint indices according to the Mesh's desired joint order
		const pxr::UsdPrim& SkinningPrim = SkinningQuery->GetPrim();
		pxr::UsdSkelBindingAPI SkelBindingAPI(SkinningPrim);
		if (SkelBindingAPI)
		{
			// If we have a joint mapper this Mesh has an explicit joint ordering, so we need to map joint indices to the skeleton's bone indices
			if (pxr::UsdSkelAnimMapperRefPtr AnimMapper = SkinningQuery->GetJointMapper())
			{
				pxr::VtArray<int> SkeletonBoneIndices;
				if (pxr::UsdSkelSkeleton BoundSkeleton = SkelBindingAPI.GetInheritedSkeleton())
				{
					if (pxr::UsdAttribute SkeletonJointsAttr = BoundSkeleton.GetJointsAttr())
					{
						pxr::VtArray<pxr::TfToken> SkeletonJoints;
						GetOrEarliest(SkeletonJointsAttr, SkeletonJoints, Options.TimeCode);

						if (SkeletonJoints.size() > 0)
						{
							// If the skeleton has N bones, this will just contain { 0, 1, 2, ..., N-1 }
							int NumUsdSkeletonBones = static_cast<int>(SkeletonJoints.size());
							for (int SkeletonBoneIndex = 0; SkeletonBoneIndex < NumUsdSkeletonBones; ++SkeletonBoneIndex)
							{
								SkeletonBoneIndices.push_back(SkeletonBoneIndex);
							}

							// Use the AnimMapper to produce the indices of the Mesh's joints within the Skeleton's list of joints.
							// Example: Imagine skeleton had { "Root", "Root/Hip", "Root/Hip/Shoulder", "Root/Hip/Shoulder/Arm",
							// "Root/Hip/Shoulder/Arm/Elbow"
							// }, and so BoneIndexRemapping was { 0, 1, 2, 3, 4 }. Consider a Mesh that specifies the explicit joints {
							// "Root/Hip/Shoulder", "Root/Hip/Shoulder/Arm" }, and so uses the indices 0 and 1 to refer to Shoulder and Arm. After
							// the Remap call SkeletonBoneIndices will hold { 2, 3 }, as those are the indices of Shoulder and Arm within the
							// skeleton's bones
							pxr::VtArray<int> BoneIndexRemapping;
							if (AnimMapper->Remap(SkeletonBoneIndices, &BoneIndexRemapping))
							{
								for (int32 Index = 0; Index < InOutMeshData.JointIndices.size(); ++Index)
								{
									// We have to remove our "NumAdditionalBones" offset from the influence's bone index because that's a UE
									// concept that the BoneIndexRemapping array doesn't really know about. After that, we have a bone index that
									// matches the USD Skeleton joint order, then we can remap with BoneIndexRemapping and add our
									// NumAdditionalBones back in so that it matches our USkeleton
									int32 CurrentIndex = InOutMeshData.JointIndices[Index];
									InOutMeshData.JointIndices[Index] = NumAdditionalBones + BoneIndexRemapping[CurrentIndex - NumAdditionalBones];
								}
							}
						}
					}
				}
			}
		}
	}

	// OpenSubdiv expects the data elements of its buffers to implement a simple interface,
	// so here we wrap the datatypes we'll be interpolating with that interface
	struct FSubdivVec2f
	{
		pxr::GfVec2f Data;

		void Clear()
		{
			Data = {0, 0};
		}

		void AddWithWeight(const pxr::GfVec2f& Src, float Weight)
		{
			Data += Src * Weight;
		}

		void AddWithWeight(const FSubdivVec2f& Src, float Weight)
		{
			Data += Src.Data * Weight;
		}
	};

	struct FSubdivVec3f
	{
		pxr::GfVec3f Data;

		void Clear()
		{
			Data = {0, 0, 0};
		}

		void AddWithWeight(const pxr::GfVec3f& Src, float Weight)
		{
			Data += Src * Weight;
		}

		void AddWithWeight(const FSubdivVec3f& Src, float Weight)
		{
			Data += Src.Data * Weight;
		}
	};

	struct FSubdivInt
	{
		int Data;

		void Clear()
		{
			Data = 0;
		}

		void AddWithWeight(const int& Src, float Weight)
		{
			Data += Src * Weight;
		}

		void AddWithWeight(const FSubdivInt& Src, float Weight)
		{
			Data += Src.Data * Weight;
		}
	};

	struct FSubdivFloat
	{
		float Data;

		void Clear()
		{
			Data = 0;
		}

		void AddWithWeight(const float& Src, float Weight)
		{
			Data += Src * Weight;
		}

		void AddWithWeight(const FSubdivFloat& Src, float Weight)
		{
			Data += Src.Data * Weight;
		}
	};

	// We're going to be doing some reinterpret casting between these, so let's try our best to make sure we're safe
	static_assert(sizeof(FSubdivVec3f) == sizeof(pxr::GfVec3f) && alignof(FSubdivVec3f) == alignof(pxr::GfVec3f));
	static_assert(sizeof(FSubdivInt) == sizeof(int) && alignof(FSubdivInt) == alignof(int));
	static_assert(sizeof(FSubdivFloat) == sizeof(float) && alignof(FSubdivFloat) == alignof(float));

	template<typename T>
	pxr::VtArray<T> ComputeFlattened(const pxr::VtArray<T>& Values, const pxr::VtArray<int>& Indices)
	{
		// Adapted from USD's _ComputeFlattened within pxr\imaging\hd\primvarSchema.cpp

		FScopedUsdAllocs Allocs;

		if (Indices.empty())
		{
			return Values;
		}

		pxr::VtArray<T> Result = pxr::VtArray<T>(Indices.size());

		bool bInvalidIndices = false;
		for (size_t Index = 0; Index < Indices.size(); ++Index)
		{
			int ValueIndex = Indices[Index];
			if (ValueIndex >= 0 && (size_t)ValueIndex < Values.size())
			{
				Result[Index] = Values[ValueIndex];
			}
			else
			{
				Result[Index] = T();
				bInvalidIndices = true;
			}
		}

		if (bInvalidIndices)
		{
			USD_LOG_WARNING(TEXT("Invalid primvar indices encountered in ComputeFlattened"));
		}

		return Result;
	};

	// In-place converts SharedValuesArray from an array of values that are shared according to the topology described in
	// 'Level' into a flattened array that has a single value for each face vertex.
	template<typename T>
	void FlattenFaceVaryingValues(T& SharedValuesArray, int32 FaceVaryingChannel, const OpenSubdiv::Far::TopologyLevel& Level)
	{
		if (SharedValuesArray.size() == 0)
		{
			return;
		}

		T FlattenedValues;
		FlattenedValues.reserve(Level.GetNumFaceVertices());
		for (int32 FaceIndex = 0; FaceIndex < Level.GetNumFaces(); ++FaceIndex)
		{
			OpenSubdiv::Far::ConstIndexArray Face = Level.GetFaceVertices(FaceIndex);
			OpenSubdiv::Far::ConstIndexArray FaceNormalsFaceVaryingIndices = Level.GetFaceFVarValues(FaceIndex, FaceVaryingChannel);

			for (int32 FaceVertexIndex = 0; FaceVertexIndex < Face.size(); ++FaceVertexIndex)
			{
				int32 FaceVertexNormalsIndex = FaceNormalsFaceVaryingIndices[FaceVertexIndex];
				FlattenedValues.push_back(SharedValuesArray[FaceVertexNormalsIndex]);
			}
		}
		Swap(FlattenedValues, SharedValuesArray);
	};

	bool SubdivideMeshData(const UsdToUnreal::FUsdMeshConversionOptions& Options, FUsdMeshData& InOutMeshData)
	{
		// References:
		// - USD's HdSt_OsdTopologyComputation::Resolve
		// - UE's FSubdividePoly::ComputeTopologySubdivision
		// - OpenSubdiv's https://github.com/PixarAnimationStudios/OpenSubdiv/blob/release/tutorials/far/tutorial_2_2/far_tutorial_2_2.cpp

		const int32 SubdivisionLevel = InOutMeshData.SubdivisionLevelOverride.Get(Options.SubdivisionLevel);
		if (SubdivisionLevel == 0 || InOutMeshData.SubdivScheme.IsEmpty() || InOutMeshData.SubdivScheme == pxr::UsdGeomTokens->none)
		{
			return false;
		}

		FScopedUsdAllocs Allocs;

		int32 TargetSubdivLevel = FMath::Max(0, FMath::Min(GMaxSubdivLevel, SubdivisionLevel));
		if (TargetSubdivLevel < SubdivisionLevel)
		{
			USD_LOG_USERWARNING(FText::Format(
				LOCTEXT("ClampedSubdiv", "Max subdivision level was clamped to {0} (controlled by the cvar '{1}')"),
				GMaxSubdivLevel,
				FText::FromString(MaxUsdSubdivLevelCvarName)
			));
		}
		if (TargetSubdivLevel < 1)
		{
			USD_LOG_USERWARNING(FText::Format(
				LOCTEXT(
					"CancellingSubdiv",
					"Cancelling out of subdividing mesh '{0}' due to target subdivision level being {1} after clamping (it needs to be at least 1 for a round of subdivision)"
				),
				FText::FromString(InOutMeshData.SourcePrimPath),
				TargetSubdivLevel
			));
			return false;
		}

		USD_LOG_INFO(TEXT("Subdividing mesh '%s' to subdivision level %d"), *InOutMeshData.SourcePrimPath, TargetSubdivLevel);

		// We need to track our faceVarying attributes when subdividing, so we'll give each one an unique
		// index into that FaceVaryingTopologies array down below. We'll use this to track how many entries that array
		// will need, and which attribute has which index
		int32 FaceVaryingChannelCounter = 0;

		// It is very likely that if a primvar is faceVarying it will be indexed, and we should use those indices as the
		// FaceVarying topology (https://openusd.org/release/api/class_usd_geom_primvar.html#UsdGeomPrimvar_Indexed_primvars).
		// In case the user provided a faceVarying attribute/primvar *without* indexing however, it means each
		// face vertex gets its own dedicated value and the topology is for them to be all "disconnected" and never share
		// vertices, which we can represent with an index array with increasing values. We can reuse that array for all un-indexed
		// attributes and primvars though, which is what we'll track here
		int32 IotaFaceVaryingChannel = INDEX_NONE;
		pxr::VtArray<int> IotaIndices;
		const int NumFaceVertices = InOutMeshData.FaceVertexIndices.size();
		TFunction<void()> CreateIotaIndicesIfNeeded = [&IotaIndices, &IotaFaceVaryingChannel, &FaceVaryingChannelCounter, NumFaceVertices]()
		{
			if (IotaFaceVaryingChannel != INDEX_NONE)
			{
				return;
			}
			IotaFaceVaryingChannel = FaceVaryingChannelCounter++;

			IotaIndices.resize(NumFaceVertices);
			for (uint32 Index = 0; Index < IotaIndices.size(); ++Index)
			{
				IotaIndices[Index] = Index;
			}
		};

		// All pointsBased/Gprim attributes we'll retrieve happen to have varying default interpolation
		const pxr::TfToken DefaultInterpolation = pxr::UsdGeomTokens->varying;

		// Points
		int32 PointsFaceVaryingChannel = INDEX_NONE;
		if (InOutMeshData.PointIndices.size() > 0)
		{
			// faceVarying indices are important for the topology and OpenSubdiv needs them, so we need to keep our
			// array indexed and flatten only after refining the mesh
			if (InOutMeshData.PointInterpolation == pxr::UsdGeomTokens->faceVarying)
			{
				PointsFaceVaryingChannel = FaceVaryingChannelCounter++;
			}
			// Indexing on vertex, varying and uniform interpolation are just to allow reusing of the values.
			// As far as I know there is no way to get these indices handled by OpenSubdiv (at least not through the pxOsd
			// wrapper), and we'll end up flattening all indexing later anyway, so we might as well flatten now
			else
			{
				InOutMeshData.Points = ComputeFlattened(InOutMeshData.Points, InOutMeshData.PointIndices);
				InOutMeshData.PointIndices = {};
			}
		}
		if (InOutMeshData.PointInterpolation == pxr::UsdGeomTokens->faceVarying && PointsFaceVaryingChannel == INDEX_NONE)
		{
			// If we're faceVarying we will need *some* indices, so create the iota indices here and use that
			CreateIotaIndicesIfNeeded();
			PointsFaceVaryingChannel = IotaFaceVaryingChannel;
		}

		// Normals
		int32 NormalsFaceVaryingChannel = INDEX_NONE;
		if (GIgnoreNormalsWhenSubdividing)
		{
			// According to https://openusd.org/release/api/class_usd_geom_mesh.html#UsdGeom_Mesh_Primvars,
			// "Normals should not be authored on a subdivision mesh, since subdivision algorithms define their own normals.
			// They should only be authored for polygonal meshes (subdivisionScheme = "none")."
			// There is no free normal computation to be had from OpenSubdiv subdivision algoriths as far as I can tell however.
			// We'd have to compute them manually as in
			// https://github.com/PixarAnimationStudios/OpenSubdiv/blob/release/tutorials/far/tutorial_2_3/far_tutorial_2_3.cpp If that is the case,
			// we may as well just ignore normals here and let RepairNormalsAndTangents fix it, since it will need to run it to compute tangents
			// anyway

			InOutMeshData.Normals = {};
			InOutMeshData.NormalIndices = {};
			InOutMeshData.NormalInterpolation = {};
		}
		else
		{
			if (InOutMeshData.NormalIndices.size() > 0)
			{
				if (InOutMeshData.NormalInterpolation == pxr::UsdGeomTokens->faceVarying)
				{
					NormalsFaceVaryingChannel = FaceVaryingChannelCounter++;
				}
				else
				{
					InOutMeshData.Normals = ComputeFlattened(InOutMeshData.Normals, InOutMeshData.NormalIndices);
					InOutMeshData.NormalIndices = {};
				}
			}
			if (InOutMeshData.NormalInterpolation == pxr::UsdGeomTokens->faceVarying && NormalsFaceVaryingChannel == INDEX_NONE)
			{
				CreateIotaIndicesIfNeeded();
				NormalsFaceVaryingChannel = IotaFaceVaryingChannel;
			}
		}

		// Velocities
		int32 VelocitiesFaceVaryingChannel = INDEX_NONE;
		if (InOutMeshData.VelocityInterpolation == pxr::UsdGeomTokens->faceVarying)
		{
			// Simple attributes can't be indexed, so if this is faceVarying then we know we need the iota indices
			CreateIotaIndicesIfNeeded();
			VelocitiesFaceVaryingChannel = IotaFaceVaryingChannel;
		}

		// DisplayColors
		int32 DisplayColorsFaceVaryingChannel = INDEX_NONE;
		if (InOutMeshData.DisplayColorIndices.size() > 0)
		{
			if (InOutMeshData.DisplayColorInterpolation == pxr::UsdGeomTokens->faceVarying)
			{
				DisplayColorsFaceVaryingChannel = FaceVaryingChannelCounter++;
			}
			else
			{
				InOutMeshData.DisplayColors = ComputeFlattened(InOutMeshData.DisplayColors, InOutMeshData.DisplayColorIndices);
				InOutMeshData.DisplayColorIndices = {};
			}
		}
		if (InOutMeshData.DisplayColorInterpolation == pxr::UsdGeomTokens->faceVarying && DisplayColorsFaceVaryingChannel == INDEX_NONE)
		{
			CreateIotaIndicesIfNeeded();
			DisplayColorsFaceVaryingChannel = IotaFaceVaryingChannel;
		}

		// DisplayOpacities
		int32 DisplayOpacitiesFaceVaryingChannel = INDEX_NONE;
		if (InOutMeshData.DisplayOpacityIndices.size() > 0)
		{
			if (InOutMeshData.DisplayOpacityInterpolation == pxr::UsdGeomTokens->faceVarying)
			{
				DisplayOpacitiesFaceVaryingChannel = FaceVaryingChannelCounter++;
			}
			else
			{
				InOutMeshData.DisplayOpacities = ComputeFlattened(InOutMeshData.DisplayOpacities, InOutMeshData.DisplayOpacityIndices);
				InOutMeshData.DisplayOpacityIndices = {};
			}
		}
		if (InOutMeshData.DisplayOpacityInterpolation == pxr::UsdGeomTokens->faceVarying && DisplayOpacitiesFaceVaryingChannel == INDEX_NONE)
		{
			CreateIotaIndicesIfNeeded();
			DisplayOpacitiesFaceVaryingChannel = IotaFaceVaryingChannel;
		}

		// UVs
		const int32 NumUVSets = InOutMeshData.UVSets.Num();
		if (!ensure(InOutMeshData.UVSetIndices.Num() == NumUVSets && InOutMeshData.UVSetInterpolations.Num() == NumUVSets))
		{
			return false;
		}
		TArray<int32> UVFaceVaryingChannels;
		UVFaceVaryingChannels.SetNumUninitialized(NumUVSets);
		for (int32 UVSetIndex = 0; UVSetIndex < NumUVSets; ++UVSetIndex)
		{
			int32& UVFaceVaryingChannel = UVFaceVaryingChannels[UVSetIndex];
			pxr::VtArray<pxr::GfVec2f>& UVSet = InOutMeshData.UVSets[UVSetIndex];
			pxr::TfToken& UVSetInterpolation = InOutMeshData.UVSetInterpolations[UVSetIndex];
			pxr::VtArray<int>& UVSetIndices = InOutMeshData.UVSetIndices[UVSetIndex];

			UVFaceVaryingChannel = INDEX_NONE;
			if (UVSetIndices.size() > 0)
			{
				if (UVSetInterpolation == pxr::UsdGeomTokens->faceVarying)
				{
					UVFaceVaryingChannel = FaceVaryingChannelCounter++;
				}
				else
				{
					UVSet = ComputeFlattened(UVSet, UVSetIndices);
					UVSetIndices = {};
				}
			}
			if (UVSetInterpolation == pxr::UsdGeomTokens->faceVarying && UVFaceVaryingChannel == INDEX_NONE)
			{
				CreateIotaIndicesIfNeeded();
				UVFaceVaryingChannel = IotaFaceVaryingChannel;
			}
		}

		pxr::TfToken MaterialIndicesInterpolation = pxr::UsdGeomTokens->uniform;

		pxr::PxOsdSubdivTags SubdivTags{
			InOutMeshData.InterpolateBoundary,
			InOutMeshData.FaceVaryingInterpolation,
			InOutMeshData.CreaseMethod,
			InOutMeshData.TriangleSubdivision,
			InOutMeshData.CreaseIndices,
			InOutMeshData.CreaseLengths,
			InOutMeshData.CreaseSharpnesses,
			InOutMeshData.CornerIndices,
			InOutMeshData.CornerSharpnesses
		};

		pxr::PxOsdMeshTopology Topology{
			InOutMeshData.SubdivScheme,
			InOutMeshData.Orientation,
			InOutMeshData.FaceVertexCounts,
			InOutMeshData.FaceVertexIndices,
			InOutMeshData.HoleIndices,
			SubdivTags
		};

		std::vector<pxr::VtArray<int>> FaceVaryingTopologies;
		FaceVaryingTopologies.resize(FaceVaryingChannelCounter);
		if (PointsFaceVaryingChannel != INDEX_NONE)
		{
			FaceVaryingTopologies[PointsFaceVaryingChannel] = InOutMeshData.PointIndices;
		}
		if (NormalsFaceVaryingChannel != INDEX_NONE)
		{
			FaceVaryingTopologies[NormalsFaceVaryingChannel] = InOutMeshData.NormalIndices;
		}
		// No need to check Velocities here as there's no way it has custom indices
		if (DisplayColorsFaceVaryingChannel != INDEX_NONE)
		{
			FaceVaryingTopologies[DisplayColorsFaceVaryingChannel] = InOutMeshData.DisplayColorIndices;
		}
		if (DisplayOpacitiesFaceVaryingChannel != INDEX_NONE)
		{
			FaceVaryingTopologies[DisplayOpacitiesFaceVaryingChannel] = InOutMeshData.DisplayOpacityIndices;
		}
		for (int32 UVSetIndex = 0; UVSetIndex < NumUVSets; ++UVSetIndex)
		{
			int32 UVFaceVaryingChannel = UVFaceVaryingChannels[UVSetIndex];
			if (UVFaceVaryingChannel != INDEX_NONE)
			{
				FaceVaryingTopologies[UVFaceVaryingChannel] = InOutMeshData.UVSetIndices[UVSetIndex];
			}
		}
		// Iota being last means we replace whatever else may have been placed at the iota channel with the actual indices
		if (IotaFaceVaryingChannel != INDEX_NONE)
		{
			FaceVaryingTopologies[IotaFaceVaryingChannel] = IotaIndices;
		}

		std::shared_ptr<OpenSubdiv::Far::TopologyRefiner> TopologyRefiner = pxr::PxOsdRefinerFactory::Create(Topology, FaceVaryingTopologies);
		if (!TopologyRefiner)
		{
			return false;
		}

		// Refine our topology (we only use uniform subdivision for now)
		OpenSubdiv::Far::TopologyRefiner::UniformOptions UniformOptions{TargetSubdivLevel};
		// From tutorial_2_2: "fullTopologyInLastLevel must be true to work with faceVarying data"
		UniformOptions.fullTopologyInLastLevel = true;
		TopologyRefiner->RefineUniform(UniformOptions);

		// We're using primvar refiners here as that is the simplest method of getting our primvars
		// subdivided, but if performance becomes an issue we could try using stencil/patch tables instead
		OpenSubdiv::Far::PrimvarRefiner PrimvarRefiner{*TopologyRefiner};

		// Temp buffers where we'll store the iterative subdivision values
		pxr::VtArray<int> TempFaceVertexCounts;
		pxr::VtArray<int> TempFaceVertexIndices;
		pxr::VtArray<pxr::GfVec3f> TempPoints;
		pxr::VtArray<pxr::GfVec3f> TempNormals;
		pxr::VtArray<pxr::GfVec3f> TempVelocities;
		pxr::VtArray<pxr::GfVec3f> TempDisplayColors;
		pxr::VtArray<float> TempDisplayOpacities;
		TArray<int32> TempMaterialIndices;	  // Using a TArray saves us a memcpy when outputting results
		TArray<pxr::VtArray<pxr::GfVec2f>> TempUVSets;

		// Resize the target buffers to be large enough to hold all refinements *simultaneously* (one next to the other).
		// This is great because we can just read/write to the same buffer as we iteratively refine
		TFunction<size_t(pxr::TfToken, int32)> GetTotalNumElements =
			[&TopologyRefiner](pxr::TfToken InterpolationType, int32 FaceVaryingChannel) -> size_t
		{
			// The "GetNumXTotal()" functions also include space for the source data as well.
			// In our case we'll keep the source data on the actual source arrays so we don't have to
			// copy them over, meaning our buffers can be a bit smaller too
			const OpenSubdiv::Far::TopologyLevel& UnsubdividedLevel = TopologyRefiner->GetLevel(0);

			if (InterpolationType == pxr::UsdGeomTokens->vertex)
			{
				return TopologyRefiner->GetNumVerticesTotal() - UnsubdividedLevel.GetNumVertices();
			}
			else if (InterpolationType == pxr::UsdGeomTokens->varying)
			{
				return TopologyRefiner->GetNumVerticesTotal() - UnsubdividedLevel.GetNumVertices();
			}
			else if (InterpolationType == pxr::UsdGeomTokens->faceVarying)
			{
				return TopologyRefiner->GetNumFVarValuesTotal(FaceVaryingChannel) - UnsubdividedLevel.GetNumFVarValues(FaceVaryingChannel);
			}
			else if (InterpolationType == pxr::UsdGeomTokens->uniform)
			{
				return TopologyRefiner->GetNumFacesTotal() - UnsubdividedLevel.GetNumFaces();
			}
			else if (InterpolationType == pxr::UsdGeomTokens->constant)
			{
				return 1;
			}
			else
			{
				return 0;
			}
		};
		TempPoints.resize(GetTotalNumElements(InOutMeshData.PointInterpolation, PointsFaceVaryingChannel));
		TempNormals.resize(GetTotalNumElements(InOutMeshData.NormalInterpolation, NormalsFaceVaryingChannel));
		TempVelocities.resize(GetTotalNumElements(InOutMeshData.VelocityInterpolation, VelocitiesFaceVaryingChannel));
		TempDisplayColors.resize(GetTotalNumElements(InOutMeshData.DisplayColorInterpolation, DisplayColorsFaceVaryingChannel));
		TempDisplayOpacities.resize(GetTotalNumElements(InOutMeshData.DisplayOpacityInterpolation, DisplayOpacitiesFaceVaryingChannel));
		TempMaterialIndices.SetNum(GetTotalNumElements(MaterialIndicesInterpolation, /*FaceVaryingChannel*/ 0));	// Always 'uniform'
		TempUVSets.SetNum(NumUVSets);
		for (int32 UVSetIndex = 0; UVSetIndex < NumUVSets; ++UVSetIndex)
		{
			pxr::VtArray<pxr::GfVec2f>& UVSet = TempUVSets[UVSetIndex];
			UVSet.resize(GetTotalNumElements(InOutMeshData.UVSetInterpolations[UVSetIndex], UVFaceVaryingChannels[UVSetIndex]));
		}

		// Use the right function from PrimvarRefiner depending on InterpolationType.
		// Has to be auto as we reuse this for FSubdivVec3f*, FSubdivFloat* and FSubdivInt* SrcPtr and DstPtr
		auto InterpolateAttribute =
			[&PrimvarRefiner](auto SrcPtr, auto DstPtr, pxr::TfToken InterpolationType, int32 CurrentRefinementLevel, int32 FaceVaryingChannel)
		{
			// If the mesh doesn't have any values for an attribute, its SrcPtr will be nullptr
			if (!SrcPtr || !DstPtr)
			{
				return;
			}

			if (InterpolationType == pxr::UsdGeomTokens->vertex)
			{
				PrimvarRefiner.Interpolate(CurrentRefinementLevel, SrcPtr, DstPtr);
			}
			else if (InterpolationType == pxr::UsdGeomTokens->varying)
			{
				PrimvarRefiner.InterpolateVarying(CurrentRefinementLevel, SrcPtr, DstPtr);
			}
			else if (InterpolationType == pxr::UsdGeomTokens->faceVarying)
			{
				PrimvarRefiner.InterpolateFaceVarying(CurrentRefinementLevel, SrcPtr, DstPtr, FaceVaryingChannel);
			}
			else if (InterpolationType == pxr::UsdGeomTokens->uniform)
			{
				PrimvarRefiner.InterpolateFaceUniform(CurrentRefinementLevel, SrcPtr, DstPtr);
			}
			else if (InterpolationType == pxr::UsdGeomTokens->constant)
			{
				// We need to move this because for the very first refinement we rely on InterpolateAttribute to
				// move the value from the source array to the dest array
				*DstPtr = *SrcPtr;
			}
		};

		// Note how these start by pointing at the actual source data. After the first refinement iteration
		// these (as well as the dst pointers) will all point at different locations within the Temp buffers
		FSubdivVec3f* SrcPointsPtr = reinterpret_cast<FSubdivVec3f*>(InOutMeshData.Points.data());
		FSubdivVec3f* SrcNormalsPtr = reinterpret_cast<FSubdivVec3f*>(InOutMeshData.Normals.data());
		FSubdivVec3f* SrcVelocitiesPtr = reinterpret_cast<FSubdivVec3f*>(InOutMeshData.Velocities.data());
		FSubdivVec3f* SrcDisplayColorsPtr = reinterpret_cast<FSubdivVec3f*>(InOutMeshData.DisplayColors.data());
		FSubdivFloat* SrcDisplayOpacitiesPtr = reinterpret_cast<FSubdivFloat*>(InOutMeshData.DisplayOpacities.data());
		FSubdivInt* SrcMaterialIndicesPtr = reinterpret_cast<FSubdivInt*>(InOutMeshData.LocalMaterialInfo.MaterialIndices.GetData());
		TArray<FSubdivVec2f*> SrcUVSetPtrs;
		SrcUVSetPtrs.SetNum(NumUVSets);
		for (int32 UVSetIndex = 0; UVSetIndex < NumUVSets; ++UVSetIndex)
		{
			SrcUVSetPtrs[UVSetIndex] = reinterpret_cast<FSubdivVec2f*>(InOutMeshData.UVSets[UVSetIndex].data());
		}

		FSubdivVec3f* DstPointsPtr = reinterpret_cast<FSubdivVec3f*>(TempPoints.data());
		FSubdivVec3f* DstNormalsPtr = reinterpret_cast<FSubdivVec3f*>(TempNormals.data());
		FSubdivVec3f* DstVelocitiesPtr = reinterpret_cast<FSubdivVec3f*>(TempVelocities.data());
		FSubdivVec3f* DstDisplayColorsPtr = reinterpret_cast<FSubdivVec3f*>(TempDisplayColors.data());
		FSubdivFloat* DstDisplayOpacitiesPtr = reinterpret_cast<FSubdivFloat*>(TempDisplayOpacities.data());
		FSubdivInt* DstMaterialIndicesPtr = reinterpret_cast<FSubdivInt*>(TempMaterialIndices.GetData());
		TArray<FSubdivVec2f*> DstUVSetPtrs;
		DstUVSetPtrs.SetNum(NumUVSets);
		for (int32 UVSetIndex = 0; UVSetIndex < NumUVSets; ++UVSetIndex)
		{
			DstUVSetPtrs[UVSetIndex] = reinterpret_cast<FSubdivVec2f*>(TempUVSets[UVSetIndex].data());
		}

		TFunction<int(const OpenSubdiv::Far::TopologyLevel&, pxr::TfToken, int32)> GetPtrIncrement =
			[](const OpenSubdiv::Far::TopologyLevel& Level, pxr::TfToken InterpolationType, int32 FaceVaryingChannel) -> int
		{
			if (InterpolationType == pxr::UsdGeomTokens->vertex)
			{
				return Level.GetNumVertices();
			}
			else if (InterpolationType == pxr::UsdGeomTokens->varying)
			{
				return Level.GetNumVertices();
			}
			else if (InterpolationType == pxr::UsdGeomTokens->faceVarying)
			{
				return Level.GetNumFVarValues(FaceVaryingChannel);
			}
			else if (InterpolationType == pxr::UsdGeomTokens->uniform)
			{
				return Level.GetNumFaces();
			}
			else
			{
				// For pxr::UsdGeomTokens->constant don't increment anything as we'll really just have a single value throughout
				return 0;
			}
		};

		// Actually refine all of our attributes/primvars
		// Inspired by
		// https://github.com/PixarAnimationStudios/OpenSubdiv/blob/7d0ab5530feef693ac0a920585b5c663b80773b3/tutorials/far/tutorial_2_2/far_tutorial_2_2.cpp#L293
		// but avoiding the initial copy from the source data arrays
		for (int32 CurrentLevel = 1; CurrentLevel <= TargetSubdivLevel; ++CurrentLevel)
		{
			InterpolateAttribute(SrcPointsPtr, DstPointsPtr, InOutMeshData.PointInterpolation, CurrentLevel, PointsFaceVaryingChannel);
			InterpolateAttribute(SrcNormalsPtr, DstNormalsPtr, InOutMeshData.NormalInterpolation, CurrentLevel, NormalsFaceVaryingChannel);
			InterpolateAttribute(SrcVelocitiesPtr, DstVelocitiesPtr, InOutMeshData.VelocityInterpolation, CurrentLevel, VelocitiesFaceVaryingChannel);
			InterpolateAttribute(
				SrcDisplayColorsPtr,
				DstDisplayColorsPtr,
				InOutMeshData.DisplayColorInterpolation,
				CurrentLevel,
				DisplayColorsFaceVaryingChannel
			);
			InterpolateAttribute(
				SrcDisplayOpacitiesPtr,
				DstDisplayOpacitiesPtr,
				InOutMeshData.DisplayOpacityInterpolation,
				CurrentLevel,
				DisplayOpacitiesFaceVaryingChannel
			);
			InterpolateAttribute(
				SrcMaterialIndicesPtr,
				DstMaterialIndicesPtr,
				MaterialIndicesInterpolation,
				CurrentLevel,
				/*FaceVaryingChannel*/ 0
			);	  // Always 'uniform'
			for (int32 UVSetIndex = 0; UVSetIndex < NumUVSets; ++UVSetIndex)
			{
				InterpolateAttribute(
					SrcUVSetPtrs[UVSetIndex],
					DstUVSetPtrs[UVSetIndex],
					InOutMeshData.UVSetInterpolations[UVSetIndex],
					CurrentLevel,
					UVFaceVaryingChannels[UVSetIndex]
				);
			}

			SrcPointsPtr = DstPointsPtr;
			SrcNormalsPtr = DstNormalsPtr;
			SrcVelocitiesPtr = DstVelocitiesPtr;
			SrcDisplayColorsPtr = DstDisplayColorsPtr;
			SrcDisplayOpacitiesPtr = DstDisplayOpacitiesPtr;
			SrcMaterialIndicesPtr = DstMaterialIndicesPtr;
			SrcUVSetPtrs = DstUVSetPtrs;

			const OpenSubdiv::Far::TopologyLevel& AfterSubdiv = TopologyRefiner->GetLevel(CurrentLevel);
			DstPointsPtr += GetPtrIncrement(AfterSubdiv, InOutMeshData.PointInterpolation, PointsFaceVaryingChannel);
			DstNormalsPtr += GetPtrIncrement(AfterSubdiv, InOutMeshData.NormalInterpolation, NormalsFaceVaryingChannel);
			DstVelocitiesPtr += GetPtrIncrement(AfterSubdiv, InOutMeshData.VelocityInterpolation, VelocitiesFaceVaryingChannel);
			DstDisplayColorsPtr += GetPtrIncrement(AfterSubdiv, InOutMeshData.DisplayColorInterpolation, DisplayColorsFaceVaryingChannel);
			DstDisplayOpacitiesPtr += GetPtrIncrement(AfterSubdiv, InOutMeshData.DisplayOpacityInterpolation, DisplayOpacitiesFaceVaryingChannel);
			DstMaterialIndicesPtr += GetPtrIncrement(AfterSubdiv, MaterialIndicesInterpolation, /*FaceVaryingChannel*/ 0);
			for (int32 UVSetIndex = 0; UVSetIndex < NumUVSets; ++UVSetIndex)
			{
				DstUVSetPtrs[UVSetIndex] += GetPtrIncrement(
					AfterSubdiv,
					InOutMeshData.UVSetInterpolations[UVSetIndex],
					UVFaceVaryingChannels[UVSetIndex]
				);
			}
		}

		// Shrink down the Result buffers to just contain the values from the last refinement.
		// We use SrcPtrs here because they were left at the start of the last refinement section of each array
		TempPoints.erase(TempPoints.begin(), reinterpret_cast<pxr::GfVec3f*>(SrcPointsPtr));
		TempNormals.erase(TempNormals.begin(), reinterpret_cast<pxr::GfVec3f*>(SrcNormalsPtr));
		TempVelocities.erase(TempVelocities.begin(), reinterpret_cast<pxr::GfVec3f*>(SrcVelocitiesPtr));
		TempDisplayColors.erase(TempDisplayColors.begin(), reinterpret_cast<pxr::GfVec3f*>(SrcDisplayColorsPtr));
		TempDisplayOpacities.erase(TempDisplayOpacities.begin(), reinterpret_cast<float*>(SrcDisplayOpacitiesPtr));
		TempMaterialIndices.RemoveAt(0, reinterpret_cast<int32*>(SrcMaterialIndicesPtr) - &TempMaterialIndices[0]);
		for (int32 UVSetIndex = 0; UVSetIndex < NumUVSets; ++UVSetIndex)
		{
			pxr::VtArray<pxr::GfVec2f>& TempUVSet = TempUVSets[UVSetIndex];
			TempUVSet.erase(TempUVSet.begin(), reinterpret_cast<pxr::GfVec2f*>(SrcUVSetPtrs[UVSetIndex]));
		}

		// If we're interpolating normals we have to take a pass to actually normalize them, as the primvar interpolation won't ensure that
		if (!GIgnoreNormalsWhenSubdividing)
		{
			for (pxr::GfVec3f& Normal : TempNormals)
			{
				Normal.Normalize();
			}
		}

		// Face vertex counts and indices
		const OpenSubdiv::Far::TopologyLevel& FinalLevel = TopologyRefiner->GetLevel(TargetSubdivLevel);
		TempFaceVertexCounts.resize(FinalLevel.GetNumFaces());
		TempFaceVertexIndices.reserve(FinalLevel.GetNumFaceVertices());
		for (int32 FaceIndex = 0; FaceIndex < FinalLevel.GetNumFaces(); ++FaceIndex)
		{
			OpenSubdiv::Far::ConstIndexArray Face = FinalLevel.GetFaceVertices(FaceIndex);
			TempFaceVertexCounts[FaceIndex] = Face.size();

			for (int32 FaceVertexIndex = 0; FaceVertexIndex < Face.size(); ++FaceVertexIndex)
			{
				TempFaceVertexIndices.push_back(Face[FaceVertexIndex]);
			}
		}

		// All faceVarying primvars can be arbitrarily indexed after subdivision (e.g. we may have 96 face vertices but end up
		// with only 54 values for a particular primvar, because the topology allowed them to be shared). Our downstream code
		// can't generally consume indexed stuff though, so here we flatten those primvar values to always be one per face vertex.
		// The other interpolation types never have this issue however (e.g. 'vertex' interpolation will always output one
		// value for each vertex)
		if (InOutMeshData.PointInterpolation == pxr::UsdGeomTokens->faceVarying)
		{
			FlattenFaceVaryingValues(TempPoints, PointsFaceVaryingChannel, FinalLevel);
		}
		if (InOutMeshData.NormalInterpolation == pxr::UsdGeomTokens->faceVarying)
		{
			FlattenFaceVaryingValues(TempNormals, NormalsFaceVaryingChannel, FinalLevel);
		}
		if (InOutMeshData.DisplayColorInterpolation == pxr::UsdGeomTokens->faceVarying)
		{
			FlattenFaceVaryingValues(TempDisplayColors, DisplayColorsFaceVaryingChannel, FinalLevel);
		}
		if (InOutMeshData.DisplayOpacityInterpolation == pxr::UsdGeomTokens->faceVarying)
		{
			FlattenFaceVaryingValues(TempDisplayOpacities, DisplayOpacitiesFaceVaryingChannel, FinalLevel);
		}
		for (int32 UVSetIndex = 0; UVSetIndex < NumUVSets; ++UVSetIndex)
		{
			if (InOutMeshData.UVSetInterpolations[UVSetIndex] == pxr::UsdGeomTokens->faceVarying)
			{
				// Note: Our downstream ConvertMeshData code *can* handle indexed UV sets, but it's simpler
				// to just flatten discard them here. We could revisit this later and evaluate the impact on
				// performance of keeping indices around though
				FlattenFaceVaryingValues(TempUVSets[UVSetIndex], UVFaceVaryingChannels[UVSetIndex], FinalLevel);
			}

			InOutMeshData.UVSetIndices[UVSetIndex] = {};
		}
		InOutMeshData.PointIndices = {};
		InOutMeshData.NormalIndices = {};
		InOutMeshData.DisplayColorIndices = {};
		InOutMeshData.DisplayOpacityIndices = {};

		// Output results
		Swap(TempFaceVertexCounts, InOutMeshData.FaceVertexCounts);
		Swap(TempFaceVertexIndices, InOutMeshData.FaceVertexIndices);
		Swap(TempPoints, InOutMeshData.Points);
		Swap(TempNormals, InOutMeshData.Normals);
		Swap(TempVelocities, InOutMeshData.Velocities);
		Swap(TempDisplayColors, InOutMeshData.DisplayColors);
		Swap(TempDisplayOpacities, InOutMeshData.DisplayOpacities);
		Swap(TempMaterialIndices, InOutMeshData.LocalMaterialInfo.MaterialIndices);
		Swap(TempUVSets, InOutMeshData.UVSets);

		return true;
	}

	void FlattenIndexedPrimvars(FUsdMeshData& InOutMeshData)
	{
		InOutMeshData.Points = ComputeFlattened(InOutMeshData.Points, InOutMeshData.PointIndices);
		InOutMeshData.PointIndices = {};

		InOutMeshData.Normals = ComputeFlattened(InOutMeshData.Normals, InOutMeshData.NormalIndices);
		InOutMeshData.NormalIndices = {};

		InOutMeshData.DisplayColors = ComputeFlattened(InOutMeshData.DisplayColors, InOutMeshData.DisplayColorIndices);
		InOutMeshData.DisplayColorIndices = {};

		InOutMeshData.DisplayOpacities = ComputeFlattened(InOutMeshData.DisplayOpacities, InOutMeshData.DisplayOpacityIndices);
		InOutMeshData.DisplayOpacityIndices = {};

		InOutMeshData.JointIndices = ComputeFlattened(InOutMeshData.JointIndices, InOutMeshData.JointIndexIndices);
		InOutMeshData.JointIndexIndices = {};

		InOutMeshData.JointWeights = ComputeFlattened(InOutMeshData.JointWeights, InOutMeshData.JointWeightIndices);
		InOutMeshData.JointWeightIndices = {};

		for (int32 UVSetIndex = 0; UVSetIndex < InOutMeshData.UVSets.Num(); ++UVSetIndex)
		{
			pxr::VtArray<pxr::GfVec2f>& UVs = InOutMeshData.UVSets[UVSetIndex];
			pxr::VtArray<int>& UVIndices = InOutMeshData.UVSetIndices[UVSetIndex];

			UVs = ComputeFlattened(UVs, UVIndices);
			UVIndices = {};
		}

		for (TPair<FString, FUsdMeshData::FPrimvar>& Pair : InOutMeshData.Primvars)
		{
			FUsdMeshData::FPrimvar& Primvar = Pair.Value;

			Visit(
				TOverloaded{
					[&Primvar](pxr::VtArray<bool>& Data)
					{
						Data = ComputeFlattened(Data, Primvar.Indices);
					},
					[&Primvar](pxr::VtArray<int32>& Data)
					{
						Data = ComputeFlattened(Data, Primvar.Indices);
					},
					[&Primvar](pxr::VtArray<float>& Data)
					{
						Data = ComputeFlattened(Data, Primvar.Indices);
					},
					[&Primvar](pxr::VtArray<pxr::GfVec2f>& Data)
					{
						Data = ComputeFlattened(Data, Primvar.Indices);
					},
					[&Primvar](pxr::VtArray<pxr::GfVec3f>& Data)
					{
						Data = ComputeFlattened(Data, Primvar.Indices);
					},
					[&Primvar](pxr::VtArray<pxr::GfVec4f>& Data)
					{
						Data = ComputeFlattened(Data, Primvar.Indices);
					}
				},
				Primvar.Data
			);

			Primvar.Indices = {};
		}
	}

	void RegisterPrimvarAttributes(const TMap<FString, FUsdMeshData::FPrimvar>& InCustomPrimvars, FMeshDescription& OutMeshDescription)
	{
		// Register Primvars attribute
		for (const TPair<FString, FUsdMeshData::FPrimvar>& Pair : InCustomPrimvars)
		{
			const FString& PrimvarName = Pair.Key;
			const FUsdMeshData::FPrimvar& CustomPrimvar = Pair.Value;

			// Register the primvar if it doesn't exist
			if (!OutMeshDescription.VertexInstanceAttributes().HasAttribute(*PrimvarName))
			{
				Visit(
					TOverloaded{
						[&PrimvarName, &OutMeshDescription](const pxr::VtArray<bool>&)
						{
							OutMeshDescription.VertexInstanceAttributes().RegisterAttribute<bool>(*PrimvarName);
						},
						[&PrimvarName, &OutMeshDescription](const pxr::VtArray<int32>&)
						{
							OutMeshDescription.VertexInstanceAttributes().RegisterAttribute<int32>(*PrimvarName);
						},
						[&PrimvarName, &OutMeshDescription](const pxr::VtArray<float>&)
						{
							OutMeshDescription.VertexInstanceAttributes().RegisterAttribute<float>(*PrimvarName);
						},
						[&PrimvarName, &OutMeshDescription](const pxr::VtArray<pxr::GfVec2f>&)
						{
							OutMeshDescription.VertexInstanceAttributes().RegisterAttribute<FVector2f>(*PrimvarName);
						},
						[&PrimvarName, &OutMeshDescription](const pxr::VtArray<pxr::GfVec3f>&)
						{
							OutMeshDescription.VertexInstanceAttributes().RegisterAttribute<FVector3f>(*PrimvarName);
						},
						[&PrimvarName, &OutMeshDescription](const pxr::VtArray<pxr::GfVec4f>&)
						{
							OutMeshDescription.VertexInstanceAttributes().RegisterAttribute<FVector4f>(*PrimvarName);
						}
					},
					CustomPrimvar.Data
				);
			}
		}
	}

	bool ConvertMeshData(
		const FUsdMeshData& InMeshData,
		const FUsdStageInfo& InStageInfo,
		const UsdToUnreal::FUsdMeshConversionOptions& InOptions,
		FMeshDescription& OutMeshDescription,
		UsdUtils::FUsdPrimMaterialAssignmentInfo& OutMaterialAssignments
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UsdToUnreal::ConvertMeshData);

		FScopedUsdAllocs Allocs;

		// ConvertMeshData can't handle indexed primvars! Make sure you call FlattenIndexedPrimvars beforehand
		ensure(InMeshData.PointIndices.empty());
		ensure(InMeshData.NormalIndices.empty());
		ensure(InMeshData.DisplayColorIndices.empty());
		ensure(InMeshData.DisplayOpacityIndices.empty());
		for (int32 UVSetIndex = 0; UVSetIndex < InMeshData.UVSets.Num(); ++UVSetIndex)
		{
			ensure(InMeshData.UVSetIndices[UVSetIndex].empty());
		}
		for (const TPair<FString, FUsdMeshData::FPrimvar>& Pair : InMeshData.Primvars)
		{
			ensure(Pair.Value.Indices.empty());
		}

		// Material assignments
		const TArray<UsdUtils::FUsdPrimMaterialSlot>& LocalMaterialSlots = InMeshData.LocalMaterialInfo.Slots;
		const TArray<int32>& FaceMaterialIndices = InMeshData.LocalMaterialInfo.MaterialIndices;

		// Position 3 in this has the value 6 --> Local material slot #3 is actually the combined material slot #6
		TArray<int32> LocalToCombinedMaterialSlotIndices;
		LocalToCombinedMaterialSlotIndices.SetNumZeroed(InMeshData.LocalMaterialInfo.Slots.Num());

		if (InOptions.bMergeIdenticalMaterialSlots)
		{
			// Build a map of our existing slots since we can hash the entire slot, and our incoming mesh may have an arbitrary number of new slots
			TMap<UsdUtils::FUsdPrimMaterialSlot, int32> CombinedMaterialSlotsToIndex;
			for (int32 Index = 0; Index < OutMaterialAssignments.Slots.Num(); ++Index)
			{
				const UsdUtils::FUsdPrimMaterialSlot& Slot = OutMaterialAssignments.Slots[Index];

				// Combine entries in this way so that we can append PrimPaths
				TMap<UsdUtils::FUsdPrimMaterialSlot, int32>::TKeyIterator KeyIt = CombinedMaterialSlotsToIndex.CreateKeyIterator(Slot);
				if (KeyIt)
				{
					KeyIt.Key().PrimPaths.Append(Slot.PrimPaths);
					KeyIt.Value() = Index;
				}
				else
				{
					CombinedMaterialSlotsToIndex.Add(Slot, Index);
				}
			}

			// Combine our LocalSlots into CombinedMaterialSlotsToIndex
			for (int32 LocalIndex = 0; LocalIndex < InMeshData.LocalMaterialInfo.Slots.Num(); ++LocalIndex)
			{
				const UsdUtils::FUsdPrimMaterialSlot& LocalSlot = InMeshData.LocalMaterialInfo.Slots[LocalIndex];

				// Combine entries in this way so that we can append PrimPaths
				TMap<UsdUtils::FUsdPrimMaterialSlot, int32>::TKeyIterator KeyIt = CombinedMaterialSlotsToIndex.CreateKeyIterator(LocalSlot);
				if (KeyIt)
				{
					KeyIt.Key().PrimPaths.Append(LocalSlot.PrimPaths);

					const int32 ExistingCombinedIndex = KeyIt.Value();
					LocalToCombinedMaterialSlotIndices[LocalIndex] = ExistingCombinedIndex;
				}
				else
				{
					int32 NewIndex = OutMaterialAssignments.Slots.Add(LocalSlot);
					CombinedMaterialSlotsToIndex.Add(LocalSlot, NewIndex);
					LocalToCombinedMaterialSlotIndices[LocalIndex] = NewIndex;
				}
			}

			// Now that we merged all prim paths into they keys of CombinedMaterialSlotsToIndex, let's copy them back into
			// our output
			for (UsdUtils::FUsdPrimMaterialSlot& Slot : OutMaterialAssignments.Slots)
			{
				TMap<UsdUtils::FUsdPrimMaterialSlot, int32>::TKeyIterator KeyIt = CombinedMaterialSlotsToIndex.CreateKeyIterator(Slot);
				ensure(KeyIt);
				Slot.PrimPaths = KeyIt.Key().PrimPaths;
			}
		}
		else
		{
			// Just append our new local material slots at the end of MaterialAssignments
			OutMaterialAssignments.Slots.Append(InMeshData.LocalMaterialInfo.Slots);
			for (int32 LocalIndex = 0; LocalIndex < InMeshData.LocalMaterialInfo.Slots.Num(); ++LocalIndex)
			{
				LocalToCombinedMaterialSlotIndices[LocalIndex] = LocalIndex + InMeshData.MaterialIndexOffset;
			}
		}

		const int32 VertexOffset = OutMeshDescription.Vertices().Num();
		const int32 VertexInstanceOffset = OutMeshDescription.VertexInstances().Num();

		FStaticMeshAttributes StaticMeshAttributes(OutMeshDescription);

		// Keep existing because we may be calling this in the context of collapsing, where our OutMeshDescription
		// already has some data
		const bool bKeepExistingAttribute = true;
		StaticMeshAttributes.Register(bKeepExistingAttribute);

		// Vertex positions
		const uint32 NumPoints = InMeshData.Points.size();
		TVertexAttributesRef<FVector3f> MeshDescriptionVertexPositions = StaticMeshAttributes.GetVertexPositions();
		TArray<FVertexID> VertexIDMap;
		{
			if (NumPoints < 3)
			{
				return false;
			}

			OutMeshDescription.ReserveNewVertices(NumPoints);
			VertexIDMap.Reserve(NumPoints);

			for (size_t LocalPointIndex = 0; LocalPointIndex < NumPoints; ++LocalPointIndex)
			{
				const pxr::GfVec3f& Point = InMeshData.Points.cdata()[LocalPointIndex];

				FVector Position = InOptions.AdditionalTransform.TransformPosition(UsdToUnreal::ConvertVector(InStageInfo, Point));

				FVertexID AddedVertexId = OutMeshDescription.CreateVertex();
				VertexIDMap.Add(AddedVertexId);
				MeshDescriptionVertexPositions[AddedVertexId] = (FVector3f)Position;
			}
		}

		uint32 NumSkippedPolygons = 0;
		uint32 NumPolygons = InMeshData.FaceVertexCounts.size();
		if (NumPolygons < 1)
		{
			return false;
		}
		if (InMeshData.FaceVertexIndices.size() < 1)
		{
			return false;
		}

		// Polygons
		{
			TArray<FVertexInstanceID> CornerInstanceIDs;
			TArray<FVertexID> CornerVerticesIDs;
			int32 CurrentVertexInstanceIndex = 0;
			TPolygonGroupAttributesRef<FName> MaterialSlotNames = StaticMeshAttributes.GetPolygonGroupMaterialSlotNames();

			// Prepare our polygon group mapping acknowledging whatever is already in the mesh description
			TMap<int32, FPolygonGroupID> PolygonGroupMapping;
			for (FPolygonGroupID PolygonGroupID : OutMeshDescription.PolygonGroups().GetElementIDs())
			{
				PolygonGroupMapping.Add(PolygonGroupID.GetValue(), PolygonGroupID);
			}

			auto EnsurePolygonGroupForIndex = [&PolygonGroupMapping, &OutMeshDescription, &MaterialSlotNames](int32 Index)
			{
				if (!PolygonGroupMapping.Contains(Index))
				{
					FPolygonGroupID NewPolygonGroup = OutMeshDescription.CreatePolygonGroup();
					PolygonGroupMapping.Add(Index, NewPolygonGroup);

					// This is important for runtime, where the material slots are matched to LOD sections based on their material slot name
					MaterialSlotNames[NewPolygonGroup] = *LexToString(NewPolygonGroup.GetValue());
				}
			};

			// Material slots
			int32 LastHandledIndex = 0;
			for (int32 LocalMaterialIndex = 0; LocalMaterialIndex < InMeshData.LocalMaterialInfo.Slots.Num(); ++LocalMaterialIndex)
			{
				const int32 CombinedMaterialIndex = LocalToCombinedMaterialSlotIndices[LocalMaterialIndex];

				if (InOptions.bMergeIdenticalMaterialSlots)
				{
					// Ensure we have polygon groups for every combined material slot index up until the one we'll use.
					// Doing this keeps a 1-to-1 mapping between polygon group IDs and combined material slot IDs.
					// This is particularly relevant when merging slots in the context of LODs, when we won't share the MeshDescriptions between
					// calls to this function, but we will share the combined material assignments. There will be nothing to prefill our
					// PolygonGroupMapping in that case, but since it's a 1-to-1 mapping it's easy to set it up here
					for (; LastHandledIndex <= CombinedMaterialIndex; ++LastHandledIndex)
					{
						EnsurePolygonGroupForIndex(LastHandledIndex);
					}
				}
				else
				{
					// If we're not merging slots we should always make brand new polygon groups for every material slot
					// we get here
					EnsurePolygonGroupForIndex(CombinedMaterialIndex);
				}
			}

			// Velocities
			if (InMeshData.Velocities.size() > 0)
			{
				if (!OutMeshDescription.VertexInstanceAttributes().HasAttribute(MeshAttribute::VertexInstance::Velocity))
				{
					OutMeshDescription.VertexInstanceAttributes().RegisterAttribute<FVector3f>(
						MeshAttribute::VertexInstance::Velocity,
						1,
						FVector3f::ZeroVector,
						EMeshAttributeFlags::Lerpable
					);
				}
			}

			// Register Primvars attribute
			RegisterPrimvarAttributes(InMeshData.Primvars, OutMeshDescription);

			// UVs
			TVertexInstanceAttributesRef<FVector2f> MeshDescriptionUVs = StaticMeshAttributes.GetVertexInstanceUVs();

			struct FUVSet
			{
				int32 UVSetIndexUE;	   // The user may only have 'uv4' and 'uv5', so we can't just use array indices to find the target UV channel
				pxr::VtVec2fArray UVs;

				pxr::TfToken InterpType = pxr::UsdGeomTokens->faceVarying;
			};

			TArray<FUVSet> UVSets;

			int32 HighestAddedUVChannel = 0;
			for (int32 UVChannelIndex = 0; UVChannelIndex < InMeshData.UVSets.Num(); ++UVChannelIndex)
			{
				FUVSet UVSet;
				UVSet.InterpType = InMeshData.UVSetInterpolations[UVChannelIndex];
				UVSet.UVSetIndexUE = UVChannelIndex;

				if (InMeshData.UVSetIndices[UVChannelIndex].size() > 0)
				{
					UVSet.UVs = InMeshData.UVSets[UVChannelIndex];

					if (UVSet.UVs.size() > 0)
					{
						UVSets.Add(MoveTemp(UVSet));
						HighestAddedUVChannel = UVSet.UVSetIndexUE;
					}
				}
				else
				{
					UVSet.UVs = InMeshData.UVSets[UVChannelIndex];
					if (UVSet.UVs.size() > 0)
					{
						UVSets.Add(MoveTemp(UVSet));
						HighestAddedUVChannel = UVSet.UVSetIndexUE;
					}
				}
			}

			// When importing multiple mesh pieces to the same static mesh.  Ensure each mesh piece has the same number of UVs
			{
				int32 ExistingUVCount = MeshDescriptionUVs.GetNumChannels();
				int32 NumUVs = FMath::Max(HighestAddedUVChannel + 1, ExistingUVCount);

				// When we provide a PrimvarToUVIndex map to this function it means we'll end up combining this
				// MeshDescription with others later (e.g. due to collapsing or multiple-LOD meshes).
				// In that case we can get better results by making sure all of the individual MeshDescriptions have the
				// same total number of UV sets, even if the unused ones are empty.
				// Otherwise, if we e.g. have a material reading UVIndex3 when we only have a single UV set, UE seems to
				// just read that one UV set anyway, which is somewhat unexpected and can be misleading
				if (InMeshData.ProvidedNumUVSets.IsSet())
				{
					NumUVs = FMath::Max<int32>(InMeshData.ProvidedNumUVSets.GetValue(), NumUVs);
				}

				NumUVs = FMath::Min<int32>(USD_PREVIEW_SURFACE_MAX_UV_SETS, NumUVs);
				// At least one UV set must exist.
				NumUVs = FMath::Max<int32>(1, NumUVs);

				// Make sure all Vertex instance have the correct number of UVs
				MeshDescriptionUVs.SetNumChannels(NumUVs);
			}

			TVertexInstanceAttributesRef<FVector3f> MeshDescriptionNormals = StaticMeshAttributes.GetVertexInstanceNormals();
			TVertexInstanceAttributesRef<FVector3f> MeshDescriptionVelocities = OutMeshDescription.VertexInstanceAttributes()
																					.GetAttributesRef<FVector3f>(
																						MeshAttribute::VertexInstance::Velocity
																					);

			using FPrimvarVertexInstanceType = TVariant<
				TVertexInstanceAttributesRef<bool>,
				TVertexInstanceAttributesRef<int32>,
				TVertexInstanceAttributesRef<float>,
				TVertexInstanceAttributesRef<FVector2f>,
				TVertexInstanceAttributesRef<FVector3f>,
				TVertexInstanceAttributesRef<FVector4f>>;

			TMap<FString, FPrimvarVertexInstanceType> MeshDescriptionPrimvar;

			for (const TPair<FString, FUsdMeshData::FPrimvar>& Pair : InMeshData.Primvars)
			{
				Visit(
					TOverloaded{
						[&MeshDescriptionPrimvar, &Pair, &OutMeshDescription](const pxr::VtArray<bool>&)
						{
							MeshDescriptionPrimvar.Emplace(
								Pair.Key,
								FPrimvarVertexInstanceType{
									TInPlaceType<TVertexInstanceAttributesRef<bool>>{},
									OutMeshDescription.VertexInstanceAttributes().GetAttributesRef<bool>(*(Pair.Key))
								}
							);
						},

						[&MeshDescriptionPrimvar, &Pair, &OutMeshDescription](const pxr::VtArray<int32>&)
						{
							MeshDescriptionPrimvar.Emplace(
								Pair.Key,
								FPrimvarVertexInstanceType{
									TInPlaceType<TVertexInstanceAttributesRef<int32>>{},
									OutMeshDescription.VertexInstanceAttributes().GetAttributesRef<int32>(*(Pair.Key))
								}
							);
						},

						[&MeshDescriptionPrimvar, &Pair, &OutMeshDescription](const pxr::VtArray<float>&)
						{
							MeshDescriptionPrimvar.Emplace(
								Pair.Key,
								FPrimvarVertexInstanceType{
									TInPlaceType<TVertexInstanceAttributesRef<float>>{},
									OutMeshDescription.VertexInstanceAttributes().GetAttributesRef<float>(*(Pair.Key))
								}
							);
						},

						[&MeshDescriptionPrimvar, &Pair, &OutMeshDescription](const pxr::VtArray<pxr::GfVec2f>&)
						{
							MeshDescriptionPrimvar.Emplace(
								Pair.Key,
								FPrimvarVertexInstanceType{
									TInPlaceType<TVertexInstanceAttributesRef<FVector2f>>{},
									OutMeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector2f>(*(Pair.Key))
								}
							);
						},

						[&MeshDescriptionPrimvar, &Pair, &OutMeshDescription](const pxr::VtArray<pxr::GfVec3f>&)
						{
							MeshDescriptionPrimvar.Emplace(
								Pair.Key,
								FPrimvarVertexInstanceType{
									TInPlaceType<TVertexInstanceAttributesRef<FVector3f>>{},
									OutMeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector3f>(*(Pair.Key))
								}
							);
						},

						[&MeshDescriptionPrimvar, &Pair, &OutMeshDescription](const pxr::VtArray<pxr::GfVec4f>&)
						{
							MeshDescriptionPrimvar.Emplace(
								Pair.Key,
								FPrimvarVertexInstanceType{
									TInPlaceType<TVertexInstanceAttributesRef<FVector4f>>{},
									OutMeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector4f>(*(Pair.Key))
								}
							);
						}
					},
					Pair.Value.Data
				);
			}

			OutMeshDescription.ReserveNewVertexInstances(InMeshData.FaceVertexCounts.size() * 3);
			OutMeshDescription.ReserveNewPolygons(InMeshData.FaceVertexCounts.size());
			OutMeshDescription.ReserveNewEdges(InMeshData.FaceVertexCounts.size() * 2);

			// Vertex color
			TVertexInstanceAttributesRef<FVector4f> MeshDescriptionColors = StaticMeshAttributes.GetVertexInstanceColors();

			for (size_t PolygonIndex = 0; PolygonIndex < InMeshData.FaceVertexCounts.size(); ++PolygonIndex)
			{
				int32 PolygonVertexCount = InMeshData.FaceVertexCounts.cdata()[PolygonIndex];
				CornerInstanceIDs.Reset(PolygonVertexCount);
				CornerVerticesIDs.Reset(PolygonVertexCount);

				for (int32 CornerIndex = 0; CornerIndex < PolygonVertexCount; ++CornerIndex, ++CurrentVertexInstanceIndex)
				{
					int32 VertexInstanceIndex = VertexInstanceOffset + CurrentVertexInstanceIndex;
					const FVertexInstanceID VertexInstanceID(VertexInstanceIndex);
					const int32 ControlPointIndex = InMeshData.FaceVertexIndices.cdata()[CurrentVertexInstanceIndex];
					const FVertexID VertexID(VertexOffset + ControlPointIndex);

					// This data is read straight from USD so there's nothing guaranteeing we have as many positions as we need
					if (VertexID.GetValue() >= MeshDescriptionVertexPositions.GetNumElements() || VertexID.GetValue() < 0)
					{
						continue;
					}

					// Make sure a face doesn't use the same vertex twice as MeshDescription doesn't like that
					if (CornerVerticesIDs.Contains(VertexID))
					{
						continue;
					}

					CornerVerticesIDs.Add(VertexID);

					FVertexInstanceID AddedVertexInstanceId = OutMeshDescription.CreateVertexInstance(VertexID);
					CornerInstanceIDs.Add(AddedVertexInstanceId);

					if (InMeshData.Normals.size() > 0)
					{
						const size_t NormalIndex = GetPrimValueIndex(
							InMeshData.NormalInterpolation,
							ControlPointIndex,
							CurrentVertexInstanceIndex,
							PolygonIndex
						);

						if (NormalIndex < InMeshData.Normals.size())
						{
							const pxr::GfVec3f& Normal = InMeshData.Normals.cdata()[NormalIndex];
							FVector TransformedNormal = InOptions.AdditionalTransform.TransformVector(UsdToUnreal::ConvertVector(InStageInfo, Normal))
															.GetSafeNormal();

							MeshDescriptionNormals[AddedVertexInstanceId] = (FVector3f)TransformedNormal.GetSafeNormal();
						}
					}

					if (InMeshData.Velocities.size() > 0)
					{
						const size_t VelocityIndex = GetPrimValueIndex(
							InMeshData.VelocityInterpolation,
							ControlPointIndex,
							CurrentVertexInstanceIndex,
							PolygonIndex
						);

						if (VelocityIndex < InMeshData.Velocities.size())
						{
							const pxr::GfVec3f& Velocity = InMeshData.Velocities.cdata()[VelocityIndex];
							FVector TransformedVelocity = InOptions.AdditionalTransform.TransformVector(
								UsdToUnreal::ConvertVector(InStageInfo, Velocity)
							);

							MeshDescriptionVelocities[AddedVertexInstanceId] = (FVector3f)TransformedVelocity;
						}
					}

					for (int32 UVSetIndex = 0; UVSetIndex < UVSets.Num(); ++UVSetIndex)
					{
						const FUVSet& UVSet = UVSets[UVSetIndex];

						const size_t ValueIndex = GetPrimValueIndex(UVSet.InterpType, ControlPointIndex, CurrentVertexInstanceIndex, PolygonIndex);

						pxr::GfVec2f UV(0.f, 0.f);

						if (UVSet.UVs.size() > ValueIndex)
						{
							UV = UVSet.UVs[ValueIndex];
						}
						else
						{
							USD_LOG_USERWARNING(FText::Format(
								LOCTEXT(
									"WrongUVIndex",
									"Trying to read UV at index {0} from prim '{1}' but the UV set {2} only has {3} values! Using zeros instead."
								),
								static_cast<uint64>(ValueIndex),
								FText::FromString(InMeshData.SourcePrimPath),
								UVSetIndex,
								static_cast<uint64>(UVSet.UVs.size())
							));
						}

						// Flip V for Unreal uv's which match directx
						FVector2f FinalUVVector(UV[0], 1.f - UV[1]);
						MeshDescriptionUVs.Set(AddedVertexInstanceId, UVSet.UVSetIndexUE, FinalUVVector);
					}

					// Vertex color
					{
						const size_t ValueIndex = GetPrimValueIndex(
							InMeshData.DisplayColorInterpolation,
							ControlPointIndex,
							CurrentVertexInstanceIndex,
							PolygonIndex
						);

						pxr::GfVec3f UsdColor(1.f, 1.f, 1.f);

						if (!InMeshData.DisplayColors.empty())
						{
							if (InMeshData.DisplayColors.size() > ValueIndex)
							{
								UsdColor = InMeshData.DisplayColors.cdata()[ValueIndex];
							}
							else
							{
								USD_LOG_USERWARNING(FText::Format(
									LOCTEXT(
										"WrongDisplayColorIndex",
										"Trying to read displayColor at index {0} from prim '{1}' but the prim only has {2} values! Using zeros instead."
									),
									static_cast<uint64>(ValueIndex),
									FText::FromString(InMeshData.SourcePrimPath),
									static_cast<uint64>(InMeshData.DisplayColors.size())
								));
							}
						}

						MeshDescriptionColors[AddedVertexInstanceId] = UsdToUnreal::ConvertColor(UsdColor);
					}

					// Vertex opacity
					{
						const size_t ValueIndex = GetPrimValueIndex(
							InMeshData.DisplayOpacityInterpolation,
							ControlPointIndex,
							CurrentVertexInstanceIndex,
							PolygonIndex
						);

						if (!InMeshData.DisplayOpacities.empty())
						{
							if (InMeshData.DisplayOpacities.size() > ValueIndex)
							{
								MeshDescriptionColors[AddedVertexInstanceId][3] = InMeshData.DisplayOpacities.cdata()[ValueIndex];
							}
							else
							{
								USD_LOG_USERWARNING(FText::Format(
									LOCTEXT(
										"WrongDisplayOpacityIndex",
										"Trying to read displayOpacity at index {0} from prim '{1}' but the prim only has {2} values! Using zeros instead."
									),
									static_cast<uint64>(ValueIndex),
									FText::FromString(InMeshData.SourcePrimPath),
									static_cast<uint64>(InMeshData.DisplayColors.size())
								));
							}
						}
					}

					// Primvars
					for (const TPair<FString, FUsdMeshData::FPrimvar>& Pair : InMeshData.Primvars)
					{
						const FString& PrimvarName = Pair.Key;
						const FUsdMeshData::FPrimvar& CustomPrimvar = Pair.Value;
						const size_t
							ValueIndex = GetPrimValueIndex(CustomPrimvar.Interpolation, ControlPointIndex, CurrentVertexInstanceIndex, PolygonIndex);

						// We visit InMeshData.Primvar.Data for each type and make sure to fill the correct TVertexInstanceAttributesRef with the
						// right type of the primvar
						Visit(
							[ValueIndex, AddedVertexInstanceId, &PrimvarName, &MeshDescriptionPrimvar, &InMeshData](auto&& Value)
							{
								if (!Value.empty())
								{
									if (Value.size() > ValueIndex)
									{
										// underlying type of the pxr::VtArray that the primvar holds
										using FPrimvarType = std::decay_t<decltype(Value)>::ElementType;

										FPrimvarVertexInstanceType& PrimvarVertexInstance = MeshDescriptionPrimvar.FindChecked(PrimvarName);

										if constexpr (std::is_same_v<FPrimvarType, pxr::GfVec2f>)
										{
											TVertexInstanceAttributesRef<FVector2f>&
												VertexInstance = PrimvarVertexInstance.Get<TVertexInstanceAttributesRef<FVector2f>>();
											VertexInstance[AddedVertexInstanceId] = FVector2f{Value[ValueIndex][0], Value[ValueIndex][1]};
										}
										else if constexpr (std::is_same_v<FPrimvarType, pxr::GfVec3f>)
										{
											TVertexInstanceAttributesRef<FVector3f>&
												VertexInstance = PrimvarVertexInstance.Get<TVertexInstanceAttributesRef<FVector3f>>();
											VertexInstance[AddedVertexInstanceId] = FVector3f{
												Value[ValueIndex][0],
												Value[ValueIndex][1],
												Value[ValueIndex][2]
											};
										}
										else if constexpr (std::is_same_v<FPrimvarType, pxr::GfVec4f>)
										{
											TVertexInstanceAttributesRef<FVector4f>&
												VertexInstance = PrimvarVertexInstance.Get<TVertexInstanceAttributesRef<FVector4f>>();
											VertexInstance[AddedVertexInstanceId] = FVector4f{
												Value[ValueIndex][0],
												Value[ValueIndex][1],
												Value[ValueIndex][2],
												Value[ValueIndex][3]
											};
										}
										else if constexpr (std::is_arithmetic_v<FPrimvarType>)
										{
											TVertexInstanceAttributesRef<FPrimvarType>&
												VertexInstance = PrimvarVertexInstance.Get<TVertexInstanceAttributesRef<FPrimvarType>>();
											VertexInstance[AddedVertexInstanceId] = Value[ValueIndex];
										}
									}
									else
									{
										USD_LOG_USERWARNING(FText::Format(
											LOCTEXT(
												"WrongPrimvarIndex",
												"Trying to read Primvar '{0}' at index {1} from prim '{2}' but the prim only has {3} values! Using zeros instead."
											),
											FText::FromString(PrimvarName),
											static_cast<uint64>(ValueIndex),
											FText::FromString(InMeshData.SourcePrimPath),
											static_cast<uint64>(InMeshData.DisplayColors.size())
										));
									}
								}
							},
							CustomPrimvar.Data
						);
					}
				}

				// This polygon was using the same vertex instance more than once and we removed too many
				// vertex indices, so now we're forced to skip the whole polygon. We'll show a warning about it though
				if (CornerVerticesIDs.Num() < 3)
				{
					++NumSkippedPolygons;
					continue;
				}

				// Polygon groups
				int32 LocalMaterialIndex = 0;
				if (FaceMaterialIndices.IsValidIndex(PolygonIndex))
				{
					LocalMaterialIndex = FaceMaterialIndices[PolygonIndex];
					if (!LocalMaterialSlots.IsValidIndex(LocalMaterialIndex))
					{
						LocalMaterialIndex = 0;
					}
				}

				const int32 CombinedMaterialIndex = LocalToCombinedMaterialSlotIndices[LocalMaterialIndex];

				// Flip geometry if needed
				// If we're applying subdivision, pxOsd will take care of
				// converting the orientation from leftHanded to rightHanded,
				// so only flip if we are NOT subdividing.
				if (InMeshData.Orientation == pxr::UsdGeomTokens->leftHanded && InMeshData.SubdivisionLevelOverride.Get(InOptions.SubdivisionLevel) < 1)
				{
					for (int32 i = 0; i < CornerInstanceIDs.Num() / 2; ++i)
					{
						Swap(CornerInstanceIDs[i], CornerInstanceIDs[CornerInstanceIDs.Num() - i - 1]);
					}
				}

				// Insert a polygon into the mesh
				FPolygonGroupID PolygonGroupID = PolygonGroupMapping[CombinedMaterialIndex];
				OutMeshDescription.CreatePolygon(PolygonGroupID, CornerInstanceIDs);
			}
		}

		// Handle skinned mesh data
		if (InMeshData.JointIndices.size() > 0)
		{
			// Collect our joint influences in an array of structs
			TArray<FUsdRawBoneInfluence> Influences;
			if (InMeshData.JointWeights.size() > (NumPoints - 1) * (InMeshData.NumInfluencesPerComponent - 1))
			{
				uint32 JointIndex = 0;
				Influences.Reserve(NumPoints);
				for (uint32 PointIndex = 0; PointIndex < NumPoints; ++PointIndex)
				{
					// The JointIndices/JointWeights contain the influences data for NumPoints * NumInfluencesPerComponent
					for (uint32 InfluenceIndex = 0; InfluenceIndex < InMeshData.NumInfluencesPerComponent; ++InfluenceIndex, ++JointIndex)
					{
						// BoneWeight could be 0 if the actual number of influences were less than NumInfluencesPerComponent for a given point so
						// just ignore it
						float BoneWeight = InMeshData.JointWeights[JointIndex];
						if (BoneWeight != 0.f)
						{
							Influences.AddUninitialized();
							Influences.Last().BoneIndex = InMeshData.JointIndices[JointIndex];
							Influences.Last().Weight = BoneWeight;
							Influences.Last().VertexIndex = PointIndex;
						}
					}
				}
			}

			FSkeletalMeshAttributes MeshAttributes(OutMeshDescription);
			MeshAttributes.Register(bKeepExistingAttribute);

			FSkinWeightsVertexAttributesRef VertexSkinWeights = MeshAttributes.GetVertexSkinWeights();

			USDSkeletalDataConversion::CopySkinWeightsToAttribute(Influences, VertexIDMap, VertexSkinWeights);
		}

		if (NumPolygons > 0 && NumSkippedPolygons > 0)
		{
			USD_LOG_WARNING(
				TEXT("Skipped %d out of %d faces when parsing the mesh for prim '%s', as those faces contained too many repeated vertex indices"),
				NumSkippedPolygons,
				NumPolygons,
				*InMeshData.SourcePrimPath
			);
		}

		// Triangulate quads and ngons in order to avoid bad recomputed normals (UE-330573) 
		if (GTriangulateQuadsAndNgons)
		{
			TArray<FPolygonID> NonTrianglesToDelete;
			for (const FPolygonID PolygonID : OutMeshDescription.Polygons().GetElementIDs())
			{
				TArrayView<const FTriangleID> TriangleIDs = OutMeshDescription.GetPolygonTriangles(PolygonID);
				if (TriangleIDs.Num() < 2)
				{
					continue;
				}

				const FPolygonGroupID PolygonGroupID = OutMeshDescription.GetPolygonPolygonGroup(PolygonID);
				for (const FTriangleID TriangleID : TriangleIDs)
				{
					FVertexInstanceID VertexInstanceIDs[] { 
						OutMeshDescription.GetTriangleVertexInstance(TriangleID, 0),
						OutMeshDescription.GetTriangleVertexInstance(TriangleID, 1), 
						OutMeshDescription.GetTriangleVertexInstance(TriangleID, 2) 
					};
				
					OutMeshDescription.CreateTriangle(PolygonGroupID, VertexInstanceIDs);
				}

				NonTrianglesToDelete.Add(PolygonID);
			}

			for (FPolygonID PolygonID : NonTrianglesToDelete)
			{
				OutMeshDescription.DeletePolygon(PolygonID);
			}
		}
		
		return true;
	}
}	 // namespace UE::UsdGeomMeshConversion::Private
namespace UsdGeomMeshImpl = UE::UsdGeomMeshConversion::Private;

namespace UsdToUnreal
{
	const FUsdMeshConversionOptions FUsdMeshConversionOptions::DefaultOptions;

	FUsdMeshConversionOptions::FUsdMeshConversionOptions()
		: AdditionalTransform(FTransform::Identity)
		, PurposesToLoad(EUsdPurpose::Render)
		, RenderContext(pxr::UsdShadeTokens->universalRenderContext)
		, MaterialPurpose(pxr::UsdShadeTokens->allPurpose)
		, TimeCode(pxr::UsdTimeCode::EarliestTime())
		, bMergeIdenticalMaterialSlots(true)
		, SubdivisionLevel(0)
		, ImportPrimvars(EImportPrimvar::Standard)
		, bOffsetJointIndicesForMultipleRootBones(true)
		, bForceCheckUnrealMaterialAttribute(false)
		, bRequireMaterialsHaveProvidedRenderContext(true)
	{
	}
}

bool UsdToUnreal::ConvertGeomMesh(
	const pxr::UsdGeomMesh& UsdMesh,
	FMeshDescription& OutMeshDescription,
	UsdUtils::FUsdPrimMaterialAssignmentInfo& OutMaterialAssignments,
	const FUsdMeshConversionOptions& Options,
	const pxr::UsdSkelSkinningQuery* SkinningQuery,
	const pxr::UsdSkelSkeletonQuery* SkeletonQuery
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UsdToUnreal::ConvertGeomMesh);

	if (!UsdMesh)
	{
		return false;
	}

	FScopedUsdAllocs Allocs;

	pxr::UsdPrim UsdPrim = UsdMesh.GetPrim();

	UsdGeomMeshImpl::FUsdMeshData MeshData;

	UsdGeomMeshImpl::CollectMeshData(UsdPrim, Options, MeshData, OutMaterialAssignments);

	UsdGeomMeshImpl::ProcessSkinnedMeshData(UsdPrim, Options, MeshData, SkinningQuery, SkeletonQuery);

	UsdGeomMeshImpl::SubdivideMeshData(Options, MeshData);

	// Make sure primvars are flattened before calling ConvertMeshData.
	// We keep faceVarying indexed primvars within CollectMeshData as they are used for subdiv,
	// and SubdivideMeshData will flatten them after subdivision.
	// If we're not subdividing though we may still have some of these indexed primvars around,
	// and ConvertMeshData can't handle them
	UsdGeomMeshImpl::FlattenIndexedPrimvars(MeshData);

	pxr::UsdStageRefPtr Stage = UsdPrim.GetStage();
	const FUsdStageInfo StageInfo(Stage);
	return UsdGeomMeshImpl::ConvertMeshData(MeshData, StageInfo, Options, OutMeshDescription, OutMaterialAssignments);
}

bool UsdToUnreal::ConvertGeomMesh(
	const pxr::UsdPrim& MeshPrim,
	FMeshDescription& InOutMeshDescription,
	UsdUtils::FUsdPrimMaterialAssignmentInfo& InOutMaterialAssignments,
	const FUsdMeshConversionOptions& CommonOptions,
	const pxr::UsdSkelSkinningQuery* SkinningQuery,
	const pxr::UsdSkelSkeletonQuery* SkeletonQuery
)
{
	pxr::UsdGeomMesh GeomMesh{MeshPrim};
	if (!GeomMesh)
	{
		return false;
	}

	return ConvertGeomMesh(GeomMesh, InOutMeshDescription, InOutMaterialAssignments, CommonOptions, SkinningQuery, SkeletonQuery);
}

bool UsdToUnreal::ConvertPointInstancerToMesh(
	const pxr::UsdGeomPointInstancer& PointInstancer,
	FMeshDescription& OutMeshDescription,
	UsdUtils::FUsdPrimMaterialAssignmentInfo& OutMaterialAssignments,
	const FUsdMeshConversionOptions& Options
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UsdToUnreal::ConvertPointInstancerToMesh);

	if (!PointInstancer)
	{
		return false;
	}

	// Bake each prototype to a single mesh description and material assignment struct
	TArray<FMeshDescription> PrototypeMeshDescriptions;
	TArray<UsdUtils::FUsdPrimMaterialAssignmentInfo> PrototypeMaterialAssignments;
	TArray<TMap<FPolygonGroupID, FPolygonGroupID>> PrototypePolygonGroupRemapping;
	uint32 NumPrototypes = 0;
	UE::FUsdStage Stage;
	{
		TArray<UE::FSdfPath> PrototypePaths;
		{
			Stage = UE::FUsdStage{PointInstancer.GetPrim().GetStage()};
			if (!Stage)
			{
				return false;
			}

			TOptional<FScopedUsdAllocs> Allocs;
			Allocs.Emplace();

			const pxr::UsdRelationship& Prototypes = PointInstancer.GetPrototypesRel();

			pxr::SdfPathVector UsdPrototypePaths;
			if (!Prototypes.GetTargets(&UsdPrototypePaths))
			{
				return false;
			}

			NumPrototypes = UsdPrototypePaths.size();
			if (NumPrototypes == 0)
			{
				return true;
			}

			Allocs.Reset();
			PrototypePaths.Reserve(NumPrototypes);
			for (const pxr::SdfPath& UsdPath : UsdPrototypePaths)
			{
				PrototypePaths.Add(UE::FSdfPath{UsdPath});
			}
			Allocs.Emplace();
		}

		PrototypeMeshDescriptions.SetNum(NumPrototypes);
		PrototypeMaterialAssignments.SetNum(NumPrototypes);
		PrototypePolygonGroupRemapping.SetNum(NumPrototypes);

		// Our AdditionalTransform should be applied after even the instance transforms, we don't want to apply it
		// directly to our prototypes
		FUsdMeshConversionOptions OptionsCopy = Options;
		OptionsCopy.AdditionalTransform = FTransform::Identity;

		for (uint32 PrototypeIndex = 0; PrototypeIndex < NumPrototypes; ++PrototypeIndex)
		{
			const UE::FSdfPath& PrototypePath = PrototypePaths[PrototypeIndex];

			UE::FUsdPrim PrototypeUsdPrim = Stage.GetPrimAtPath(PrototypePath);
			if (!PrototypeUsdPrim)
			{
				USD_LOG_WARNING(
					TEXT("Failed to find prototype '%s' for PointInstancer '%s' within ConvertPointInstancerToMesh"),
					*UsdToUnreal::ConvertPath(PrototypePath),
					*UsdToUnreal::ConvertPath(PointInstancer.GetPrim().GetPrimPath())
				);
				continue;
			}

			const bool bSkipRootPrimTransformAndVisibility = false;
			ConvertGeomMeshSubtree(
				PrototypeUsdPrim,
				PrototypeMeshDescriptions[PrototypeIndex],
				PrototypeMaterialAssignments[PrototypeIndex],
				OptionsCopy,
				bSkipRootPrimTransformAndVisibility,
				bSkipRootPrimTransformAndVisibility
			);
		}
	}

	// Handle combined prototype material slots.
	// Sets up PrototypePolygonGroupRemapping so that our new faces are remapped from the prototype's mesh description polygon groups to the
	// combined mesh description's polygon groups when AppendMeshDescription is called.
	// Note: We always setup our mesh description polygon groups in the same order as the material assignment slots, so this is not so complicated
	for (uint32 PrototypeIndex = 0; PrototypeIndex < NumPrototypes; ++PrototypeIndex)
	{
		UsdUtils::FUsdPrimMaterialAssignmentInfo& PrototypeMaterialAssignment = PrototypeMaterialAssignments[PrototypeIndex];
		TMap<FPolygonGroupID, FPolygonGroupID>& PrototypeToCombinedMeshPolygonGroupMap = PrototypePolygonGroupRemapping[PrototypeIndex];

		if (Options.bMergeIdenticalMaterialSlots)
		{
			// Build a map of our existing slots since we can hash the entire slot, and our incoming mesh may have an arbitrary number of new slots
			TMap<UsdUtils::FUsdPrimMaterialSlot, int32> CombinedMaterialSlotsToIndex;
			for (int32 Index = 0; Index < OutMaterialAssignments.Slots.Num(); ++Index)
			{
				const UsdUtils::FUsdPrimMaterialSlot& Slot = OutMaterialAssignments.Slots[Index];
				CombinedMaterialSlotsToIndex.Add(Slot, Index);
			}

			for (int32 PrototypeMaterialSlotIndex = 0; PrototypeMaterialSlotIndex < PrototypeMaterialAssignment.Slots.Num();
				 ++PrototypeMaterialSlotIndex)
			{
				const UsdUtils::FUsdPrimMaterialSlot& LocalSlot = PrototypeMaterialAssignment.Slots[PrototypeMaterialSlotIndex];
				if (int32* ExistingCombinedIndex = CombinedMaterialSlotsToIndex.Find(LocalSlot))
				{
					PrototypeToCombinedMeshPolygonGroupMap.Add(PrototypeMaterialSlotIndex, *ExistingCombinedIndex);
				}
				else
				{
					OutMaterialAssignments.Slots.Add(LocalSlot);
					PrototypeToCombinedMeshPolygonGroupMap.Add(PrototypeMaterialSlotIndex, OutMaterialAssignments.Slots.Num() - 1);
				}
			}
		}
		else
		{
			const int32 NumExistingMaterialSlots = OutMaterialAssignments.Slots.Num();
			OutMaterialAssignments.Slots.Append(PrototypeMaterialAssignment.Slots);

			for (int32 PrototypeMaterialSlotIndex = 0; PrototypeMaterialSlotIndex < PrototypeMaterialAssignment.Slots.Num();
				 ++PrototypeMaterialSlotIndex)
			{
				PrototypeToCombinedMeshPolygonGroupMap.Add(PrototypeMaterialSlotIndex, NumExistingMaterialSlots + PrototypeMaterialSlotIndex);
			}
		}
	}

	// Make sure we have the polygon groups we expect. Appending the mesh descriptions will not create new polygon groups if we're using a
	// PolygonGroupsDelegate, which we will
	const int32 NumExistingPolygonGroups = OutMeshDescription.PolygonGroups().Num();
	for (int32 NumMissingPolygonGroups = OutMaterialAssignments.Slots.Num() - NumExistingPolygonGroups; NumMissingPolygonGroups > 0;
		 --NumMissingPolygonGroups)
	{
		OutMeshDescription.CreatePolygonGroup();
	}

	// Double-check our target mesh description has the attributes we need
	FStaticMeshAttributes StaticMeshAttributes(OutMeshDescription);

	// Keep existing because we may be calling this in the context of collapsing, where our OutMeshDescription
	// already has some data
	const bool bKeepExistingAttribute = true;
	StaticMeshAttributes.Register(bKeepExistingAttribute);

	// Append mesh descriptions
	FUsdStageInfo StageInfo{PointInstancer.GetPrim().GetStage()};
	for (uint32 PrototypeIndex = 0; PrototypeIndex < NumPrototypes; ++PrototypeIndex)
	{
		const FMeshDescription& PrototypeMeshDescription = PrototypeMeshDescriptions[PrototypeIndex];

		// We may generate some empty meshes in case a prototype is invisible, for example
		if (PrototypeMeshDescription.IsEmpty())
		{
			continue;
		}

		TArray<FTransform> InstanceTransforms;
		bool bSuccess = UsdUtils::GetPointInstancerTransforms(StageInfo, PointInstancer, PrototypeIndex, Options.TimeCode, InstanceTransforms);
		if (!bSuccess)
		{
			USD_LOG_ERROR(
				TEXT("Failed to retrieve point instancer transforms for prototype index '%u' of point instancer '%s'"),
				PrototypeIndex,
				*UsdToUnreal::ConvertPath(PointInstancer.GetPrim().GetPrimPath())
			);

			continue;
		}

		const int32 NumInstances = InstanceTransforms.Num();

		OutMeshDescription.ReserveNewVertices(PrototypeMeshDescription.Vertices().Num() * NumInstances);
		OutMeshDescription.ReserveNewVertexInstances(PrototypeMeshDescription.VertexInstances().Num() * NumInstances);
		OutMeshDescription.ReserveNewEdges(PrototypeMeshDescription.Edges().Num() * NumInstances);
		OutMeshDescription.ReserveNewTriangles(PrototypeMeshDescription.Triangles().Num() * NumInstances);

		FStaticMeshOperations::FAppendSettings Settings;
		Settings.PolygonGroupsDelegate = FAppendPolygonGroupsDelegate::CreateLambda(
			[&PrototypePolygonGroupRemapping,
			 PrototypeIndex](const FMeshDescription& SourceMesh, FMeshDescription& TargetMesh, PolygonGroupMap& RemapPolygonGroups)
			{
				RemapPolygonGroups = PrototypePolygonGroupRemapping[PrototypeIndex];
			}
		);

		// TODO: Maybe we should make a new overload of AppendMeshDescriptions that can do this more efficiently, since all we need is to change the
		// transform repeatedly?
		for (const FTransform& Transform : InstanceTransforms)
		{
			Settings.MeshTransform = Transform * Options.AdditionalTransform;
			FStaticMeshOperations::AppendMeshDescription(PrototypeMeshDescription, OutMeshDescription, Settings);
		}
	}

	return true;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool UsdToUnreal::ConvertGeomMeshHierarchy(
	const pxr::UsdPrim& Prim,
	FMeshDescription& InOutMeshDescription,
	UsdUtils::FUsdPrimMaterialAssignmentInfo& InOutMaterialAssignments,
	const FUsdMeshConversionOptions& CommonOptions,
	bool bSkipRootPrimTransformAndVisibility
)
{
	return ConvertGeomMeshSubtree(
		Prim,
		InOutMeshDescription,
		InOutMaterialAssignments,
		CommonOptions,
		bSkipRootPrimTransformAndVisibility,
		bSkipRootPrimTransformAndVisibility
	);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool UsdToUnreal::ConvertGeomMeshSubtree(
	const pxr::UsdPrim& Prim,
	FMeshDescription& OutMeshDescription,
	UsdUtils::FUsdPrimMaterialAssignmentInfo& OutMaterialAssignments,
	const FUsdMeshConversionOptions& Options,
	bool bSkipRootPrimTransform,
	bool bSkipRootPrimVisibility
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UsdToUnreal::ConvertGeomMeshSubtree);

	if (!Prim)
	{
		return false;
	}

	FStaticMeshAttributes StaticMeshAttributes(OutMeshDescription);

	// Keep existing because we may be calling this in the context of collapsing, where our OutMeshDescription
	// already has some data
	const bool bKeepExistingAttribute = true;
	StaticMeshAttributes.Register(bKeepExistingAttribute);

	// Pass a copy down so that we can repeatedly overwrite the AdditionalTransform and still
	// provide the options object to ConvertGeomMesh and ConvertPointInstancerToMesh
	FUsdMeshConversionOptions OptionsCopy = Options;

	// Prepass to figure out the best primvars to use for the entire collapsed mesh UV sets
	if (OutMaterialAssignments.PrimvarToUVIndex.Num() == 0)
	{
		OutMaterialAssignments.PrimvarToUVIndex = UsdGeomMeshImpl::CollectSubtreePrimvars(Prim, Options, bSkipRootPrimVisibility);
	}

	const bool bIsInSkelRoot = static_cast<bool>(UsdUtils::GetClosestParentSkelRoot(Prim));
	return UsdGeomMeshImpl::RecursivelyCollapseChildMeshes(
		Prim,
		OutMeshDescription,
		OutMaterialAssignments,
		OptionsCopy,
		bSkipRootPrimTransform,
		bSkipRootPrimVisibility,
		bIsInSkelRoot
	);
}

bool UsdToUnreal::ConvertGeomPrimitive(
	const pxr::UsdPrim& InPrim,
	FMeshDescription& InOutMeshDescription,
	UsdUtils::FUsdPrimMaterialAssignmentInfo& InOutMaterialAssignments,
	const FUsdMeshConversionOptions& InOptions
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UsdToUnreal::ConvertGeomPrimitive);

	if (!InPrim)
	{
		return false;
	}

	FScopedUsdAllocs UsdAllocs;

	UsdGeomMeshImpl::FUsdMeshData MeshData;

	// Collect all attributes authored as usual
	UsdGeomMeshImpl::CollectMeshData(InPrim, InOptions, MeshData, InOutMaterialAssignments);

	// Generate primitive points and topology on-demand
	{
		// Remember that USD arrays are copy-on-write, so these are both "pointers", as long as we
		// don't try writing (or using non-const operator[]) from PrimitivePoints
		pxr::VtVec3fArray PrimitivePoints;
		const pxr::PxOsdMeshTopology* PrimitiveTopology = nullptr;
		TOptional<pxr::PxOsdMeshTopology> CreatedTopology;

		if (pxr::UsdGeomCapsule Capsule = pxr::UsdGeomCapsule{InPrim})
		{
			pxr::TfToken Axis = UsdUtils::GetUsdValue<pxr::TfToken>(Capsule.GetAxisAttr(), pxr::UsdTimeCode::Default());

			PrimitivePoints = pxr::UsdImagingGenerateCapsuleMeshPoints(
				UsdGeomMeshImpl::DefaultCapsuleMeshHeight,
				UsdGeomMeshImpl::DefaultCapsuleMeshRadius,
				Axis
			);
			PrimitiveTopology = &pxr::UsdImagingGetCapsuleMeshTopology();
		}
		else if (pxr::UsdGeomCapsule_1 Capsule_1 = pxr::UsdGeomCapsule_1{InPrim})
		{
			const double RadiusTop = UsdGeomMeshImpl::GetDefaultOrEarliest<double>(Capsule_1.GetRadiusTopAttr());
			const double RadiusBottom = UsdGeomMeshImpl::GetDefaultOrEarliest<double>(Capsule_1.GetRadiusBottomAttr());
			const double RadiusRatio = RadiusTop != 0.0f ? (RadiusBottom / RadiusTop) : 1.0f;
			const double ActualRadiusTop = UsdGeomMeshImpl::DefaultCapsuleMeshRadius;
			const double ActualRadiusBottom = ActualRadiusTop * RadiusRatio;

			pxr::TfToken Axis = UsdUtils::GetUsdValue<pxr::TfToken>(Capsule_1.GetAxisAttr(), pxr::UsdTimeCode::Default());

			CreatedTopology = pxr::GeomUtilCapsuleMeshGenerator::GenerateTopology(
				UsdGeomMeshImpl::DefaultCapsuleNumRadialSegments,
				UsdGeomMeshImpl::DefaultCapsuleNumCapAxialSegments
			);
			PrimitiveTopology = CreatedTopology.GetPtrOrNull();

			PrimitivePoints = UsdGeomMeshImpl::GetTwoRadiiCapsuleMeshPoints(
				Axis,
				ActualRadiusBottom,
				ActualRadiusTop,
				UsdGeomMeshImpl::DefaultCapsuleMeshHeight
			);
		}
		else if (pxr::UsdGeomCone Cone = pxr::UsdGeomCone{InPrim})
		{
			pxr::TfToken Axis = UsdUtils::GetUsdValue<pxr::TfToken>(Cone.GetAxisAttr(), pxr::UsdTimeCode::Default());

			PrimitivePoints = UsdGeomMeshImpl::GetUnitConeMeshPoints(Axis);
			PrimitiveTopology = &pxr::UsdImagingGetUnitConeMeshTopology();
		}
		else if (pxr::UsdGeomCube Cube = pxr::UsdGeomCube{InPrim})
		{
			PrimitivePoints = pxr::UsdImagingGetUnitCubeMeshPoints();
			PrimitiveTopology = &pxr::UsdImagingGetUnitCubeMeshTopology();
		}
		else if (pxr::UsdGeomCylinder Cylinder = pxr::UsdGeomCylinder{InPrim})
		{
			pxr::TfToken Axis = UsdUtils::GetUsdValue<pxr::TfToken>(Cylinder.GetAxisAttr(), pxr::UsdTimeCode::Default());

			PrimitivePoints = UsdGeomMeshImpl::GetUnitCylinderMeshPoints(Axis);
			PrimitiveTopology = &pxr::UsdImagingGetUnitCylinderMeshTopology();
		}
		else if (pxr::UsdGeomCylinder_1 Cylinder_1 = pxr::UsdGeomCylinder_1{InPrim})
		{
			const double RadiusTop = UsdGeomMeshImpl::GetDefaultOrEarliest<double>(Cylinder_1.GetRadiusTopAttr());
			const double RadiusBottom = UsdGeomMeshImpl::GetDefaultOrEarliest<double>(Cylinder_1.GetRadiusBottomAttr());
			const double RadiusRatio = RadiusTop != 0.0f ? (RadiusBottom / RadiusTop) : 1.0f;
			const double ActualRadiusTop = UsdGeomMeshImpl::DefaultCylinderMeshRadius;
			const double ActualRadiusBottom = ActualRadiusTop * RadiusRatio;

			pxr::TfToken Axis = UsdUtils::GetUsdValue<pxr::TfToken>(Cylinder_1.GetAxisAttr(), pxr::UsdTimeCode::Default());

			CreatedTopology = pxr::GeomUtilCylinderMeshGenerator::GenerateTopology(UsdGeomMeshImpl::DefaultCylinderNumRadialSegments);
			PrimitiveTopology = CreatedTopology.GetPtrOrNull();

			PrimitivePoints = UsdGeomMeshImpl::GetTwoRadiiCylinderMeshPoints(
				Axis,
				ActualRadiusBottom,
				ActualRadiusTop,
				UsdGeomMeshImpl::DefaultCylinderMeshHeight
			);
		}
		else if (pxr::UsdGeomPlane Plane = pxr::UsdGeomPlane{InPrim})
		{
			const double Width = 1.0f;
			const double Length = 1.0f;

			pxr::TfToken Axis = UsdUtils::GetUsdValue<pxr::TfToken>(Plane.GetAxisAttr(), pxr::UsdTimeCode::Default());

			PrimitivePoints = pxr::UsdImagingGeneratePlaneMeshPoints(Width, Length, Axis);
			PrimitiveTopology = &pxr::UsdImagingGetPlaneTopology();
		}
		else if (pxr::UsdGeomSphere Sphere = pxr::UsdGeomSphere{InPrim})
		{
			PrimitivePoints = pxr::UsdImagingGetUnitSphereMeshPoints();
			PrimitiveTopology = &pxr::UsdImagingGetUnitSphereMeshTopology();
		}

		if (!PrimitiveTopology || PrimitivePoints.empty())
		{
			return false;
		}

		MeshData.FaceVertexCounts = PrimitiveTopology->GetFaceVertexCounts();
		MeshData.FaceVertexIndices = PrimitiveTopology->GetFaceVertexIndices();
		MeshData.Points = PrimitivePoints;
		MeshData.PointInterpolation = pxr::UsdGeomTokens->vertex;
	}

	SubdivideMeshData(InOptions, MeshData);

	FlattenIndexedPrimvars(MeshData);

	pxr::UsdStageRefPtr Stage = InPrim.GetStage();
	const FUsdStageInfo StageInfo(Stage);
	return ConvertMeshData(MeshData, StageInfo, InOptions, InOutMeshDescription, InOutMaterialAssignments);
}

bool UsdToUnreal::ConvertGeomPrimitiveTransform(const pxr::UsdPrim& InPrim, const pxr::UsdTimeCode& InTimeCode, FTransform& OutTransform)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UsdToUnreal::ConvertGeomPrimitive);

	if (!InPrim || !InPrim.IsA<pxr::UsdGeomGprim>() || UsdUtils::GetAppliedDrawMode(InPrim) != EUsdDrawMode::Default)
	{
		return false;
	}

	FScopedUsdAllocs UsdAllocs;

	pxr::UsdStageRefPtr Stage = InPrim.GetStage();
	const FUsdStageInfo StageInfo(Stage);

	TFunction<pxr::GfMatrix4d(pxr::TfToken, double, double)> GetScalingTransform = [](pxr::TfToken Axis, double Longitudinal, double Transversal)
	{
		if (Axis == pxr::UsdGeomTokens->x)
		{
			return pxr::GfMatrix4d(Longitudinal, 0.0, 0.0, 0.0, 0.0, Transversal, 0.0, 0.0, 0.0, 0.0, Transversal, 0.0, 0.0, 0.0, 0.0, 1.0);
		}
		else if (Axis == pxr::UsdGeomTokens->y)
		{
			return pxr::GfMatrix4d(Transversal, 0.0, 0.0, 0.0, 0.0, Longitudinal, 0.0, 0.0, 0.0, 0.0, Transversal, 0.0, 0.0, 0.0, 0.0, 1.0);
		}
		else
		{
			return pxr::GfMatrix4d(Transversal, 0.0, 0.0, 0.0, 0.0, Transversal, 0.0, 0.0, 0.0, 0.0, Longitudinal, 0.0, 0.0, 0.0, 0.0, 1.0);
		}
	};

	if (InPrim.IsA<pxr::UsdGeomCapsule>() || InPrim.IsA<pxr::UsdGeomCapsule_1>())
	{
		pxr::UsdAttribute RadiusAttr;
		pxr::UsdAttribute HeightAttr;
		pxr::UsdAttribute AxisAttr;
		if (pxr::UsdGeomCapsule Capsule = pxr::UsdGeomCapsule{InPrim})
		{
			RadiusAttr = Capsule.GetRadiusAttr();
			HeightAttr = Capsule.GetHeightAttr();
			AxisAttr = Capsule.GetAxisAttr();
		}
		else if (pxr::UsdGeomCapsule_1 Capsule_1 = pxr::UsdGeomCapsule_1{InPrim})
		{
			RadiusAttr = Capsule_1.GetRadiusTopAttr();
			HeightAttr = Capsule_1.GetHeightAttr();
			AxisAttr = Capsule_1.GetAxisAttr();

			UsdGeomMeshImpl::WarnIfAnimated(Capsule_1.GetRadiusBottomAttr());
		}

		const double Radius = UsdUtils::GetUsdValue<double>(RadiusAttr, InTimeCode);
		const double Height = UsdUtils::GetUsdValue<double>(HeightAttr, InTimeCode);
		pxr::TfToken Axis = UsdGeomMeshImpl::GetAxisWithValidation(AxisAttr);

		// We use these "Scaling" factors instead of direct height/radius because we're assuming
		// we'll have generated this capsule mesh using ConvertGeomPrimitive, where we provide
		// UsdImagingGenerateCapsuleMeshPoints with DefaultCapsuleMeshHeight and
		// DefaultCapsuleMeshRadius. If our current height/radius match those, we need to create
		// an identity transform. If our height is twice as that, our axis direction needs to have
		// a scaling of 2.0, etc.
		// Also keep in mind that the capsule total height is (Radius + Height + Radius).
		double HeightScaling = (Height + 2.0f * Radius)
							   / (UsdGeomMeshImpl::DefaultCapsuleMeshHeight + 2.0f * UsdGeomMeshImpl::DefaultCapsuleMeshRadius);
		double RadiusScaling = (Radius / UsdGeomMeshImpl::DefaultCapsuleMeshRadius);
		pxr::GfMatrix4d PrimitiveTransform = GetScalingTransform(Axis, HeightScaling, RadiusScaling);

		OutTransform = UsdToUnreal::ConvertMatrix(StageInfo, PrimitiveTransform);
		return true;
	}
	else if (pxr::UsdGeomCone Cone = pxr::UsdGeomCone{InPrim})
	{
		const double Radius = UsdUtils::GetUsdValue<double>(Cone.GetRadiusAttr(), InTimeCode);
		const double Height = UsdUtils::GetUsdValue<double>(Cone.GetHeightAttr(), InTimeCode);
		pxr::TfToken Axis = UsdGeomMeshImpl::GetAxisWithValidation(Cone.GetAxisAttr());

		const double Diameter = 2.0 * Radius;
		pxr::GfMatrix4d PrimitiveTransform = GetScalingTransform(Axis, Height, Diameter);

		OutTransform = UsdToUnreal::ConvertMatrix(StageInfo, PrimitiveTransform);
		return true;
	}
	else if (pxr::UsdGeomCube Cube = pxr::UsdGeomCube{InPrim})
	{
		const double Size = UsdUtils::GetUsdValue<double>(Cube.GetSizeAttr(), InTimeCode);
		const pxr::GfMatrix4d UsdTransform = pxr::UsdImagingGenerateSphereOrCubeTransform(Size);

		OutTransform = UsdToUnreal::ConvertMatrix(StageInfo, UsdTransform);
		return true;
	}
	else if (InPrim.IsA<pxr::UsdGeomCylinder>() || InPrim.IsA<pxr::UsdGeomCylinder_1>())
	{
		pxr::UsdAttribute RadiusAttr;
		pxr::UsdAttribute HeightAttr;
		pxr::UsdAttribute AxisAttr;
		if (pxr::UsdGeomCylinder Cylinder = pxr::UsdGeomCylinder{InPrim})
		{
			RadiusAttr = Cylinder.GetRadiusAttr();
			HeightAttr = Cylinder.GetHeightAttr();
			AxisAttr = Cylinder.GetAxisAttr();
		}
		else if (pxr::UsdGeomCylinder_1 Cylinder_1 = pxr::UsdGeomCylinder_1{InPrim})
		{
			RadiusAttr = Cylinder_1.GetRadiusTopAttr();
			HeightAttr = Cylinder_1.GetHeightAttr();
			AxisAttr = Cylinder_1.GetAxisAttr();

			UsdGeomMeshImpl::WarnIfAnimated(Cylinder_1.GetRadiusBottomAttr());
		}

		const double Radius = UsdUtils::GetUsdValue<double>(RadiusAttr, InTimeCode);
		const double Height = UsdUtils::GetUsdValue<double>(HeightAttr, InTimeCode);
		pxr::TfToken Axis = UsdGeomMeshImpl::GetAxisWithValidation(AxisAttr);

		const double Diameter = 2.0 * Radius;
		pxr::GfMatrix4d PrimitiveTransform = GetScalingTransform(Axis, Height, Diameter);

		OutTransform = UsdToUnreal::ConvertMatrix(StageInfo, PrimitiveTransform);
		return true;
	}
	else if (pxr::UsdGeomPlane Plane = pxr::UsdGeomPlane{InPrim})
	{
		const double Width = UsdUtils::GetUsdValue<double>(Plane.GetWidthAttr(), InTimeCode);
		const double Length = UsdUtils::GetUsdValue<double>(Plane.GetLengthAttr(), InTimeCode);
		pxr::TfToken Axis = UsdGeomMeshImpl::GetAxisWithValidation(Plane.GetAxisAttr());

		OutTransform = FTransform::Identity;

		// Generate a scaling transform in USD coordinate system
		if (Axis == pxr::UsdGeomTokens->x)
		{
			OutTransform.SetScale3D(FVector{1.0f, Length, Width});
		}
		else if (Axis == pxr::UsdGeomTokens->y)
		{
			OutTransform.SetScale3D(FVector{Width, 1.0f, Length});
		}
		else if (Axis == pxr::UsdGeomTokens->z)
		{
			OutTransform.SetScale3D(FVector{Width, Length, 1.0f});
		}

		// Convert that transform to the UE coordinate system
		OutTransform = UsdUtils::ConvertAxes(StageInfo.UpAxis == EUsdUpAxis::ZAxis, OutTransform);
		return true;
	}
	else if (pxr::UsdGeomSphere Sphere = pxr::UsdGeomSphere{InPrim})
	{
		const double Radius = UsdUtils::GetUsdValue<double>(Sphere.GetRadiusAttr(), InTimeCode);
		const double Diameter = Radius * 2.0;
		const pxr::GfMatrix4d UsdTransform = pxr::UsdImagingGenerateSphereOrCubeTransform(Diameter);

		OutTransform = UsdToUnreal::ConvertMatrix(StageInfo, UsdTransform);
		return true;
	}

	return false;
}

UsdUtils::FUsdPrimMaterialAssignmentInfo UsdUtils::GetPrimMaterialAssignments(
	const pxr::UsdPrim& UsdPrim,
	const pxr::UsdTimeCode TimeCode,
	bool bProvideMaterialIndices,
	const pxr::TfToken& RenderContext,
	const pxr::TfToken& MaterialPurpose,
	bool bForceCheckUnrealMaterialAttribute,
	bool bRequireMaterialsHaveProvidedRenderContext
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UsdUtils::GetPrimMaterialAssignments);

	// We handle sections when calling this on the parent Mesh prims, not when calling on the sections themselves
	if (!UsdPrim || UsdPrim.IsA<pxr::UsdGeomSubset>())
	{
		return {};
	}

	FScopedUsdAllocs Allocs;

	auto FetchFirstUEMaterialFromAttribute = [](const pxr::UsdPrim& UsdPrim, const pxr::UsdTimeCode TimeCode) -> TOptional<FString>
	{
		FString ValidPackagePath;
		if (pxr::UsdAttribute MaterialAttribute = UsdPrim.GetAttribute(UnrealIdentifiers::MaterialAssignment))
		{
			std::string UEMaterial;
			if (MaterialAttribute.Get(&UEMaterial, TimeCode) && UEMaterial.size() > 0)
			{
				ValidPackagePath = UsdToUnreal::ConvertString(UEMaterial);
			}
		}

		if (!ValidPackagePath.IsEmpty())
		{
			// We can't TryLoad() or LoadObject<> this right now as we may be in an Async thread.
			// The FAssetData may not be ready yet however, in case we're loading a stage right when launching the
			// editor, so here we just settle for finding any valid object
			FSoftObjectPath SoftObjectPath{ValidPackagePath};
			if (SoftObjectPath.IsValid())
			{
				return ValidPackagePath;
			}

			USD_LOG_USERWARNING(FText::Format(
				LOCTEXT(
					"UnrealMaterialNotFound",
					"Could not find a valid material at path '{0}', targetted by prim '{1}'s unrealMaterial attribute. Material assignment will fallback to USD materials and display color data."
				),
				FText::FromString(ValidPackagePath),
				FText::FromString(UsdToUnreal::ConvertPath(UsdPrim.GetPath()))
			));
		}

		return {};
	};

	auto FetchMaterialByComputingBoundMaterial =
		[&MaterialPurpose, &RenderContext, bRequireMaterialsHaveProvidedRenderContext]
		(const pxr::UsdPrim& UsdPrim) -> TOptional<FString>
		{
			pxr::UsdShadeMaterialBindingAPI BindingAPI(UsdPrim);
			pxr::UsdShadeMaterial ShadeMaterial = BindingAPI.ComputeBoundMaterial(MaterialPurpose);
			if (!ShadeMaterial)
			{
				return {};
			}

			if (bRequireMaterialsHaveProvidedRenderContext)
			{
				pxr::UsdShadeShader SurfaceShader = ShadeMaterial.ComputeSurfaceSource(RenderContext);
				if (!SurfaceShader)
				{
					return {};
				}
			}

			pxr::UsdPrim ShadeMaterialPrim = ShadeMaterial.GetPrim();
			if (ShadeMaterialPrim)
			{
				const std::string ShadingEngineName = ShadeMaterialPrim.GetPrimPath().GetString();
				if (!ShadingEngineName.empty())
				{
					return UsdToUnreal::ConvertString(ShadingEngineName);
				}
			}

			return {};
		};

	FUsdPrimMaterialAssignmentInfo Result;

	uint64 NumFaces = 0;
	{
		pxr::VtArray<int> FaceVertexCounts = UsdGeomMeshImpl::GetFaceVertexCounts(UsdPrim, TimeCode);
		NumFaces = FaceVertexCounts.size();

		if (bProvideMaterialIndices)
		{
			// Note how we're defaulting to slot zero here, which is our "main assignment"
			Result.MaterialIndices.SetNumZeroed(NumFaces);
		}
	}

	bool bIsDoubleSided = false;
	if (pxr::UsdGeomMesh Mesh = pxr::UsdGeomMesh{UsdPrim})
	{
		if (pxr::UsdAttribute Attr = Mesh.GetDoubleSidedAttr())
		{
			pxr::VtValue AttrValue;
			if (Attr.Get(&AttrValue) && AttrValue.IsHolding<bool>())
			{
				bIsDoubleSided = AttrValue.UncheckedGet<bool>();
			}
		}
	}

	FString PrimPath = UsdToUnreal::ConvertPath(UsdPrim.GetPath());

	bool bNeedsMainAssignment = true;

	// Priority 0: GeomSubset partitions
	std::vector<pxr::UsdGeomSubset> GeomSubsets = pxr::UsdShadeMaterialBindingAPI(UsdPrim).GetMaterialBindSubsets();
	if (GeomSubsets.size() > 0)
	{
		for (uint32 GeomSubsetIndex = 0; GeomSubsetIndex < GeomSubsets.size(); ++GeomSubsetIndex)
		{
			const pxr::UsdGeomSubset& GeomSubset = GeomSubsets[GeomSubsetIndex];
			pxr::UsdPrim GeomSubsetPrim = GeomSubset.GetPrim();
			FString GeomSubsetPath = UsdToUnreal::ConvertPath(GeomSubsetPrim.GetPath());
			bool bHasAssignment = false;

			// Priority 0.1: Material is an unreal asset
			if (RenderContext == UnrealIdentifiers::Unreal)
			{
				// Priority 0.1.1: Partition has an unreal rendercontext material prim binding
				if (!bHasAssignment)
				{
					pxr::UsdShadeMaterialBindingAPI BindingAPI(GeomSubsetPrim);
					if (pxr::UsdShadeMaterial ShadeMaterial = BindingAPI.ComputeBoundMaterial(MaterialPurpose))
					{
						if (TOptional<FString> UnrealMaterial = UsdUtils::GetUnrealSurfaceOutput(ShadeMaterial.GetPrim()))
						{
							FUsdPrimMaterialSlot& Slot = Result.Slots.Emplace_GetRef();
							Slot.MaterialSource = UnrealMaterial.GetValue();
							Slot.AssignmentType = UsdUtils::EPrimAssignmentType::UnrealMaterial;
							Slot.bMeshIsDoubleSided = bIsDoubleSided;
							Slot.PrimPaths.Add(GeomSubsetPath);
							bHasAssignment = true;
						}
					}
				}
			}

			// Priority 0.1.2: Partitition has an unrealMaterial attribute directly on it
			if (!bHasAssignment)
			{
				if (bForceCheckUnrealMaterialAttribute || RenderContext == UnrealIdentifiers::Unreal)
				{
					if (TOptional<FString> UnrealMaterial = FetchFirstUEMaterialFromAttribute(GeomSubsetPrim, TimeCode))
					{
						FUsdPrimMaterialSlot& Slot = Result.Slots.Emplace_GetRef();
						Slot.MaterialSource = UnrealMaterial.GetValue();
						Slot.AssignmentType = UsdUtils::EPrimAssignmentType::UnrealMaterial;
						Slot.bMeshIsDoubleSided = bIsDoubleSided;
						Slot.PrimPaths.Add(GeomSubsetPath);
						bHasAssignment = true;
					}
				}
			}

			// Priority 0.2: computing bound material
			if (!bHasAssignment)
			{
				if (TOptional<FString> BoundMaterial = FetchMaterialByComputingBoundMaterial(GeomSubsetPrim))
				{
					FUsdPrimMaterialSlot& Slot = Result.Slots.Emplace_GetRef();
					Slot.MaterialSource = BoundMaterial.GetValue();
					Slot.AssignmentType = UsdUtils::EPrimAssignmentType::MaterialPrim;
					Slot.bMeshIsDoubleSided = bIsDoubleSided;
					Slot.PrimPaths.Add(GeomSubsetPath);
					bHasAssignment = true;
				}
			}

			// Priority 0.3: Create a section anyway so that we always get a slot for each geom subset.
			// We leave the assignment type cleared here, and will fill this in later with whatever we
			// extract as a "main" material assignment.
			// Note that we may have yet another "leftover" slot if our partition doesn't specify all faces,
			// and that will be separate to this slot
			if (!bHasAssignment)
			{
				FUsdPrimMaterialSlot& Slot = Result.Slots.Emplace_GetRef();
				Slot.PrimPaths.Add(GeomSubsetPath);
				bHasAssignment = true;
			}

			if (bProvideMaterialIndices)
			{
				pxr::VtIntArray PolygonIndicesInSubset;
				GeomSubset.GetIndicesAttr().Get(&PolygonIndicesInSubset, TimeCode);

				int32 LastAssignmentIndex = Result.Slots.Num() - 1;
				for (int PolygonIndex : PolygonIndicesInSubset)
				{
					// #ueent_todo: There can be issues with PolygonIndex being bigger that the number of faces with varying GeomSubsets
					if (Result.MaterialIndices.IsValidIndex(PolygonIndex))
					{
						Result.MaterialIndices[PolygonIndex] = LastAssignmentIndex;
					}
				}
			}
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(GetUnassignedFaces);

			// Check if we have any unassigned faces
			//
			// Just summing the indices like this is an approximation, because they may be invalid/repeated indices, etc.
			// Ideally we'd call pxr::UsdGeomSubset::GetUnassignedIndices right away here to know for sure if we need an additional slot.
			// That can be slow however, and this function is in the hot path of the info cache build, so given that invalid partitions
			// are invalid data in the first place, this approximation is hopefully a good enough compromise in order to not slow down the
			// general case too much
			uint64 SubsetIndexTotal = 0;
			for (const pxr::UsdGeomSubset& Subset : GeomSubsets)
			{
				pxr::VtIntArray PolygonIndicesInSubset;
				Subset.GetIndicesAttr().Get(&PolygonIndicesInSubset, TimeCode);

				SubsetIndexTotal += PolygonIndicesInSubset.size();
			}

			// Extra slot for unspecified faces.
			// We need to fetch this even if we won't provide indices because we may need to create an additional slot for unassigned polygons
			const bool bHasUnassignedIndices = SubsetIndexTotal != NumFaces;
			if (bHasUnassignedIndices)
			{
				USD_LOG_WARNING(TEXT("Found an invalid GeomSubsets partition in prim '%s'"), *UsdToUnreal::ConvertPath(UsdPrim.GetPath()));

				// Assign these leftover indices to the *next* material slot we'll create (doesn't exist yet),
				// which will be the "main" material assignment slot
				int32 LeftoverSlotIndex = Result.Slots.Num();

				if (bProvideMaterialIndices)
				{
					pxr::VtIntArray UnassignedIndices = pxr::UsdGeomSubset::GetUnassignedIndices(GeomSubsets, NumFaces);
					for (int PolygonIndex : UnassignedIndices)
					{
						Result.MaterialIndices[PolygonIndex] = LeftoverSlotIndex;
					}
				}
			}
			else
			{
				bNeedsMainAssignment = false;
			}
		}
	}

	TOptional<UsdUtils::FDisplayColorMaterial> DisplayColor;

	bool bHasMainAssignment = false;
	if (bNeedsMainAssignment)
	{
		// Priority 1: Material is an unreal asset
		if (RenderContext == UnrealIdentifiers::Unreal)
		{
			// Priority 1.1: unreal rendercontext material prim
			// Note how we don't test this BindingAPI for truthiness: This allows us to compute a bound material
			// even if this prim is just inheriting a material binding, but doesn't actually have the API itself
			pxr::UsdShadeMaterialBindingAPI BindingAPI{UsdPrim};
			if (pxr::UsdShadeMaterial ShadeMaterial = BindingAPI.ComputeBoundMaterial(MaterialPurpose))
			{
				if (TOptional<FString> UnrealMaterial = UsdUtils::GetUnrealSurfaceOutput(ShadeMaterial.GetPrim()))
				{
					FUsdPrimMaterialSlot& Slot = Result.Slots.Emplace_GetRef();
					Slot.MaterialSource = UnrealMaterial.GetValue();
					Slot.AssignmentType = UsdUtils::EPrimAssignmentType::UnrealMaterial;
					Slot.bMeshIsDoubleSided = bIsDoubleSided;
					Slot.PrimPaths.Add(PrimPath);

					bHasMainAssignment = true;
				}
			}
		}

		// Priority 1.2: unrealMaterial attribute directly on the prim
		if (!bHasMainAssignment)
		{
			if (bForceCheckUnrealMaterialAttribute || RenderContext == UnrealIdentifiers::Unreal)
			{
				if (TOptional<FString> UnrealMaterial = FetchFirstUEMaterialFromAttribute(UsdPrim, TimeCode))
				{
					FUsdPrimMaterialSlot& Slot = Result.Slots.Emplace_GetRef();
					Slot.MaterialSource = UnrealMaterial.GetValue();
					Slot.AssignmentType = UsdUtils::EPrimAssignmentType::UnrealMaterial;
					Slot.bMeshIsDoubleSided = bIsDoubleSided;
					Slot.PrimPaths.Add(PrimPath);

					bHasMainAssignment = true;
				}
			}
		}

		// Priority 2: material binding directly on the prim
		if (!bHasMainAssignment)
		{
			if (TOptional<FString> BoundMaterial = FetchMaterialByComputingBoundMaterial(UsdPrim))
			{
				FUsdPrimMaterialSlot& Slot = Result.Slots.Emplace_GetRef();
				Slot.MaterialSource = BoundMaterial.GetValue();
				Slot.AssignmentType = UsdUtils::EPrimAssignmentType::MaterialPrim;
				Slot.bMeshIsDoubleSided = bIsDoubleSided;
				Slot.PrimPaths.Add(PrimPath);

				bHasMainAssignment = true;
			}
		}

		// Priority 3: vertex color material using displayColor/displayOpacity information for the entire mesh
		// Note: This will in general always succeed for any mesh prim, as the schema will provide fallback values
		// for displayColor and displayOpacity
		if (!bHasMainAssignment)
		{
			DisplayColor = ExtractDisplayColorMaterial(pxr::UsdGeomGprim{UsdPrim}, TimeCode);
			if (DisplayColor)
			{
				FUsdPrimMaterialSlot& Slot = Result.Slots.Emplace_GetRef();
				Slot.MaterialSource = DisplayColor.GetValue().ToString();
				Slot.AssignmentType = UsdUtils::EPrimAssignmentType::DisplayColor;
				Slot.bMeshIsDoubleSided = bIsDoubleSided;
				Slot.PrimPaths.Add(PrimPath);

				bHasMainAssignment = true;
			}
		}
	}
	ensure(bHasMainAssignment || !bNeedsMainAssignment);

	// If we have any slot without an actual material assignment yet, copy over the material assignment from
	// the "main" slot, or fallback to displayColor. This is how we have unspecified faces or geomsubsets without
	// assignmets "fallback" to using the main material assignment
	if (Result.Slots.Num() >= 1)
	{
		FString FallbackMaterialSource;
		EPrimAssignmentType FallbackAssignmentType = EPrimAssignmentType::None;

		if (bHasMainAssignment)
		{
			// Our main slot is the last created one at this point
			FUsdPrimMaterialSlot& MainSlot = Result.Slots[Result.Slots.Num() - 1];
			FallbackAssignmentType = MainSlot.AssignmentType;
			FallbackMaterialSource = MainSlot.MaterialSource;
		}
		else
		{
			if (!DisplayColor.IsSet())
			{
				DisplayColor = ExtractDisplayColorMaterial(pxr::UsdGeomMesh(UsdPrim), TimeCode);
				if (ensure(DisplayColor.IsSet()))
				{
					FallbackAssignmentType = UsdUtils::EPrimAssignmentType::DisplayColor;
					FallbackMaterialSource = DisplayColor.GetValue().ToString();
				}
			}
		}

		for (int32 Index = 0; Index < Result.Slots.Num(); ++Index)
		{
			FUsdPrimMaterialSlot& Slot = Result.Slots[Index];
			if (Slot.AssignmentType == UsdUtils::EPrimAssignmentType::None)
			{
				Slot.AssignmentType = FallbackAssignmentType;
				Slot.MaterialSource = FallbackMaterialSource;
			}
		}
	}
	// Priority 5: Make sure there is always at least one slot, even if empty
	else if (Result.Slots.Num() < 1)
	{
		Result.Slots.Emplace();
	}

	return Result;
}

bool UnrealToUsd::ConvertStaticMesh(
	const UStaticMesh* StaticMesh,
	pxr::UsdPrim& UsdPrim,
	const pxr::UsdTimeCode TimeCode,
	UE::FUsdStage* StageForMaterialAssignments,
	int32 LowestMeshLOD,
	int32 HighestMeshLOD,
	bool bExportStaticMeshSourceData
)
{
	FScopedUsdAllocs UsdAllocs;

	pxr::UsdStageRefPtr Stage = UsdPrim.GetStage();
	if (!Stage)
	{
		return false;
	}

	const FUsdStageInfo StageInfo(Stage);

	int32 NumLODs = StaticMesh->GetNumLODs();
	if (NumLODs < 1)
	{
		return false;
	}

	// Make sure they're both >= 0 (the options dialog slider is clamped, but this may be called directly)
	LowestMeshLOD = FMath::Clamp(LowestMeshLOD, 0, NumLODs - 1);
	HighestMeshLOD = FMath::Clamp(HighestMeshLOD, 0, NumLODs - 1);

	// Make sure Lowest <= Highest
	int32 Temp = FMath::Min(LowestMeshLOD, HighestMeshLOD);
	HighestMeshLOD = FMath::Max(LowestMeshLOD, HighestMeshLOD);
	LowestMeshLOD = Temp;

	// Make sure it's at least 1 LOD level
	NumLODs = FMath::Max(HighestMeshLOD - LowestMeshLOD + 1, 1);

#if WITH_EDITOR
	const bool bNaniteEnabled = StaticMesh->IsNaniteEnabled();
#else
	const bool bNaniteEnabled = false;
#endif	  // WITH_EDITOR

	pxr::UsdVariantSets VariantSets = UsdPrim.GetVariantSets();
	pxr::UsdVariantSet VariantSet = VariantSets.GetVariantSet(UnrealIdentifiers::LOD);
	if (NumLODs > 1 && VariantSets.HasVariantSet(UnrealIdentifiers::LOD))
	{
		USD_LOG_ERROR(
			TEXT("Failed to export higher LODs for mesh '%s', as the target prim already has a variant set named '%s'!"),
			*StaticMesh->GetName(),
			*UsdToUnreal::ConvertToken(UnrealIdentifiers::LOD)
		);
		NumLODs = 1;
	}

	bool bExportMultipleLODs = NumLODs > 1;

	pxr::SdfPath ParentPrimPath = UsdPrim.GetPath();
	std::string LowestLODAdded = "";

	// Collect all material assignments, referenced by the sections' material indices
	bool bHasMaterialAssignments = false;
	TArray<FString> MaterialIndexToContentPath;			   // This one is used when exporting static mesh render data (we retain the order)
	TMap<FName, int32> ImportedMaterialSlotNameToIndex;	   // This is used when exporting FMeshDescriptions (we use the imported slot names)
	const TArray<FStaticMaterial>& StaticMaterials = StaticMesh->GetStaticMaterials();
	for (int32 MaterialIndex = 0; MaterialIndex < StaticMaterials.Num(); ++MaterialIndex)
	{
		const FStaticMaterial& StaticMaterial = StaticMaterials[MaterialIndex];

		FString AssignedMaterialPathName;
		if (UMaterialInterface* Material = StaticMaterial.MaterialInterface)
		{
			if (Material->GetOutermost() != GetTransientPackage())
			{
				AssignedMaterialPathName = Material->GetPathName();
				bHasMaterialAssignments = true;
			}
		}

		MaterialIndexToContentPath.Add(AssignedMaterialPathName);
		ImportedMaterialSlotNameToIndex.Add(StaticMaterial.ImportedMaterialSlotName, MaterialIndex);
	}
	if (!bHasMaterialAssignments)
	{
		// Prevent creation of the UnrealMaterials prims in case we don't have any assignments at all
		MaterialIndexToContentPath.Reset();
		ImportedMaterialSlotNameToIndex.Reset();
	}
	// Author material bindings on the dedicated stage if we have one
	pxr::UsdStageRefPtr MaterialStage = StageForMaterialAssignments ? static_cast<pxr::UsdStageRefPtr>(*StageForMaterialAssignments) : Stage;

	// Do this outside the variant edit context or else it's going to be a weaker opinion than the stuff outside
	// the variant, and it won't really do anything for UsdPrim if it already exists.
	// Use an Xform for the parent prim because it will be our defaultPrim for this layer, and our referencer code will
	// try copying the schema of the defaultPrim onto the referencer prim to make sure they match. If we were typeless
	// here, so would our referencer and we wouldn't be able to put a transform on it
	UsdPrim = Stage->DefinePrim(UsdPrim.GetPath(), UnrealToUsd::ConvertToken(bExportMultipleLODs ? TEXT("Xform") : TEXT("Mesh")).Get());

	TUsdStore<pxr::VtArray<pxr::GfVec3f>> USDBounds = UnrealToUsd::ConvertBounds(StageInfo, StaticMesh->GetBoundingBox());

#if WITH_EDITOR
	const bool bExportNaniteDataAsSourceData = GExportNaniteSourceMeshData && StaticMesh->IsNaniteEnabled()
											   && StaticMesh->IsHiResMeshDescriptionValid();
#endif

	bool bExported = false;
	for (int32 LODIndex = LowestMeshLOD; LODIndex <= HighestMeshLOD; ++LODIndex)
	{
		// LOD0, LOD1, etc
		std::string VariantName = UnrealIdentifiers::LOD.GetString() + UnrealToUsd::ConvertString(*LexToString(LODIndex)).Get();
		if (LowestLODAdded.size() == 0)
		{
			LowestLODAdded = VariantName;
		}

		// Enable the variant edit context, if we are creating variant LODs
		TOptional<pxr::UsdEditContext> EditContext;
		if (bExportMultipleLODs)
		{
			if (!VariantSet.AddVariant(VariantName))
			{
				continue;
			}

			VariantSet.SetVariantSelection(VariantName);
			EditContext.Emplace(VariantSet.GetVariantEditContext());
		}

		pxr::SdfPath LODPrimPath = ParentPrimPath.AppendPath(pxr::SdfPath(VariantName));

		pxr::UsdGeomMesh TargetMesh;
		pxr::UsdPrim MaterialPrim = UsdPrim;
		if (bExportMultipleLODs)
		{
			// Add the mesh data to a child prim with the Mesh schema
			pxr::UsdPrim UsdLODPrim = Stage->DefinePrim(LODPrimPath, UnrealToUsd::ConvertToken(TEXT("Mesh")).Get());
			TargetMesh = pxr::UsdGeomMesh{UsdLODPrim};

			MaterialPrim = MaterialStage->OverridePrim(LODPrimPath);
		}
		else
		{
			TargetMesh = pxr::UsdGeomMesh{UsdPrim};

			MaterialPrim = MaterialStage->OverridePrim(UsdPrim.GetPath());
		}

		// Enable Nanite if we have it
		if (bNaniteEnabled)
		{
			if (pxr::UsdAttribute Attr = TargetMesh.GetPrim().CreateAttribute(UnrealIdentifiers::UnrealNaniteOverride, pxr::SdfValueTypeNames->Token))
			{
				Attr.Set(UnrealIdentifiers::UnrealNaniteOverrideEnable);
				UsdUtils::NotifyIfOverriddenOpinion(Attr);
			}
		}

		// Export extents
		if (USDBounds.Get().size() > 0)
		{
			if (pxr::UsdAttribute Attr = TargetMesh.CreateExtentAttr())
			{
				Attr.Set(USDBounds.Get());
			}
		}

#if WITH_EDITOR
		// Try exporting source data
		//  - Reference: FFbxExporter::ExportStaticMeshToFBX
		const bool bUseNaniteData = LODIndex == 0 && bExportNaniteDataAsSourceData && StaticMesh->IsNaniteEnabled()
									&& StaticMesh->IsHiResMeshDescriptionValid();
		const bool bHasSourceData = StaticMesh->IsMeshDescriptionValid(LODIndex);
		bool bExportSourceForLOD = bExportStaticMeshSourceData && (bUseNaniteData || bHasSourceData);

		if (bExportSourceForLOD)
		{
			FMatrix AdditionalTransform = FTransform::Identity.ToMatrixWithScale();
			if (bUseNaniteData)
			{
				if (const FMeshDescription* MeshDescription = StaticMesh->GetHiResMeshDescription())
				{
					bExported = UsdGeomMeshImpl::ConvertMeshDescription(
						*MeshDescription,
						TargetMesh,
						AdditionalTransform,
						TimeCode,
						&MaterialIndexToContentPath,
						&ImportedMaterialSlotNameToIndex,
						&MaterialPrim
					);
				}
			}
			else
			{
				if (const FMeshDescription* MeshDescription = StaticMesh->GetMeshDescription(LODIndex))
				{
					bExported = UsdGeomMeshImpl::ConvertMeshDescription(
						*MeshDescription,
						TargetMesh,
						AdditionalTransform,
						TimeCode,
						&MaterialIndexToContentPath,
						&ImportedMaterialSlotNameToIndex,
						&MaterialPrim
					);
				}
			}
		}
		else
#endif	  // WITH_EDITOR
		{
			// If we want to export the render data, get it and check its integrity
			const FStaticMeshLODResources* RenderMesh = &StaticMesh->GetLODForExport(LODIndex);
			if (!RenderMesh || RenderMesh->VertexBuffers.StaticMeshVertexBuffer.GetNumVertices() == 0 || RenderMesh->Sections.Num() == 0)
			{
				USD_LOG_WARNING(TEXT("Found invalid render data for LOD '%d' of '%s'!"), LODIndex, *StaticMesh->GetPathName());
				continue;
			}

			// Export render data LOD
			bExported = UsdGeomMeshImpl::ConvertStaticMeshLOD(*RenderMesh, TargetMesh, TimeCode, MaterialIndexToContentPath, MaterialPrim);
		}

		if (!bExported)
		{
			USD_LOG_WARNING(
				TEXT("Failed to export LOD '%d' of mesh '%s' onto prim '%s'!"),
				LODIndex,
				*StaticMesh->GetPathName(),
				*UsdToUnreal::ConvertPath(TargetMesh.GetPrim().GetPrimPath())
			);
			break;
		}
	}

	// Reset variant set to start with the lowest lod selected
	if (bExportMultipleLODs)
	{
		VariantSets.GetVariantSet(UnrealIdentifiers::LOD).SetVariantSelection(LowestLODAdded);
	}

	return bExported;
}

bool UnrealToUsd::ConvertMeshDescriptions(
	const TArray<FMeshDescription>& LODIndexToMeshDescription,
	pxr::UsdPrim& UsdPrim,
	const FMatrix& AdditionalTransform,
	const pxr::UsdTimeCode TimeCode
)
{
	FScopedUsdAllocs UsdAllocs;

	pxr::UsdStageRefPtr Stage = UsdPrim.GetStage();
	if (!Stage)
	{
		return false;
	}

	const FUsdStageInfo StageInfo(Stage);

	int32 NumLODs = LODIndexToMeshDescription.Num();
	if (NumLODs < 1)
	{
		return false;
	}

	pxr::UsdVariantSets VariantSets = UsdPrim.GetVariantSets();
	if (NumLODs > 1 && VariantSets.HasVariantSet(UnrealIdentifiers::LOD))
	{
		USD_LOG_ERROR(
			TEXT("Failed to convert higher mesh description LODs for prim '%s', as the target prim already has a variant set named '%s'!"),
			*UsdToUnreal::ConvertPath(UsdPrim.GetPath()),
			*UsdToUnreal::ConvertToken(UnrealIdentifiers::LOD)
		);
		NumLODs = 1;
	}

	bool bExportMultipleLODs = NumLODs > 1;

	pxr::SdfPath ParentPrimPath = UsdPrim.GetPath();
	std::string LowestLODAdded = "";

	// Check the comment on the analogous line on ConvertStaticMesh
	UsdPrim = Stage->DefinePrim(UsdPrim.GetPath(), UnrealToUsd::ConvertToken(bExportMultipleLODs ? TEXT("Xform") : TEXT("Mesh")).Get());

	for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
	{
		const FMeshDescription& MeshDescription = LODIndexToMeshDescription[LODIndex];

		// LOD0, LOD1, etc
		std::string VariantName = UnrealIdentifiers::LOD.GetString() + UnrealToUsd::ConvertString(*LexToString(LODIndex)).Get();
		if (LowestLODAdded.size() == 0)
		{
			LowestLODAdded = VariantName;
		}

		pxr::SdfPath LODPrimPath = ParentPrimPath.AppendPath(pxr::SdfPath(VariantName));

		// Enable the variant edit context, if we are creating variant LODs
		TOptional<pxr::UsdEditContext> EditContext;
		if (bExportMultipleLODs)
		{
			pxr::UsdVariantSet VariantSet = VariantSets.GetVariantSet(UnrealIdentifiers::LOD);
			if (!VariantSet.AddVariant(VariantName))
			{
				continue;
			}

			VariantSet.SetVariantSelection(VariantName);
			EditContext.Emplace(VariantSet.GetVariantEditContext());
		}

		pxr::UsdGeomMesh TargetMesh;
		if (bExportMultipleLODs)
		{
			// Add the mesh data to a child prim with the Mesh schema
			pxr::UsdPrim UsdLODPrim = Stage->DefinePrim(LODPrimPath, UnrealToUsd::ConvertToken(TEXT("Mesh")).Get());
			TargetMesh = pxr::UsdGeomMesh{UsdLODPrim};
		}
		else
		{
			TargetMesh = pxr::UsdGeomMesh{UsdPrim};
		}

		if (!UsdGeomMeshImpl::ConvertMeshDescription(MeshDescription, TargetMesh, AdditionalTransform, TimeCode))
		{
			return false;
		}
	}

	// Reset variant set to start with the lowest lod selected
	if (bExportMultipleLODs)
	{
		VariantSets.GetVariantSet(UnrealIdentifiers::LOD).SetVariantSelection(LowestLODAdded);
	}

	return true;
}

namespace UE::UsdGeometryCacheConversion::Private
{
	void AppendGeometryCacheMeshData(const FGeometryCacheMeshData& InMeshData, FGeometryCacheMeshData& InOutFlattenedMeshData)
	{
		// MeshData are flattened together by appending their data...
		const int32 VertexIndexOffset = InOutFlattenedMeshData.Positions.Num();
		const int32 IndicesIndexOffset = InOutFlattenedMeshData.Indices.Num();

		InOutFlattenedMeshData.Positions.Append(InMeshData.Positions);
		InOutFlattenedMeshData.TextureCoordinates.Append(InMeshData.TextureCoordinates);
		InOutFlattenedMeshData.TangentsX.Append(InMeshData.TangentsX);
		InOutFlattenedMeshData.TangentsZ.Append(InMeshData.TangentsZ);
		InOutFlattenedMeshData.Colors.Append(InMeshData.Colors);
		InOutFlattenedMeshData.MotionVectors.Append(InMeshData.MotionVectors);
		InOutFlattenedMeshData.ImportedVertexNumbers.Append(InMeshData.ImportedVertexNumbers);

		// ... and adjusting the indices with the proper offset
		InOutFlattenedMeshData.Indices.Reserve(InOutFlattenedMeshData.Indices.Num() + InMeshData.Indices.Num());
		Algo::Transform(
			InMeshData.Indices,
			InOutFlattenedMeshData.Indices,
			[VertexIndexOffset](uint32 Index)
			{
				return Index + VertexIndexOffset;
			}
		);

		// Same with the BatchInfo's StartIndex, which describes where each mesh section starts
		for (const FGeometryCacheMeshBatchInfo& BatchInfo : InMeshData.BatchesInfo)
		{
			FGeometryCacheMeshBatchInfo AdjustedBatchInfo(BatchInfo);
			AdjustedBatchInfo.StartIndex += IndicesIndexOffset;
			InOutFlattenedMeshData.BatchesInfo.Add(AdjustedBatchInfo);
		}

		// Also merge the VertexInfo attributes that are checked when converting the MeshData
		InOutFlattenedMeshData.VertexInfo.bHasTangentX |= InMeshData.VertexInfo.bHasTangentX;
		InOutFlattenedMeshData.VertexInfo.bHasTangentZ |= InMeshData.VertexInfo.bHasTangentZ;
		InOutFlattenedMeshData.VertexInfo.bHasUV0 |= InMeshData.VertexInfo.bHasUV0;
		InOutFlattenedMeshData.VertexInfo.bHasColor0 |= InMeshData.VertexInfo.bHasColor0;
		InOutFlattenedMeshData.VertexInfo.bHasMotionVectors |= InMeshData.VertexInfo.bHasMotionVectors;
		InOutFlattenedMeshData.VertexInfo.bHasImportedVertexNumbers |= InMeshData.VertexInfo.bHasImportedVertexNumbers;
	}

	FGeometryCacheMeshData GetFlattenedGeometryCacheMeshData(const UGeometryCache* GeometryCache, int32 FrameIndex)
	{
		FGeometryCacheMeshData FlattenedMeshData;
		if (GeometryCache->Tracks.Num() == 1)
		{
			GeometryCache->Tracks[0]->GetMeshDataAtSampleIndex(FrameIndex, FlattenedMeshData);
		}
		else
		{
			// MeshData for each track are aggregated together into a single flattened MeshData
			for (int32 TrackIndex = 0; TrackIndex < GeometryCache->Tracks.Num(); ++TrackIndex)
			{
				FGeometryCacheMeshData TrackMeshData;
				GeometryCache->Tracks[TrackIndex]->GetMeshDataAtSampleIndex(FrameIndex, TrackMeshData);

				AppendGeometryCacheMeshData(TrackMeshData, FlattenedMeshData);
			}
		}
		return MoveTemp(FlattenedMeshData);
	}

	struct FGeometryCacheExportContext
	{
		FGeometryCacheExportContext(const UGeometryCache& GeometryCache)
			: SlotNames(GeometryCache.MaterialSlotNames)
		{
			// The GeometryCache's EndFrame is exclusive since it's there to allow frame interpolation past the real last frame
			InclusiveEndFrame = FMath::Max(GeometryCache.GetEndFrame(), GeometryCache.GetStartFrame() + 1);
			FrameRate = FMath::RoundToFloat((GeometryCache.GetEndFrame() - GeometryCache.GetStartFrame()) / GeometryCache.CalculateDuration());
		}

		FGeometryCacheExportContext(const TArray<FName>& InSlotNames)
			: SlotNames(InSlotNames)
		{
		}

		TArray<FName> SlotNames;
		int32 InclusiveEndFrame;
		float FrameRate;

		// Cached values of the last written attribute values
		// Since int cannot be interpolated, the missing timesampled attribute values will be the "held"
		// values of the previous written timesample
		pxr::VtArray<int> FaceVertexCounts;
		pxr::VtArray<int> FaceVertexIndices;
	};

	class IUnrealMeshData
	{
	public:
		virtual int32 GetNumVertices() const = 0;
		virtual int32 GetNumIndices() const = 0;
		virtual FVector3f GetVertexPosition(int32 Index) const = 0;
		virtual int32 GetVertexIndex(int32 Index) const = 0;
		virtual bool IsValid() const = 0;
		virtual bool HasNormals() const = 0;
		virtual FVector GetNormal(int32 Index) const = 0;
		virtual bool HasUV() const = 0;
		virtual FVector2D GetUV(int32 Index) const = 0;
		virtual bool HasVertexColors() const = 0;
		virtual FColor GetVertexColor(int32 Index) const = 0;
		virtual bool HasVelocities() const = 0;
		virtual FVector3f GetVelocity(int32 Index) const = 0;
		virtual int32 GetNumSections() const = 0;
		virtual uint32 GetSectionNumTriangles(int32 SectionIndex) const = 0;
		virtual uint32 GetSectionStartIndex(int32 SectionIndex) const = 0;
		virtual uint32 GetSectionMaterialIndex(int32 SectionIndex) const = 0;
		virtual bool HasMaterialAssignment() const = 0;
		virtual ~IUnrealMeshData()
		{
		}
	};

	class FGeometryCacheMeshDataWrapper : public IUnrealMeshData
	{
	public:
		FGeometryCacheMeshDataWrapper(const FGeometryCacheMeshData& InMeshData)
			: MeshData(InMeshData)
		{
		}

		int32 GetNumVertices() const override
		{
			return MeshData.Positions.Num();
		}
		int32 GetNumIndices() const override
		{
			return MeshData.Indices.Num();
		}
		FVector3f GetVertexPosition(int32 Index) const override
		{
			return MeshData.Positions[Index];
		}
		int32 GetVertexIndex(int32 Index) const override
		{
			return MeshData.Indices[Index];
		}
		bool IsValid() const override
		{
			return MeshData.Positions.Num() > 0;
		};
		bool HasNormals() const override
		{
			return MeshData.VertexInfo.bHasTangentZ;
		}
		FVector GetNormal(int32 Index) const override
		{
			return MeshData.TangentsZ[Index].ToFVector();
		}
		bool HasUV() const override
		{
			return MeshData.VertexInfo.bHasUV0;
		}
		FVector2D GetUV(int32 Index) const override
		{
			return FVector2D(MeshData.TextureCoordinates[Index]);
		}
		bool HasVertexColors() const override
		{
			return MeshData.VertexInfo.bHasColor0;
		}
		FColor GetVertexColor(int32 Index) const override
		{
			return MeshData.Colors[Index];
		}
		bool HasVelocities() const override
		{
			return MeshData.VertexInfo.bHasMotionVectors && !MeshData.MotionVectors.IsEmpty();
		}
		FVector3f GetVelocity(int32 Index) const override
		{
			return MeshData.MotionVectors.IsValidIndex(Index) ? MeshData.MotionVectors[Index] : FVector3f::ZeroVector;
		}
		int32 GetNumSections() const override
		{
			return MeshData.BatchesInfo.Num();
		}
		uint32 GetSectionNumTriangles(int32 SectionIndex) const override
		{
			return MeshData.BatchesInfo[SectionIndex].NumTriangles;
		}
		uint32 GetSectionStartIndex(int32 SectionIndex) const override
		{
			return MeshData.BatchesInfo[SectionIndex].StartIndex;
		}
		uint32 GetSectionMaterialIndex(int32 SectionIndex) const override
		{
			return MeshData.BatchesInfo[SectionIndex].MaterialIndex;
		}
		bool HasMaterialAssignment() const override
		{
			return true;
		}

	private:
		const FGeometryCacheMeshData& MeshData;
	};

	class FSkeletalMeshDataWrapper : public IUnrealMeshData
	{
	public:
		FSkeletalMeshDataWrapper(const FSkeletalMeshLODRenderData& InLODData, bool bVertexColors)
			: LODData(InLODData)
			, bHasVertexColors(bVertexColors)
		{
		}

		int32 GetNumVertices() const override
		{
			return LODData.GetNumVertices();
		}
		int32 GetNumIndices() const override
		{
			return LODData.MultiSizeIndexContainer.GetIndexBuffer()->Num();
		}
		FVector3f GetVertexPosition(int32 Index) const override
		{
			return LODData.StaticVertexBuffers.PositionVertexBuffer.VertexPosition(Index);
		}
		int32 GetVertexIndex(int32 Index) const override
		{
			return LODData.MultiSizeIndexContainer.GetIndexBuffer()->Get(Index);
		}
		bool IsValid() const override
		{
			return GetNumVertices() > 0;
		};
		bool HasNormals() const override
		{
			return true;
		}
		FVector GetNormal(int32 Index) const override
		{
			return FVector(LODData.StaticVertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(Index));
		}
		bool HasUV() const override
		{
			return LODData.StaticVertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords() > 0;
		}
		FVector2D GetUV(int32 Index) const override
		{
			const int32 UVIndex = 0;
			return FVector2D(LODData.StaticVertexBuffers.StaticMeshVertexBuffer.GetVertexUV(Index, UVIndex));
		}
		bool HasVertexColors() const override
		{
			return bHasVertexColors;
		}
		FColor GetVertexColor(int32 Index) const override
		{
			return LODData.StaticVertexBuffers.ColorVertexBuffer.VertexColor(Index);
		}
		bool HasVelocities() const override
		{
			return false;
		}
		FVector3f GetVelocity(int32 Index) const override
		{
			return {};
		}
		int32 GetNumSections() const override
		{
			return LODData.RenderSections.Num();
		}
		uint32 GetSectionNumTriangles(int32 SectionIndex) const override
		{
			return LODData.RenderSections[SectionIndex].NumTriangles;
		}
		uint32 GetSectionStartIndex(int32 SectionIndex) const override
		{
			return LODData.RenderSections[SectionIndex].BaseIndex;
		}
		uint32 GetSectionMaterialIndex(int32 SectionIndex) const override
		{
			return LODData.RenderSections[SectionIndex].MaterialIndex;
		}
		bool HasMaterialAssignment() const override
		{
			return true;
		}

	private:
		const FSkeletalMeshLODRenderData& LODData;
		bool bHasVertexColors;
	};

	// Wrapper for skinned vertices of a skeletal mesh as rendered through a skeletal mesh component
	class FSkinnedVerticesDataWrapper : public FSkeletalMeshDataWrapper
	{
	public:
		FSkinnedVerticesDataWrapper(
			const TArray<FFinalSkinVertex>& InSkinnedVertices,
			const FSkeletalMeshLODRenderData& InLODData,
			bool bVertexColors
		)
			: FSkeletalMeshDataWrapper(InLODData, bVertexColors)
			, SkinnedVertices(InSkinnedVertices)
		{
		}

		int32 GetNumVertices() const override
		{
			return SkinnedVertices.Num();
		}
		FVector3f GetVertexPosition(int32 Index) const override
		{
			return SkinnedVertices[Index].Position;
		}
		bool HasNormals() const override
		{
			return true;
		}
		FVector GetNormal(int32 Index) const override
		{
			return SkinnedVertices[Index].TangentZ.ToFVector();
		}
		bool HasUV() const override
		{
			// Use the UV of the skeletal mesh
			return false;
		}
		// In case we want to use the UV from the skinned vertices, but they are not animated anyway
		// FVector2D GetUV(int32 Index) const override
		//{
		//	return SkinnedVertices[Index].TextureCoordinates[0];
		//}
		bool HasVertexColors() const override
		{
			return false;
		}
		FColor GetVertexColor(int32 Index) const override
		{
			return {};
		}
		bool HasMaterialAssignment() const override
		{
			// Use the material assignment/section info of the skeletal mesh
			return false;
		}

	private:
		const TArray<FFinalSkinVertex>& SkinnedVertices;
	};

	void ConvertMeshData(
		const IUnrealMeshData& MeshData,
		const TArray<FString>& MaterialAssignments,
		const pxr::UsdTimeCode TimeCode,
		pxr::UsdPrim PrimToReceiveMaterialAssignments,
		FGeometryCacheExportContext& ExportContext,
		pxr::UsdGeomMesh& UsdMesh
	)
	{
		pxr::UsdPrim MeshPrim = UsdMesh.GetPrim();
		pxr::UsdStageRefPtr Stage = MeshPrim.GetStage();
		if (!Stage)
		{
			return;
		}
		const FUsdStageInfo StageInfo{Stage};

		// Vertices
		{
			const int32 VertexCount = MeshData.GetNumVertices();

			// Points
			{
				pxr::UsdAttribute Points = UsdMesh.CreatePointsAttr();
				if (Points)
				{
					pxr::VtArray<pxr::GfVec3f> PointsArray;
					PointsArray.reserve(VertexCount);

					for (int32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
					{
						PointsArray.push_back(UnrealToUsd::ConvertVectorFloat(StageInfo, (FVector)MeshData.GetVertexPosition(VertexIndex)));
					}

					Points.Set(PointsArray, TimeCode);
				}
			}

			// Normals
			if (MeshData.HasNormals())
			{
				// We need to emit this if we're writing normals (which we always are) because any DCC that can
				// actually subdivide (like usdview) will just discard authored normals and fully recompute them
				// on-demand in case they have a valid subdivision scheme (which is the default state).
				// Reference: https://graphics.pixar.com/usd/release/api/class_usd_geom_mesh.html#UsdGeom_Mesh_Normals
				if (pxr::UsdAttribute SubdivisionAttr = UsdMesh.CreateSubdivisionSchemeAttr())
				{
					ensure(SubdivisionAttr.Set(pxr::UsdGeomTokens->none));
				}

				pxr::UsdAttribute NormalsAttribute = UsdMesh.CreateNormalsAttr();
				if (NormalsAttribute)
				{
					pxr::VtArray<pxr::GfVec3f> Normals;
					Normals.reserve(VertexCount);

					for (int32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
					{
						FVector VertexNormal = MeshData.GetNormal(VertexIndex);
						Normals.push_back(UnrealToUsd::ConvertVectorFloat(StageInfo, VertexNormal));
					}

					NormalsAttribute.Set(Normals, TimeCode);
				}
			}

			// UVs
			if (MeshData.HasUV())
			{
				// Only one UV set is supported
				const int32 TexCoordSourceIndex = 0;
				pxr::TfToken UsdUVSetName = UsdUtils::GetUVSetName(TexCoordSourceIndex).Get();

				pxr::UsdGeomPrimvar PrimvarST = pxr::UsdGeomPrimvarsAPI(MeshPrim)
													.CreatePrimvar(UsdUVSetName, pxr::SdfValueTypeNames->TexCoord2fArray, pxr::UsdGeomTokens->vertex);

				if (PrimvarST)
				{
					pxr::VtVec2fArray UVs;

					for (int32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
					{
						FVector2D TexCoord = FVector2D(MeshData.GetUV(VertexIndex));
						TexCoord[1] = 1.f - TexCoord[1];

						UVs.push_back(UnrealToUsd::ConvertVectorFloat(TexCoord));
					}

					PrimvarST.Set(UVs, TimeCode);
				}
			}

			// Vertex colors
			if (MeshData.HasVertexColors())
			{
				pxr::UsdGeomPrimvar DisplayColorPrimvar = UsdMesh.CreateDisplayColorPrimvar(pxr::UsdGeomTokens->vertex);
				pxr::UsdGeomPrimvar DisplayOpacityPrimvar = UsdMesh.CreateDisplayOpacityPrimvar(pxr::UsdGeomTokens->vertex);

				if (DisplayColorPrimvar && DisplayOpacityPrimvar)
				{
					pxr::VtArray<pxr::GfVec3f> DisplayColors;
					DisplayColors.reserve(VertexCount);

					pxr::VtArray<float> DisplayOpacities;
					DisplayOpacities.reserve(VertexCount);

					for (int32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
					{
						const FColor& VertexColor = MeshData.GetVertexColor(VertexIndex);

						// The color in the MeshData is already stored as linear
						pxr::GfVec4f Color = UnrealToUsd::ConvertColor(VertexColor.ReinterpretAsLinear());
						DisplayColors.push_back(pxr::GfVec3f(Color[0], Color[1], Color[2]));
						DisplayOpacities.push_back(Color[3]);
					}

					DisplayColorPrimvar.Set(DisplayColors, TimeCode);
					DisplayOpacityPrimvar.Set(DisplayOpacities, TimeCode);
				}
			}

			// Velocities
			if (MeshData.HasVelocities())
			{
				pxr::UsdAttribute VelocitiesAttribute = UsdMesh.CreateVelocitiesAttr();
				if (VelocitiesAttribute)
				{
					pxr::VtArray<pxr::GfVec3f> Velocities;
					Velocities.reserve(VertexCount);

					for (int32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
					{
						// The motion vectors in the MeshData are stored as unit per frame so convert it back to unit per second
						Velocities.push_back(
							UnrealToUsd::ConvertVectorFloat(StageInfo, (FVector)-MeshData.GetVelocity(VertexIndex) * ExportContext.FrameRate)
						);
					}

					VelocitiesAttribute.Set(Velocities, TimeCode);
				}
			}
		}

		// Faces
		{
			const int32 NumIndices = MeshData.GetNumIndices();
			const int32 FaceCount = NumIndices / 3;
			// Face Vertex Counts
			{
				pxr::UsdAttribute FaceCountsAttribute = UsdMesh.CreateFaceVertexCountsAttr();

				if (FaceCountsAttribute)
				{
					pxr::VtArray<int> FaceVertexCounts;
					FaceVertexCounts.reserve(FaceCount);

					for (int32 FaceIndex = 0; FaceIndex < FaceCount; ++FaceIndex)
					{
						FaceVertexCounts.push_back(3);
					}

					if (ExportContext.FaceVertexCounts != FaceVertexCounts)
					{
						FaceCountsAttribute.Set(FaceVertexCounts, TimeCode);
						ExportContext.FaceVertexCounts = FaceVertexCounts;
					}
				}
			}

			// Face Vertex Indices
			{
				pxr::UsdAttribute FaceVertexIndicesAttribute = UsdMesh.GetFaceVertexIndicesAttr();

				if (FaceVertexIndicesAttribute)
				{
					pxr::VtArray<int> FaceVertexIndices;
					FaceVertexIndices.reserve(NumIndices);

					for (int32 Index = 0; Index < NumIndices; ++Index)
					{
						FaceVertexIndices.push_back(MeshData.GetVertexIndex(Index));
					}

					if (ExportContext.FaceVertexIndices != FaceVertexIndices)
					{
						FaceVertexIndicesAttribute.Set(FaceVertexIndices, TimeCode);
						ExportContext.FaceVertexIndices = FaceVertexIndices;
					}
				}
			}
		}

		// Material assignments
		if (MeshData.HasMaterialAssignment())
		{
			// This LOD has a single material assignment, just create/bind an UnrealMaterial child prim directly
			if (MaterialAssignments.Num() == 1)
			{
				UsdUtils::AuthorUnrealMaterialBinding(PrimToReceiveMaterialAssignments, MaterialAssignments[0]);
			}
			// Multiple material assignments to the same mesh. Need to create a GeomSubset for each UE mesh section
			else if (MaterialAssignments.Num() > 1)
			{
				TSet<FString> UsedSectionNames;
				// Need to fetch all triangles of a section, and add their indices
				for (int32 SectionIndex = 0; SectionIndex < MeshData.GetNumSections(); ++SectionIndex)
				{
					// Note that we will continue authoring the GeomSubsets on even if we later find out we have no material assignment (just
					// "") for this section, so as to satisfy the "partition" family condition (below)
					FString SectionName;
					if (ExportContext.SlotNames.IsValidIndex(SectionIndex))
					{
						SectionName = ExportContext.SlotNames[SectionIndex].ToString();
						// Make sure the SectionName contains no invalid characters for a prim identifier, otherwise the prim creation will fail
						SectionName = UsdUtils::SanitizeUsdIdentifier(*SectionName);
						SectionName = UsdUnreal::ObjectUtils::GetUniqueName(SectionName, UsedSectionNames);
						UsedSectionNames.Add(SectionName);
					}
					else
					{
						SectionName = FString::Printf(TEXT("Section%d"), SectionIndex);
					}

					FSdfPath PrimPath(*SectionName);
					pxr::UsdPrim GeomSubsetPrim = Stage->DefinePrim(
						MeshPrim.GetPath().AppendPath(PrimPath),
						UnrealToUsd::ConvertToken(TEXT("GeomSubset")).Get()
					);

					// MaterialPrim may be in another stage, so we may need another GeomSubset there
					pxr::UsdPrim MaterialGeomSubsetPrim = GeomSubsetPrim;
					if (PrimToReceiveMaterialAssignments.GetStage() != MeshPrim.GetStage())
					{
						MaterialGeomSubsetPrim = PrimToReceiveMaterialAssignments.GetStage()->OverridePrim(
							PrimToReceiveMaterialAssignments.GetPath().AppendPath(PrimPath)
						);
					}

					pxr::UsdGeomSubset GeomSubsetSchema{GeomSubsetPrim};

					// Element type attribute
					// Write the geomsubset attributes only once since they are at Default time anyway
					pxr::UsdAttribute ElementTypeAttr = GeomSubsetSchema.CreateElementTypeAttr();
					if (!ElementTypeAttr.HasAuthoredValue())
					{
						ElementTypeAttr.Set(pxr::UsdGeomTokens->face);

						// Indices attribute
						const uint32 TriangleCount = MeshData.GetSectionNumTriangles(SectionIndex);
						const uint32 FirstTriangleIndex = MeshData.GetSectionStartIndex(SectionIndex) / 3;	  // StartIndex is the first *vertex*
																											  // instance index
						pxr::VtArray<int> IndicesAttrValue;
						for (uint32 TriangleIndex = FirstTriangleIndex; TriangleIndex - FirstTriangleIndex < TriangleCount; ++TriangleIndex)
						{
							// Note that we add VertexInstances in sequence to the usda file for the faceVertexInstances attribute, which
							// also constitutes our triangle order
							IndicesAttrValue.push_back(static_cast<int>(TriangleIndex));
						}

						// Since family name and type attributes must be set at time Default, set the Indices at time Default too
						// #ueent_todo: Add support for varying geomsubsets. This can happen with animation where sections
						// visibility are toggled on/off
						pxr::UsdAttribute IndicesAttr = GeomSubsetSchema.CreateIndicesAttr();
						IndicesAttr.Set(IndicesAttrValue);

						// Family name attribute
						pxr::UsdAttribute FamilyNameAttr = GeomSubsetSchema.CreateFamilyNameAttr();
						FamilyNameAttr.Set(pxr::UsdShadeTokens->materialBind);

						// Family type
						pxr::UsdGeomSubset::SetFamilyType(UsdMesh, pxr::UsdShadeTokens->materialBind, pxr::UsdGeomTokens->partition);

						// material:binding relationship
						uint32 MaterialIndex = MeshData.GetSectionMaterialIndex(SectionIndex);
						if (MaterialAssignments.IsValidIndex(MaterialIndex))
						{
							UsdUtils::AuthorUnrealMaterialBinding(MaterialGeomSubsetPrim, MaterialAssignments[MaterialIndex]);
						}
					}
				}
			}
		}
	}
}	 // namespace UE::UsdGeometryCacheConversion::Private

bool UnrealToUsd::ConvertGeometryCache(const UGeometryCache* GeometryCache, pxr::UsdPrim& UsdPrim, UE::FUsdStage* StageForMaterialAssignments)
{
	using namespace UE::UsdGeometryCacheConversion::Private;

	FScopedUsdAllocs UsdAllocs;

	pxr::UsdStageRefPtr Stage = UsdPrim.GetStage();
	if (!GeometryCache || !Stage)
	{
		return false;
	}

	const FUsdStageInfo StageInfo(Stage);

	// Collect all material assignments, referenced by the sections' material indices
	bool bHasMaterialAssignments = false;
	TArray<FString> MaterialAssignments;
	for (const UMaterialInterface* Material : GeometryCache->Materials)
	{
		FString AssignedMaterialPathName;
		if (Material)
		{
			if (Material->GetOutermost() != GetTransientPackage())
			{
				AssignedMaterialPathName = Material->GetPathName();
				bHasMaterialAssignments = true;
			}
		}

		MaterialAssignments.Add(AssignedMaterialPathName);
	}
	if (!bHasMaterialAssignments)
	{
		// Prevent creation of the unrealMaterials attribute in case we don't have any assignments at all
		MaterialAssignments.Reset();
	}

	// Author material bindings on the dedicated stage if we have one
	pxr::UsdStageRefPtr MaterialStage;
	if (StageForMaterialAssignments)
	{
		MaterialStage = static_cast<pxr::UsdStageRefPtr>(*StageForMaterialAssignments);
	}
	else
	{
		MaterialStage = Stage;
	}

	pxr::UsdGeomMesh TargetMesh{UsdPrim};
	pxr::UsdPrim MaterialPrim = MaterialStage->OverridePrim(UsdPrim.GetPath());
	pxr::UsdAttribute ExtentsAttr = TargetMesh ? TargetMesh.CreateExtentAttr() : pxr::UsdAttribute{};

	FGeometryCacheExportContext ExportContext(*GeometryCache);
	const int32 StartFrame = GeometryCache->GetStartFrame();
	const int32 EndFrame = ExportContext.InclusiveEndFrame;
	int32 ActualStartFrame = -1;

	for (int32 FrameIndex = StartFrame; FrameIndex <= EndFrame; ++FrameIndex)
	{
		FGeometryCacheMeshData MeshData = GetFlattenedGeometryCacheMeshData(GeometryCache, FrameIndex - StartFrame);
		// First frame of the animation cannot be empty otherwise the geometry cache translator would not be able to detect the animation
		// It is allowed to have empty frames during or at the end of the animation, eg. for fluid sim or FX that disappear
		const bool bIsValidFrame = MeshData.Positions.Num() > 0 || ActualStartFrame > 0;
		if (bIsValidFrame)
		{
			if (ActualStartFrame == -1)
			{
				// The actual start frame is the first frame with some data
				ActualStartFrame = FrameIndex;
			}
			const float TimeCode = FrameIndex;
			ConvertMeshData(FGeometryCacheMeshDataWrapper(MeshData), MaterialAssignments, TimeCode, MaterialPrim, ExportContext, TargetMesh);
		}

		if (MeshData.BoundingBox.IsValid && ExtentsAttr)
		{
			TUsdStore<pxr::VtArray<pxr::GfVec3f>> USDBounds = UnrealToUsd::ConvertBounds(StageInfo, FBox{MeshData.BoundingBox});
			ExtentsAttr.Set(USDBounds.Get(), pxr::UsdTimeCode(FrameIndex));
		}
	}

	// Configure time metadata for the stage
	UE::FUsdStage UsdStage(MaterialStage);
	UsdUtils::AddTimeCodeRangeToLayer(UsdStage.GetRootLayer(), ActualStartFrame, EndFrame);
	UsdStage.SetTimeCodesPerSecond(ExportContext.FrameRate);

	return true;
}

bool UnrealToUsd::ConvertSkeletalMeshToStaticMesh(
	const USkeletalMesh* SkeletalMesh,
	pxr::UsdPrim& UsdPrim,
	const pxr::UsdTimeCode TimeCode,
	UE::FUsdStage* StageForMaterialAssignments
)
{
	using namespace UE::UsdGeometryCacheConversion::Private;

	if (!SkeletalMesh)
	{
		return false;
	}

	FScopedUsdAllocs UsdAllocs;

	pxr::UsdGeomMesh MeshPrim(UsdPrim);
	if (!MeshPrim)
	{
		return false;
	}

	pxr::UsdStageRefPtr Stage = UsdPrim.GetStage();
	if (!Stage)
	{
		return false;
	}

	const FUsdStageInfo StageInfo(Stage);

	// Collect all material assignments, referenced by the sections' material indices
	bool bHasMaterialAssignments = false;
	TArray<FString> MaterialAssignments;
	TArray<FName> SlotNames;
	for (const FSkeletalMaterial& Material : SkeletalMesh->GetMaterials())
	{
		FString AssignedMaterialPathName;
		if (Material.MaterialInterface->GetOutermost() != GetTransientPackage())
		{
			AssignedMaterialPathName = Material.MaterialInterface->GetPathName();
			SlotNames.Add(Material.MaterialSlotName);
			bHasMaterialAssignments = true;
		}

		MaterialAssignments.Add(AssignedMaterialPathName);
	}
	if (!bHasMaterialAssignments)
	{
		// Prevent creation of the unrealMaterials attribute in case we don't have any assignments at all
		MaterialAssignments.Reset();
	}

	// Author material bindings on the dedicated stage if we have one
	pxr::UsdStageRefPtr MaterialStage;
	if (StageForMaterialAssignments)
	{
		MaterialStage = static_cast<pxr::UsdStageRefPtr>(*StageForMaterialAssignments);
	}
	else
	{
		MaterialStage = Stage;
	}

	const FSkeletalMeshRenderData& SkelMeshRenderData = *SkeletalMesh->GetResourceForRendering();

	const int32 LODLevel = 0;
	const FSkeletalMeshLODRenderData& LODData = SkelMeshRenderData.LODRenderData[LODLevel];

	FGeometryCacheExportContext ExportContext(SlotNames);
	pxr::UsdPrim MaterialPrim = MaterialStage->OverridePrim(UsdPrim.GetPath());
	ConvertMeshData(
		FSkeletalMeshDataWrapper(LODData, SkeletalMesh->GetHasVertexColors()),
		MaterialAssignments,
		TimeCode,
		MaterialPrim,
		ExportContext,
		MeshPrim
	);

	pxr::UsdAttribute ExtentsAttr = MeshPrim.CreateExtentAttr();
	const FBox BoundingBox = SkeletalMesh->GetBounds().GetBox();
	if (BoundingBox.IsValid && ExtentsAttr)
	{
		TUsdStore<pxr::VtArray<pxr::GfVec3f>> USDBounds = UnrealToUsd::ConvertBounds(StageInfo, BoundingBox);
		ExtentsAttr.Set(USDBounds.Get(), TimeCode);
	}

	return true;
}

bool UnrealToUsd::ConvertAnimSequenceToAnimatedMesh(
	UAnimSequence* AnimSequence,
	USkeletalMesh* SkeletalMesh,
	pxr::UsdPrim& UsdPrim,
	UE::FUsdStage* StageForMaterialAssignments
)
{
	using namespace UE::UsdGeometryCacheConversion::Private;

	UWorld* World = IUsdClassesModule::GetCurrentWorld();
	if (!AnimSequence || !SkeletalMesh || !World)
	{
		return false;
	}

	FScopedUsdAllocs UsdAllocs;

	pxr::UsdGeomMesh MeshPrim(UsdPrim);
	if (!MeshPrim)
	{
		return false;
	}

	pxr::UsdStageRefPtr Stage = UsdPrim.GetStage();
	if (!Stage)
	{
		return false;
	}

	// Collect all material assignments, referenced by the sections' material indices
	bool bHasMaterialAssignments = false;
	TArray<FString> MaterialAssignments;
	TArray<FName> SlotNames;
	for (const FSkeletalMaterial& Material : SkeletalMesh->GetMaterials())
	{
		FString AssignedMaterialPathName;
		if (Material.MaterialInterface->GetOutermost() != GetTransientPackage())
		{
			AssignedMaterialPathName = Material.MaterialInterface->GetPathName();
			SlotNames.Add(Material.MaterialSlotName);
			bHasMaterialAssignments = true;
		}

		MaterialAssignments.Add(AssignedMaterialPathName);
	}
	if (!bHasMaterialAssignments)
	{
		// Prevent creation of the unrealMaterials attribute in case we don't have any assignments at all
		MaterialAssignments.Reset();
	}

	// Author material bindings on the dedicated stage if we have one
	pxr::UsdStageRefPtr MaterialStage;
	if (StageForMaterialAssignments)
	{
		MaterialStage = static_cast<pxr::UsdStageRefPtr>(*StageForMaterialAssignments);
	}
	else
	{
		MaterialStage = Stage;
	}

	pxr::UsdPrim MaterialPrim = MaterialStage->OverridePrim(UsdPrim.GetPath());

	// Create a temp SkeletalMeshActor on which to play back the AnimSequence and get the skinned vertices to bake
	FActorSpawnParameters SpawnParameters;
	SpawnParameters.ObjectFlags = RF_Transient;
	SpawnParameters.Name = *UsdToUnreal::ConvertString(UsdPrim.GetName());
	SpawnParameters.NameMode = FActorSpawnParameters::ESpawnActorNameMode::Requested;	 // Will generate a unique name in case of a conflict

	ASkeletalMeshActor* SkelMeshActor = Cast<ASkeletalMeshActor>(World->SpawnActor(ASkeletalMeshActor::StaticClass(), nullptr, SpawnParameters));

	USkeletalMeshComponent* SkelMeshComponent = SkelMeshActor->GetSkeletalMeshComponent();
	SkelMeshComponent->SetSkeletalMeshAsset(SkeletalMesh);
	SkelMeshComponent->OverrideAnimationData(AnimSequence);

	const FSkeletalMeshRenderData& SkelMeshRenderData = *SkeletalMesh->GetResourceForRendering();

	const int32 LODLevel = 0;
	const FSkeletalMeshLODRenderData& LODData = SkelMeshRenderData.LODRenderData[LODLevel];

	const int32 StartFrame = 0;
	const int32 EndFrame = AnimSequence->GetNumberOfSampledKeys() - 1;
	const double FPS = AnimSequence->GetSamplingFrameRate().AsDecimal();
	const bool bHasVertexColors = SkeletalMesh->GetHasVertexColors();
	FGeometryCacheExportContext ExportContext(SlotNames);
	for (int32 FrameIndex = StartFrame; FrameIndex <= EndFrame; ++FrameIndex)
	{
		double Time = FrameIndex / FPS;
		SkelMeshComponent->SetPosition((float)Time);

		if (USkeletalMeshComponent* Leader = Cast<USkeletalMeshComponent>(SkelMeshComponent->LeaderPoseComponent.Get()))
		{
			UsdUtils::RefreshSkeletalMeshComponent(*Leader);
		}
		UsdUtils::RefreshSkeletalMeshComponent(*SkelMeshComponent);

		if (AActor* Owner = SkelMeshComponent->GetOwner())
		{
			Owner->Tick(0.0f);
		}

		TArray<FFinalSkinVertex> SkinnedVertices;
		SkelMeshComponent->GetCPUSkinnedVertices(SkinnedVertices, LODLevel);

		ConvertMeshData(
			FSkinnedVerticesDataWrapper(SkinnedVertices, LODData, bHasVertexColors),
			MaterialAssignments,
			FrameIndex,
			MaterialPrim,
			ExportContext,
			MeshPrim
		);
	}

	World->DestroyActor(SkelMeshActor);

	return true;
}

bool UnrealToUsd::CreateSkeletalAnimationToMeshBaker(
	UE::FUsdPrim& UsdPrim,
	USkeletalMeshComponent& SkelMeshComponent,
	UnrealToUsd::FComponentBaker& OutBaker
)
{
	using namespace UE::UsdGeometryCacheConversion::Private;

	USkeletalMesh* SkeletalMesh = SkelMeshComponent.GetSkeletalMeshAsset();
	if (!SkeletalMesh)
	{
		return false;
	}

	FScopedUsdAllocs Allocs;

	pxr::UsdGeomMesh MeshPrim(UsdPrim);
	if (!MeshPrim)
	{
		return false;
	}

	pxr::UsdStageRefPtr Stage = UsdPrim.GetStage();
	if (!Stage)
	{
		return false;
	}

	// Collect all material assignments, referenced by the sections' material indices
	bool bHasMaterialAssignments = false;
	TArray<FString> MaterialAssignments;
	TArray<FName> SlotNames;
	for (const FSkeletalMaterial& Material : SkeletalMesh->GetMaterials())
	{
		FString AssignedMaterialPathName;
		if (Material.MaterialInterface->GetOutermost() != GetTransientPackage())
		{
			AssignedMaterialPathName = Material.MaterialInterface->GetPathName();
			SlotNames.Add(Material.MaterialSlotName);
			bHasMaterialAssignments = true;
		}

		MaterialAssignments.Add(AssignedMaterialPathName);
	}
	if (!bHasMaterialAssignments)
	{
		// Prevent creation of the unrealMaterials attribute in case we don't have any assignments at all
		MaterialAssignments.Reset();
	}

	OutBaker.ComponentPath = SkelMeshComponent.GetPathName();
	OutBaker.BakerType = UnrealToUsd::EBakingType::Skeletal;

	const FSkeletalMeshRenderData& SkelMeshRenderData = *SkeletalMesh->GetResourceForRendering();

	const int32 LODLevel = 0;
	const FSkeletalMeshLODRenderData& LODData = SkelMeshRenderData.LODRenderData[LODLevel];

	static IConsoleVariable* SkipConstantValuesCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("USD.LevelSequenceExport.SkipConstantValues"));
	const bool bSkipConstantValues = SkipConstantValuesCvar && SkipConstantValuesCvar->GetBool();

	const bool bHasVertexColors = SkeletalMesh->GetHasVertexColors();
	OutBaker.BakerFunction = [&SkelMeshComponent,
							  &LODData,
							  MaterialAssignments,
							  UsdPrim,
							  bHasVertexColors,
							  ExportContext = FGeometryCacheExportContext(SlotNames),
							  LastValue = TOptional<TArray<FFinalSkinVertex>>(),
							  LastTimeCode = -DBL_MAX,
							  bSkipConstantValues](double UsdTimeCode) mutable
	{
		FScopedUsdAllocs InnerAllocs;

		if (USkeletalMeshComponent* Leader = Cast<USkeletalMeshComponent>(SkelMeshComponent.LeaderPoseComponent.Get()))
		{
			UsdUtils::RefreshSkeletalMeshComponent(*Leader);
		}
		UsdUtils::RefreshSkeletalMeshComponent(SkelMeshComponent);

		if (AActor* Owner = SkelMeshComponent.GetOwner())
		{
			Owner->Tick(0.0f);
		}

		const int32 LODLevel = 0;
		TArray<FFinalSkinVertex> SkinnedVertices;
		SkelMeshComponent.GetCPUSkinnedVertices(SkinnedVertices, LODLevel);

		// This logic is copied over from the CreateCachedAttrSetter functions on USDPrimConversion.cpp so we don't have
		// to expose neither CreateCachedAttrSetter nor the ConvertMeshData/IUnrealMeshData
		bool bNewValueIsEqual = false;
		if (LastValue.IsSet())
		{
			if (LastValue.GetValue().Num() != SkinnedVertices.Num())
			{
				bNewValueIsEqual = false;
			}
			else
			{
				bNewValueIsEqual = true;
				for (int32 Index = 0; Index < SkinnedVertices.Num(); ++Index)
				{
					const FFinalSkinVertex& LastValueEntry = LastValue.GetValue()[Index];
					const FFinalSkinVertex& NewValueEntry = SkinnedVertices[Index];

					// Only checking these as these are the only members of FFinalSkinVertex that
					// FSkinnedVerticesDataWrapper reads anyway
					if (!LastValueEntry.Position.Equals(NewValueEntry.Position) || LastValueEntry.TangentZ != NewValueEntry.TangentZ)
					{
						bNewValueIsEqual = false;
						break;
					}
				}
			}
		}
		if (bSkipConstantValues && bNewValueIsEqual)
		{
			LastTimeCode = UsdTimeCode;
			return;
		}

		pxr::UsdGeomMesh MeshPrim(UsdPrim);
		ConvertMeshData(
			FSkinnedVerticesDataWrapper(SkinnedVertices, LODData, bHasVertexColors),
			MaterialAssignments,
			UsdTimeCode,
			UsdPrim,
			ExportContext,
			MeshPrim
		);

		if (!bNewValueIsEqual && LastValue.IsSet())
		{
			ConvertMeshData(
				FSkinnedVerticesDataWrapper(LastValue.GetValue(), LODData, bHasVertexColors),
				MaterialAssignments,
				LastTimeCode,
				UsdPrim,
				ExportContext,
				MeshPrim
			);
		}

		LastValue = SkinnedVertices;
		LastTimeCode = UsdTimeCode;
	};

	return true;
}

void UsdUtils::RepairNormalsAndTangents(const FString& PrimPath, FMeshDescription& MeshDescription)
{
	FStaticMeshConstAttributes Attributes{MeshDescription};
	TArrayView<const FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals().GetRawArray();

	// Similar to FStaticMeshOperations::AreNormalsAndTangentsValid but we don't care about tangents since we never
	// read those from USD
	uint64 InvalidNormalCount = 0;
	for (const FVertexInstanceID VertexInstanceID : MeshDescription.VertexInstances().GetElementIDs())
	{
		if (VertexInstanceNormals[VertexInstanceID].IsNearlyZero() || VertexInstanceNormals[VertexInstanceID].ContainsNaN())
		{
			++InvalidNormalCount;
		}
	}
	if (InvalidNormalCount == 0)
	{
		return;
	}

	const float InvalidNormalFraction = (float)InvalidNormalCount / (float)VertexInstanceNormals.Num();

	// We always need to do this at this point as ComputeTangentsAndNormals will end up computing tangents anyway
	// and our triangle tangents are always invalid
	FStaticMeshOperations::ComputeTriangleTangentsAndNormals(MeshDescription);

	// Make sure our normals can be rebuilt from MeshDescription::InitializeAutoGeneratedAttributes in case some tool needs them.
	// Always force-compute tangents here as we never have them anyway. If we don't force them to be recomputed we'll get
	// the worst of both worlds as some of these will be arbitrarily recomputed anyway, and some will be left invalid
	EComputeNTBsFlags Options = GSkipMeshTangentComputation ? EComputeNTBsFlags::None
															: EComputeNTBsFlags::UseMikkTSpace | EComputeNTBsFlags::Tangents;

	// Repairing can take a long time for degenerate triangles (UE-194839)
	Options |= EComputeNTBsFlags::IgnoreDegenerateTriangles;

	if (InvalidNormalFraction >= GMeshNormalRepairThreshold)
	{
		Options |= EComputeNTBsFlags::Normals;
	}

	FStaticMeshOperations::ComputeTangentsAndNormals(MeshDescription, Options);
}

TOptional<UsdUtils::FDisplayColorMaterial> UsdUtils::ExtractDisplayColorMaterial(const pxr::UsdGeomGprim& Gprim, const pxr::UsdTimeCode TimeCode)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UsdUtils::ExtractDisplayColorMaterial);

	if (!Gprim)
	{
		return {};
	}

	if (!Gprim.GetDisplayOpacityAttr().IsDefined() && !Gprim.GetDisplayColorAttr().IsDefined())
	{
		return {};
	}

	UsdUtils::FDisplayColorMaterial Desc;

	// Opacity
	pxr::VtArray<float> UsdOpacities = UsdUtils::GetUsdValue<pxr::VtArray<float>>(Gprim.GetDisplayOpacityAttr(), TimeCode);
	for (float Opacity : UsdOpacities)
	{
		Desc.bHasOpacity = !FMath::IsNearlyEqual(Opacity, 1.f);
		if (Desc.bHasOpacity)
		{
			break;
		}
	}

	// Double-sided
	if (Gprim.GetDoubleSidedAttr().IsDefined())
	{
		Desc.bIsDoubleSided = UsdUtils::GetUsdValue<bool>(Gprim.GetDoubleSidedAttr(), TimeCode);
	}

	return Desc;
}

namespace UE::UsdGeomMeshConversion::Private
{
	bool DoesPrimContainMeshLODsInternal(const pxr::UsdPrim& Prim)
	{
		FScopedUsdAllocs Allocs;

		if (!Prim)
		{
			return false;
		}

		const std::string LODString = UnrealIdentifiers::LOD.GetString();

		pxr::UsdVariantSets VariantSets = Prim.GetVariantSets();
		if (!VariantSets.HasVariantSet(LODString))
		{
			return false;
		}

		std::string Selection = VariantSets.GetVariantSet(LODString).GetVariantSelection();
		int32 LODIndex = UsdGeomMeshImpl::GetLODIndexFromName(Selection);
		if (LODIndex == INDEX_NONE)
		{
			return false;
		}

		return true;
	}
}	 // namespace UE::UsdGeomMeshConversion::Private

bool UsdUtils::DoesPrimContainMeshLODs(const pxr::UsdPrim& Prim)
{
	const bool bHasValidVariantSetup = UE::UsdGeomMeshConversion::Private::DoesPrimContainMeshLODsInternal(Prim);
	if (bHasValidVariantSetup)
	{
		FScopedUsdAllocs Allocs;

		// Check if it has at least one mesh too
		pxr::UsdPrimSiblingRange PrimRange = Prim.GetChildren();
		for (pxr::UsdPrimSiblingRange::iterator PrimRangeIt = PrimRange.begin(); PrimRangeIt != PrimRange.end(); ++PrimRangeIt)
		{
			const pxr::UsdPrim& Child = *PrimRangeIt;
			if (pxr::UsdGeomMesh ChildMesh{Child})
			{
				return true;
			}
		}
	}

	return false;
}

bool UsdUtils::IsGeomMeshALOD(const pxr::UsdPrim& UsdMeshPrim)
{
	FScopedUsdAllocs Allocs;

	pxr::UsdGeomMesh UsdMesh{UsdMeshPrim};
	if (!UsdMesh)
	{
		return false;
	}

	// Note that we can't robustly check whether UsdMeshPrim "is inside of the LOD variant set" or not,
	// because that can vary *per layer*... For example, a stage with layers root.usda and sub.usda can have
	// the MeshA prim inside the LOD variant on root.usa, and MeshB prim inside the LOD variant on sub.usda.
	// The LOD variant setup is a set of rules we specify ourselves and the users must adhere to, and one of
	// them is to have a single Mesh prim as a child of the variant set prim. This means that as soon
	// as the user puts more than one Mesh prim inside of the variant set prim, we're already in a "garbage in"
	// scenario, and will likely generate some garbage in turn. We'll emit a bunch of warning for that though.

	return UE::UsdGeomMeshConversion::Private::DoesPrimContainMeshLODsInternal(UsdMeshPrim.GetParent());
}

bool UsdUtils::IsCollisionEnabledForPrim(const pxr::UsdPrim& UsdPrim)
{
	if (pxr::UsdPhysicsCollisionAPI CollisionAPI{UsdPrim})
	{
		bool bIsCollisionEnabled = false;
		pxr::UsdAttribute CollisionAttr = CollisionAPI.GetCollisionEnabledAttr();
		if (CollisionAttr && CollisionAttr.Get(&bIsCollisionEnabled) && bIsCollisionEnabled)
		{
			return true;
		}
	}

	return false;
}

bool UsdUtils::IsCollisionMesh(const pxr::UsdPrim& UsdPrim)
{
	// https://openusd.org/release/api/usd_physics_page_front.html#usdPhysics_collision_shapes
	// "Collision meshes may be specified explicitly by adding the custom collider mesh as a sibling to the
	// original graphics mesh, UsdGeomImageable purpose to "guide" so it does not render, and apply
	// UsdPhysicsCollisionAPI and UsdPhysicsMeshCollisionAPI to it specifying no approximation."
	FScopedUsdAllocs Allocs;

	pxr::UsdGeomMesh UsdMesh{UsdPrim};
	if (!UsdMesh)
	{
		return false;
	}

	if (!IsCollisionEnabledForPrim(UsdPrim))
	{
		return false;
	}

	if (pxr::UsdPhysicsMeshCollisionAPI MeshCollisionAPI{UsdPrim})
	{
		pxr::TfToken Approximation(pxr::UsdPhysicsTokens->none);
		if (pxr::UsdAttribute ApproximationAttr = MeshCollisionAPI.GetApproximationAttr())
		{
			ApproximationAttr.Get(&Approximation);
		}

		if (Approximation != pxr::UsdPhysicsTokens->none)
		{
			return false;
		}
	}
	else
	{
		return false;
	}

	if (pxr::UsdAttribute PurposeAttr = UsdMesh.GetPurposeAttr())
	{
		pxr::TfToken Purpose;
		PurposeAttr.Get(&Purpose);
		if (Purpose != pxr::UsdGeomTokens->guide)
		{
			return false;
		}
	}

	return true;
}

EUsdCollisionType UsdUtils::GetCollisionApproximationType(const pxr::UsdPrim& UsdPrim)
{
	EUsdCollisionType Approximation(EUsdCollisionType::None);
	if (pxr::UsdGeomMesh UsdMesh{UsdPrim})
	{
		// Get the collision approximation type (only meshes should have UsdPhysicsMeshCollisionAPI)
		if (pxr::UsdPhysicsMeshCollisionAPI MeshCollisionAPI{UsdMesh})
		{
			pxr::TfToken ApproximationValue{pxr::UsdPhysicsTokens->none};
			pxr::UsdAttribute Attr = MeshCollisionAPI.GetApproximationAttr();
			if (Attr && Attr.Get(&ApproximationValue))
			{
				if (ApproximationValue == pxr::UsdPhysicsTokens->convexDecomposition)
				{
#if WITH_EDITOR
					return EUsdCollisionType::ConvexDecomposition;
#else
					return EUsdCollisionType::ConvexHull;
#endif
				}
				else if (ApproximationValue == pxr::UsdPhysicsTokens->convexHull)
				{
					return EUsdCollisionType::ConvexHull;
				}
				else if (ApproximationValue == pxr::UsdPhysicsTokens->boundingSphere)
				{
					return EUsdCollisionType::Sphere;
				}
				else if (ApproximationValue == pxr::UsdPhysicsTokens->boundingCube)
				{
					return EUsdCollisionType::Cube;
				}
				else if (ApproximationValue == pxr::UsdPhysicsTokens->meshSimplification)
				{
					return EUsdCollisionType::MeshSimplification;
				}
			}
		}
	}
	else
	{
		// Collision for primitives are converted to their closest approximation
		if (UsdPrim.IsA(pxr::UsdGeomTokens->Capsule) || UsdPrim.IsA(pxr::UsdGeomTokens->Capsule_1))
		{
			return EUsdCollisionType::Capsule;
		}
		else if (UsdPrim.IsA(pxr::UsdGeomTokens->Cone))
		{
			return EUsdCollisionType::CustomMesh;
		}
		else if (UsdPrim.IsA(pxr::UsdGeomTokens->Cube))
		{
			return EUsdCollisionType::Cube;
		}
		else if (UsdPrim.IsA(pxr::UsdGeomTokens->Cylinder) || UsdPrim.IsA(pxr::UsdGeomTokens->Cylinder_1))
		{
			return EUsdCollisionType::CustomMesh;
		}
		else if (UsdPrim.IsA(pxr::UsdGeomTokens->Sphere))
		{
			return EUsdCollisionType::Sphere;
		}
		else if (UsdPrim.IsA(pxr::UsdGeomTokens->Plane))
		{
			return EUsdCollisionType::CustomMesh;
		}
	}

	return EUsdCollisionType::None;
}

int32 UsdUtils::GetNumberOfLODVariants(const pxr::UsdPrim& Prim)
{
	FScopedUsdAllocs Allocs;

	const std::string LODString = UnrealIdentifiers::LOD.GetString();

	pxr::UsdVariantSets VariantSets = Prim.GetVariantSets();
	if (!VariantSets.HasVariantSet(LODString))
	{
		return 1;
	}

	return VariantSets.GetVariantSet(LODString).GetVariantNames().size();
}

bool UsdUtils::IterateLODMeshes(const pxr::UsdPrim& ParentPrim, TFunction<bool(const pxr::UsdGeomMesh& LODMesh, int32 LODIndex)> Func)
{
	if (!ParentPrim)
	{
		return false;
	}

	TOptional<FScopedUsdAllocs> Allocs;
	Allocs.Emplace();

	const std::string LODString = UnrealIdentifiers::LOD.GetString();

	pxr::UsdVariantSets VariantSets = ParentPrim.GetVariantSets();
	if (!VariantSets.HasVariantSet(LODString))
	{
		return false;
	}

	pxr::UsdVariantSet LODVariantSet = VariantSets.GetVariantSet(LODString);
	const std::string OriginalVariant = LODVariantSet.GetVariantSelection();

	pxr::UsdStageRefPtr Stage = ParentPrim.GetStage();
	pxr::UsdEditContext EditContext{Stage, Stage->GetRootLayer()};

	bool bHasValidVariant = false;
	for (const std::string& LODVariantName : VariantSets.GetVariantSet(LODString).GetVariantNames())
	{
		int32 LODIndex = UsdGeomMeshImpl::GetLODIndexFromName(LODVariantName);
		if (LODIndex == INDEX_NONE)
		{
			continue;
		}

		LODVariantSet.SetVariantSelection(LODVariantName);

		pxr::UsdGeomMesh LODMesh;
		pxr::TfToken TargetChildNameToken{LODVariantName};

		// Search for our LOD child mesh
		pxr::UsdPrimSiblingRange PrimRange = ParentPrim.GetChildren();
		for (pxr::UsdPrimSiblingRange::iterator PrimRangeIt = PrimRange.begin(); PrimRangeIt != PrimRange.end(); ++PrimRangeIt)
		{
			const pxr::UsdPrim& Child = *PrimRangeIt;
			if (pxr::UsdGeomMesh ChildMesh{Child})
			{
				if (Child.GetName() == TargetChildNameToken)
				{
					LODMesh = ChildMesh;
					// Don't break here so we can show warnings if the user has other prims here (that we may end up ignoring)
					// USD doesn't allow name collisions anyway, so there won't be any other prim named TargetChildNameToken
				}
				else
				{
					USD_LOG_USERWARNING(FText::Format(
						LOCTEXT(
							"UnexpectedLODPrim",
							"Unexpected prim '{0}' inside LOD variant '{1}'. For automatic parsing of LODs, each LOD variant should contain only a single Mesh prim named the same as the variant!"
						),
						FText::FromString(UsdToUnreal::ConvertPath(Child.GetPath())),
						FText::FromString(UsdToUnreal::ConvertString(LODVariantName))
					));
				}
			}
		}
		if (!LODMesh)
		{
			continue;
		}

		bHasValidVariant = true;

		// Reset our forced allocator as we don't know what Func expects
		Allocs.Reset();
		bool bContinue = Func(LODMesh, LODIndex);
		Allocs.Emplace();
		if (!bContinue)
		{
			break;
		}
	}

	LODVariantSet.SetVariantSelection(OriginalVariant);
	return bHasValidVariant;
}

void UsdUtils::ReplaceUnrealMaterialsWithBaked(
	const UE::FUsdStage& Stage,
	const UE::FSdfLayer& LayerToAuthorIn,
	const TMap<FString, FString>& BakedMaterials,
	bool bIsAssetLayer,
	bool bUsePayload
)
{
	FScopedUsdAllocs Allocs;

	struct FMaterialScopePrim
	{
		FMaterialScopePrim(pxr::UsdStageRefPtr ScopeStage, pxr::UsdPrim ParentPrim)
		{
			pxr::SdfPath Path = ParentPrim.GetPrimPath().AppendPath(pxr::SdfPath{"Materials"});
			Prim = ScopeStage->DefinePrim(Path, UnrealToUsd::ConvertToken(TEXT("Scope")).Get());

			// Initialize our UsedPrimNames correctly, so we can guarantee we'll never have name collisions
			if (Prim)
			{
				for (pxr::UsdPrim Child : Prim.GetFilteredChildren(pxr::UsdTraverseInstanceProxies(pxr::UsdPrimAllPrimsPredicate)))
				{
					UsedPrimNames.Add(UsdToUnreal::ConvertToken(Child.GetName()));
				}
			}
		}

		pxr::UsdPrim Prim;
		TSet<FString> UsedPrimNames;
		TMap<FString, pxr::UsdPrim> BakedFileNameToMatPrim;
	};

	TOptional<FMaterialScopePrim> StageMatScope;

	pxr::UsdStageRefPtr UsdStage{Stage};

	// Recursively traverses the stage, doing the material assignment replacements.
	// This handles Mesh prims as well as GeomSubset prims.
	// Note how we receive the stage as an argument instead of capturing it from the outer scope:
	// This ensures the inner function doesn't hold a reference to the stage
	TFunction<void(
		pxr::UsdStageRefPtr StageToTraverse,
		pxr::UsdPrim Prim,
		TOptional<FMaterialScopePrim> & MatPrimScope,
		TOptional<pxr::UsdVariantSet> OuterVariantSet
	)>
		TraverseForMaterialReplacement;
	TraverseForMaterialReplacement =
		[&TraverseForMaterialReplacement, &UsdStage, &LayerToAuthorIn, &BakedMaterials, bIsAssetLayer, bUsePayload, &StageMatScope](
			pxr::UsdStageRefPtr StageToTraverse,
			pxr::UsdPrim Prim,
			TOptional<FMaterialScopePrim>& MatPrimScope,
			TOptional<pxr::UsdVariantSet> OuterVariantSet
		)
	{
		// Recurse into children before doing anything as we may need to parse LODs
		pxr::UsdVariantSet VarSet = Prim.GetVariantSet(UnrealIdentifiers::LOD);
		std::vector<std::string> LODs = VarSet.GetVariantNames();
		if (LODs.size() > 0)
		{
			TOptional<std::string> OriginalSelection = VarSet.HasAuthoredVariantSelection() ? VarSet.GetVariantSelection() : TOptional<std::string>{};

			// Prims within variant sets can't have relationships to prims outside the scope of the prim that
			// contains the variant set itself. This means we'll need a new material scope prim if we're stepping
			// into a variant within an asset layer, so that any material proxy prims we author are contained within it.
			// Note that we only do this for asset layers: If we're parsing the root layer, any LOD variant sets we can step into
			// are brought in via references to asset files, and we know that referenced subtree only has relationships to
			// things within that same subtree ( which will be entirely brought in to the root layer ). This means we can
			// just keep inner_mat_prim_scope as None and default to using the layer's mat scope prim if we need one
			TOptional<FMaterialScopePrim> InnerMatPrimScope = bIsAssetLayer ? FMaterialScopePrim{StageToTraverse, Prim}
																			: TOptional<FMaterialScopePrim>{};

			// Switch into each of the LOD variants the prim has, and recurse into the child prims
			for (const std::string& Variant : LODs)
			{
				{
					pxr::UsdEditContext Context{StageToTraverse, StageToTraverse->GetSessionLayer()};
					VarSet.SetVariantSelection(Variant);
				}

				for (const pxr::UsdPrim& Child : Prim.GetChildren())
				{
					TraverseForMaterialReplacement(StageToTraverse, Child, InnerMatPrimScope, VarSet);
				}
			}

			// Restore the variant selection to what it originally was
			pxr::UsdEditContext Context{StageToTraverse, StageToTraverse->GetSessionLayer()};
			if (OriginalSelection.IsSet())
			{
				VarSet.SetVariantSelection(OriginalSelection.GetValue());
			}
			else
			{
				VarSet.ClearVariantSelection();
			}
		}
		else
		{
			for (const pxr::UsdPrim& Child : Prim.GetChildren())
			{
				TraverseForMaterialReplacement(StageToTraverse, Child, MatPrimScope, OuterVariantSet);
			}
		}

		// Don't try fetching attributes from the pseudoroot as we'll obviously never have a material binding here
		// and we may get some USD warnings
		if (Prim.IsPseudoRoot())
		{
			return;
		}

		std::string UnrealMaterialAttrAssetPath;
		FString UnrealMaterialPrimAssetPath;

		pxr::UsdAttribute UnrealMaterialAttr = Prim.GetAttribute(UnrealIdentifiers::MaterialAssignment);
		pxr::UsdShadeMaterial UnrealMaterial;

		pxr::UsdShadeMaterialBindingAPI MaterialBindingAPI{Prim};
		if (MaterialBindingAPI)
		{
			// We always emit UnrealMaterials with allpurpose bindings, so we can use default arguments for
			// ComputeBoundMaterial
			if (pxr::UsdShadeMaterial BoundMaterial = MaterialBindingAPI.ComputeBoundMaterial())
			{
				UnrealMaterial = BoundMaterial;

				TOptional<FString> ExistingUEAssetReference = UsdUtils::GetUnrealSurfaceOutput(UnrealMaterial.GetPrim());
				if (ExistingUEAssetReference.IsSet())
				{
					UnrealMaterialPrimAssetPath = MoveTemp(ExistingUEAssetReference.GetValue());
				}
			}
		}

		if (!UnrealMaterial && (!UnrealMaterialAttr || !UnrealMaterialAttr.Get<std::string>(&UnrealMaterialAttrAssetPath)))
		{
			return;
		}

		pxr::UsdPrim UnrealMaterialPrim = UnrealMaterial.GetPrim();

		// Prioritize the Unreal material since import will do so too
		FString UnrealMaterialAssetPath = UnrealMaterialPrimAssetPath.IsEmpty() ? UsdToUnreal::ConvertString(UnrealMaterialAttrAssetPath)
																				: UnrealMaterialPrimAssetPath;

		FString BakedFilename = BakedMaterials.FindRef(UnrealMaterialAssetPath);

		// If we have a valid UE asset but just haven't baked it, something went wrong: Just leave everything alone and abort
		if (!UnrealMaterialAssetPath.IsEmpty() && BakedFilename.IsEmpty())
		{
			return;
		}

		pxr::SdfPath UnrealMaterialAttrPath = UnrealMaterialAttr ? UnrealMaterialAttr.GetPath() : pxr::SdfPath{};
		pxr::SdfPath UnrealMaterialPrimPath = UnrealMaterial ? UnrealMaterialPrim.GetPrimPath() : pxr::SdfPath{};

		// Find out if we need to remove / author material bindings within an actual variant or outside of it, as an over.
		// We don't do this when using payloads because our override prims aren't inside the actual LOD variants : They just
		// directly override a mesh called e.g. 'LOD3' as if it's a child prim, so that the override automatically only
		// does anything when we happen to have the variant that enables the LOD3 Mesh
		const bool bAuthorInsideVariants = OuterVariantSet.IsSet() && bIsAssetLayer && !bUsePayload;

		if (bAuthorInsideVariants)
		{
			pxr::UsdVariantSet& OuterVariantSetValue = OuterVariantSet.GetValue();
			pxr::SdfPath VarPrimPath = OuterVariantSetValue.GetPrim().GetPath();
			pxr::SdfPath VarPrimPathWithVar = VarPrimPath.AppendVariantSelection(
				OuterVariantSetValue.GetName(),
				OuterVariantSetValue.GetVariantSelection()
			);

			if (UnrealMaterialAttrPath.HasPrefix(VarPrimPath))
			{
				// This builds a path like '/MyMesh{LOD=LOD0}LOD0.unrealMaterial',
				// or '/MyMesh{LOD=LOD0}LOD0/Section1.unrealMaterial'.This is required because we'll query the layer
				// for a spec path below, and this path must contain the variant selection in it, which the path returned
				// from attr.GetPath() doesn't contain
				UnrealMaterialAttrPath = UnrealMaterialAttrPath.ReplacePrefix(VarPrimPath, VarPrimPathWithVar);
			}

			if (UnrealMaterialPrimPath.HasPrefix(VarPrimPath))
			{
				UnrealMaterialPrimPath = UnrealMaterialPrimPath.ReplacePrefix(VarPrimPath, VarPrimPathWithVar);
			}
		}

		// We always want to replace things in whatever layer they were authored, and not just override with
		// a stronger opinion, so search through all sublayers to find the ones with the specs we are targeting.
		// UsedLayers here instead of layer stack because we may be exporting using payloads, and payload layers
		// don't show up on the layer stack list but do show up on the UsedLayers list.
		// We fetch these layers every time because variant switching may cause referenced layers to be dropped,
		// in case they were only used by prims inside a particular variant. This means we can also discover new
		// layers as we switch into other layers, so we really need to call this every time.
		for (pxr::SdfLayerHandle Layer : UsdStage->GetUsedLayers())
		{
			pxr::SdfAttributeSpecHandle UnrealMaterialAttrSpec = Layer->GetAttributeAtPath(UnrealMaterialAttrPath);
			pxr::SdfPrimSpecHandle UnrealMaterialPrimSpec = Layer->GetPrimAtPath(UnrealMaterialPrimPath);
			if (!UnrealMaterialAttrSpec && !UnrealMaterialPrimSpec)
			{
				continue;
			}

			pxr::UsdEditContext Context{StageToTraverse, Layer};

			// It was just an empty UE asset path, so just cancel now as our BakedFilename can't possibly be useful
			if (UnrealMaterialAssetPath.IsEmpty())
			{
				continue;
			}

			// Get the proxy prim for the material within this layer
			// (or create one outside the variant edit context)
			pxr::UsdPrim MatPrim;
			{
				pxr::UsdEditContext MatContext(StageToTraverse, pxr::SdfLayerRefPtr{LayerToAuthorIn});

				// We are already referencing an unreal material prim: Let's just augment it with a reference to the baked
				// material usd asset layer.
				// Note how this will likely not be within MatPrimScope but instead will be a child of the Mesh/GeomSubset.
				// This is fine, and in the future we'll likely exclusively do this since it will handle mesh-specific
				// material baking much better, as it will allow even having separate bakes for each LOD
				if (UnrealMaterial && UnrealMaterialPrimSpec)
				{
					MatPrim = UnrealMaterialPrim;

					bool bAlreadyHasReference = false;

					// Make sure we don't reference it more than once. This shouldn't be needed since we'll only ever run into
					// these unreal material prims once per Mesh/GeomSubset, but when creating MatScopePrims we can guarantee we
					// add a reference only once by adding it along with the Material prim creation, so it would be nice to be able to
					// guarantee it here as well
					pxr::SdfReferencesProxy ReferencesProxy = UnrealMaterialPrimSpec->GetReferenceList();
					for (const pxr::SdfReference& UsdReference : ReferencesProxy.GetAddedOrExplicitItems())
					{
						FString ReferencedFilePath = UsdToUnreal::ConvertString(UsdReference.GetAssetPath());
						FString LayerPath = UsdToUnreal::ConvertString(Layer->GetRealPath());

						if (!LayerPath.IsEmpty())
						{
							ReferencedFilePath = FPaths::ConvertRelativePathToFull(LayerPath, ReferencedFilePath);
						}

						if (FPaths::IsSamePath(ReferencedFilePath, BakedFilename))
						{
							bAlreadyHasReference = true;
							break;
						}
					}

					if (!bAlreadyHasReference)
					{
						// Without this, if we tried exporting material overrides for LOD meshes they would
						// end up outside of the variant set
						TOptional<pxr::UsdEditContext> VarContext;
						if (bAuthorInsideVariants)
						{
							VarContext.Emplace(OuterVariantSet.GetValue().GetVariantEditContext());
						}

						UE::FUsdPrim UEMatPrim{MatPrim};

						const UE::FSdfPath TargetPrimPath;
						const EReferencerTypeHandling Handling = EReferencerTypeHandling::ClearReferencerType;
						UsdUtils::AddReference(UEMatPrim, *BakedFilename, Handling, TargetPrimPath);
					}
				}
				// Need a MatScopePrim authored somewhere within this layer
				else
				{
					FMaterialScopePrim* MatPrimScopePtr = nullptr;

					if (MatPrimScope.IsSet())
					{
						MatPrimScopePtr = &MatPrimScope.GetValue();
					}
					else
					{
						// On-demand create a *single* material scope prim for the stage, if we're not inside a variant set
						if (!StageMatScope.IsSet())
						{
							// If a prim from a stage references another layer, USD's composition will effectively
							// paste the default prim of the referenced layer over the referencing prim. Because of
							// this, the subprims within the hierarchy of that default prim can't ever have
							// relationships to other prims outside that of that same hierarchy, as those prims
							// will not be present on the referencing stage at all. This is why we author our stage
							// materials scope under the default prim, and not the pseudoroot
							StageMatScope = FMaterialScopePrim{StageToTraverse, StageToTraverse->GetDefaultPrim()};
						}
						MatPrimScopePtr = &StageMatScope.GetValue();
					}

					// This should never happen
					if (!ensure(MatPrimScopePtr))
					{
						continue;
					}

					// We already have a material proxy prim for this UE material within MatPrimScope, so just reuse it
					if (pxr::UsdPrim* FoundPrim = MatPrimScopePtr->BakedFileNameToMatPrim.Find(BakedFilename))
					{
						MatPrim = *FoundPrim;
					}
					// Create a new material proxy prim for this UE material within MatPrimScope
					else
					{
						FString MatName = FPaths::GetBaseFilename(UnrealMaterialAssetPath);
						MatName = UsdToUnreal::ConvertString(pxr::TfMakeValidIdentifier(UnrealToUsd::ConvertString(*MatName).Get()));
						FString MatPrimName = UsdUnreal::ObjectUtils::GetUniqueName(MatName, MatPrimScopePtr->UsedPrimNames);
						MatPrimScopePtr->UsedPrimNames.Add(MatPrimName);

						MatPrim = StageToTraverse->DefinePrim(
							MatPrimScopePtr->Prim.GetPath().AppendChild(UnrealToUsd::ConvertToken(*MatPrimName).Get())
						);

						// We should only keep track and reuse the material proxy prims that we create within the MatPrimScope, not
						// the ones we have appropriated from within Mesh/GeomSubset from being UnrealPrims
						MatPrimScopePtr->BakedFileNameToMatPrim.Add(BakedFilename, MatPrim);

						UE::FUsdPrim UEMatPrim{MatPrim};
						UsdUtils::AddReference(UEMatPrim, *BakedFilename);
					}
				}
			}

			// Make sure we have a binding to the material prim and the material binding API
			if (pxr::UsdShadeMaterial MaterialToBind{MatPrim})
			{
				TOptional<pxr::UsdEditContext> VarContext;
				if (bAuthorInsideVariants)
				{
					VarContext.Emplace(OuterVariantSet.GetValue().GetVariantEditContext());
				}

				if (pxr::UsdShadeMaterialBindingAPI AppliedMaterialBindingAPI = pxr::UsdShadeMaterialBindingAPI::Apply(Prim))
				{
					AppliedMaterialBindingAPI.Bind(MaterialToBind);
				}
			}
		}
	};

	pxr::UsdPrim Root = Stage.GetPseudoRoot();
	TOptional<FMaterialScopePrim> Empty;
	TraverseForMaterialReplacement(UsdStage, Root, Empty, {});
}

namespace UE::UsdGeomMeshConversion::Private
{
	template<typename T>
	inline void HashArrayAttribute(FMD5& MD5, const pxr::UsdAttribute& Attribute, double TimeCode)
	{
		if (Attribute)
		{
			pxr::VtArray<T> Value;
			Attribute.Get(&Value, TimeCode);
			MD5.Update((uint8*)Value.cdata(), Value.size() * sizeof(T));
		}
	}

	template<typename T>
	inline void HashArrayPrimvar(FMD5& MD5, const pxr::UsdGeomPrimvar& Primvar, double TimeCode)
	{
		if (Primvar)
		{
			pxr::VtArray<T> Value;
			Primvar.Get(&Value, TimeCode);
			MD5.Update((uint8*)Value.cdata(), Value.size() * sizeof(T));

			pxr::VtArray<int> Indices;
			if (Primvar.GetIndices(&Indices, TimeCode))
			{
				MD5.Update((uint8*)Indices.cdata(), Indices.size() * sizeof(int));
			}
		}
	}

	inline void HashTokenAttribute(FMD5& MD5, const pxr::UsdAttribute& Attribute, double TimeCode)
	{
		if (Attribute)
		{
			pxr::TfToken Token;
			Attribute.Get(&Token, TimeCode);
			MD5.Update(reinterpret_cast<const uint8*>(Token.data()), Token.size());
		}
	}
}	 // namespace UE::UsdGeomMeshConversion::Private

FString UsdUtils::HashGeomMeshPrim(const UE::FUsdStage& Stage, const FString& PrimPath, double TimeCode)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UsdUtils::HashGeomMeshPrim);

	FMD5 MD5;

	HashGeomMeshPrim(Stage, PrimPath, TimeCode, MD5);

	uint8 Digest[16];
	MD5.Final(Digest);

	FString Hash;
	for (int32 i = 0; i < 16; ++i)
	{
		Hash += FString::Printf(TEXT("%02x"), Digest[i]);
	}
	return Hash;
}

void UsdUtils::HashGeomMeshPrim(const UE::FUsdStage& Stage, const FString& PrimPath, double TimeCode, FMD5& InOutHashState)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UsdUtils::HashGeomMeshPrim);

	using namespace pxr;
	using namespace UsdGeomMeshImpl;

	FScopedUsdAllocs Allocs;

	UsdPrim UsdPrim = pxr::UsdPrim(Stage.GetPrimAtPath(UE::FSdfPath(*PrimPath)));
	if (!UsdPrim)
	{
		return;
	}

	UsdGeomMesh UsdMesh(UsdPrim);
	if (!UsdMesh)
	{
		return;
	}

	if (pxr::UsdGeomPrimvar PointsPrimvar = pxr::UsdGeomPrimvar(UsdPrim.GetAttribute(UnrealIdentifiers::PrimvarsPoints)))
	{
		HashArrayAttribute<GfVec3f>(InOutHashState, UsdPrim.GetAttribute(UnrealIdentifiers::PrimvarsPoints), TimeCode);
	}
	else
	{
		HashArrayAttribute<GfVec3f>(InOutHashState, UsdMesh.GetPointsAttr(), TimeCode);
	}

	if (pxr::UsdGeomPrimvar NormalsPrimvar = pxr::UsdGeomPrimvar(UsdPrim.GetAttribute(UnrealIdentifiers::PrimvarsNormals)))
	{
		HashArrayAttribute<GfVec3f>(InOutHashState, UsdPrim.GetAttribute(UnrealIdentifiers::PrimvarsNormals), TimeCode);
	}
	else
	{
		HashArrayAttribute<GfVec3f>(InOutHashState, UsdMesh.GetNormalsAttr(), TimeCode);
	}

	HashArrayPrimvar<GfVec3f>(InOutHashState, UsdMesh.GetDisplayColorPrimvar(), TimeCode);
	HashArrayPrimvar<float>(InOutHashState, UsdMesh.GetDisplayOpacityPrimvar(), TimeCode);

	// Note: The actual subdivision level used is not factored in here because currently the single caller of this function is
	// GetUsdStreamDDCKey, which hashes it directly. The Static/Skeletal mesh code paths won't need to currently hash it directly
	// because the generated FMeshDescription or FSkeletalMeshImportData is hashed directly, and by then we already have subdivided
	// the mesh data, and what we end up with will naturally depend on the level of subdivision

	pxr::UsdAttribute SubdivSchemeAttr = UsdMesh.GetSubdivisionSchemeAttr();
	pxr::TfToken SubdivScheme;
	if (SubdivSchemeAttr && SubdivSchemeAttr.Get(&SubdivScheme, TimeCode) && SubdivScheme != pxr::UsdGeomTokens->none)
	{
		HashTokenAttribute(InOutHashState, UsdMesh.GetSubdivisionSchemeAttr(), TimeCode);
		HashTokenAttribute(InOutHashState, UsdMesh.GetFaceVaryingLinearInterpolationAttr(), TimeCode);
		HashTokenAttribute(InOutHashState, UsdMesh.GetTriangleSubdivisionRuleAttr(), TimeCode);
		HashArrayAttribute<int>(InOutHashState, UsdMesh.GetCornerIndicesAttr(), TimeCode);
		HashArrayAttribute<float>(InOutHashState, UsdMesh.GetCornerSharpnessesAttr(), TimeCode);
		HashArrayAttribute<int>(InOutHashState, UsdMesh.GetCreaseIndicesAttr(), TimeCode);
		HashArrayAttribute<int>(InOutHashState, UsdMesh.GetCreaseLengthsAttr(), TimeCode);
		HashArrayAttribute<float>(InOutHashState, UsdMesh.GetCreaseSharpnessesAttr(), TimeCode);
		HashArrayAttribute<int>(InOutHashState, UsdMesh.GetHoleIndicesAttr(), TimeCode);
	}

	// TODO: This is not providing render context or material purpose, so it will never consider float2f primvars
	// for the hash, which could be an issue in very exotic cases
	TArray<TUsdStore<UsdGeomPrimvar>> PrimvarsByUVIndex = UsdUtils::GetUVSetPrimvars(UsdPrim);
	for (int32 UVChannelIndex = 0; UVChannelIndex < PrimvarsByUVIndex.Num(); ++UVChannelIndex)
	{
		if (!PrimvarsByUVIndex.IsValidIndex(UVChannelIndex))
		{
			break;
		}

		HashArrayPrimvar<GfVec2f>(InOutHashState, PrimvarsByUVIndex[UVChannelIndex].Get(), TimeCode);
	}

	// The number of geomsubsets will give the upper limit of the number of sections in the mesh
	std::vector<pxr::UsdGeomSubset> GeomSubsets = pxr::UsdShadeMaterialBindingAPI(UsdPrim).GetMaterialBindSubsets();
	const int32 NumGeomSubsets = static_cast<int32>(GeomSubsets.size());
	InOutHashState.Update((uint8*)&NumGeomSubsets, sizeof(NumGeomSubsets));
}

bool UsdUtils::GetPointInstancerTransforms(
	const FUsdStageInfo& StageInfo,
	const pxr::UsdGeomPointInstancer& PointInstancer,
	const int32 ProtoIndex,
	pxr::UsdTimeCode EvalTime,
	TArray<FTransform>& OutInstanceTransforms
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(GetPointInstancerTransforms);

	if (!PointInstancer)
	{
		return false;
	}

	FScopedUsdAllocs UsdAllocs;

	const pxr::VtArray<int> UsdProtoIndices = UsdUtils::GetUsdValue<pxr::VtArray<int>>(PointInstancer.GetProtoIndicesAttr(), EvalTime);

	pxr::VtMatrix4dArray UsdInstanceTransforms;

	// We don't want the prototype root prim's transforms to be included in these, as they'll already be baked into the meshes themselves
	if (!PointInstancer.ComputeInstanceTransformsAtTime(&UsdInstanceTransforms, EvalTime, EvalTime, pxr::UsdGeomPointInstancer::ExcludeProtoXform))
	{
		return false;
	}

	int32 Index = 0;

	const int32 NumInstances = GMaxInstancesPerPointInstancer >= 0
								   ? FMath::Min(static_cast<int32>(UsdInstanceTransforms.size()), GMaxInstancesPerPointInstancer)
								   : static_cast<int32>(UsdInstanceTransforms.size());

	{
		FScopedUnrealAllocs UnrealAllocs;

		OutInstanceTransforms.Reset(NumInstances);

		for (const pxr::GfMatrix4d& UsdMatrix : UsdInstanceTransforms)
		{
			if (Index == NumInstances)
			{
				break;
			}

			if (UsdProtoIndices[Index] == ProtoIndex)
			{
				OutInstanceTransforms.Add(UsdToUnreal::ConvertMatrix(StageInfo, UsdMatrix));
			}

			++Index;
		}
	}

	return true;
}

void UsdUtils::GetPointInstancerInstancesData(const pxr::UsdPrim& Prim, FUsdPointInstancerInstancesData& InstancesData)
{
	FScopedUsdAllocs Allocs;

	pxr::UsdGeomPointInstancer PointInstancer(Prim);
	if (!PointInstancer)
	{
		return;
	}

	pxr::SdfPathVector PrototypePaths;
	if (!PointInstancer.GetPrototypesRel().GetTargets(&PrototypePaths))
	{
		return;
	}

	UE::FUsdStage Stage(Prim.GetStage());
	const FUsdStageInfo StageInfo(Prim.GetStage());

	InstancesData.Prototypes.SetNum(PrototypePaths.size());

	for (int32 PrototypeIndex = 0; PrototypeIndex < PrototypePaths.size(); ++PrototypeIndex)
	{
		const pxr::SdfPath& PrototypePath = PrototypePaths[PrototypeIndex];
		InstancesData.Prototypes[PrototypeIndex].PathString = UsdToUnreal::ConvertPath(PrototypePath);

		UsdUtils::GetPointInstancerTransforms(StageInfo, PointInstancer, PrototypeIndex, UsdUtils::GetEarliestTimeCode(), InstancesData.Prototypes[PrototypeIndex].InstanceTransforms);
	}
}

bool UsdUtils::IsAnimatedMesh(const pxr::UsdPrim& Prim)
{
	FScopedUsdAllocs UsdAllocs;

	pxr::UsdGeomMesh Mesh = pxr::UsdGeomMesh(Prim);
	if (!Mesh)
	{
		return false;
	}

	bool bHasAttributesTimeSamples = false;
	{
		constexpr bool bIncludeInherited = false;
		pxr::TfTokenVector GeomMeshAttributeNames = pxr::UsdGeomMesh::GetSchemaAttributeNames(bIncludeInherited);
		pxr::TfTokenVector GeomPointBasedAttributeNames = pxr::UsdGeomPointBased::GetSchemaAttributeNames(bIncludeInherited);

		GeomMeshAttributeNames.reserve(GeomMeshAttributeNames.size() + GeomPointBasedAttributeNames.size());
		GeomMeshAttributeNames.insert(GeomMeshAttributeNames.end(), GeomPointBasedAttributeNames.begin(), GeomPointBasedAttributeNames.end());

		for (const pxr::TfToken& AttributeName : GeomMeshAttributeNames)
		{
			const pxr::UsdAttribute& Attribute = Prim.GetAttribute(AttributeName);

			if (Attribute && Attribute.ValueMightBeTimeVarying())
			{
				bHasAttributesTimeSamples = true;
				break;
			}
		}
	}

	return bHasAttributesTimeSamples;
}

UsdUtils::EMeshTopologyVariance UsdUtils::GetMeshTopologyVariance(const pxr::UsdGeomMesh& UsdMesh)
{
	FScopedUsdAllocs UsdAllocs;

	pxr::UsdAttribute Points = UsdMesh.GetPointsAttr();
	if (!Points)
	{
		return EMeshTopologyVariance::Constant;
	}

	pxr::UsdAttribute FaceCountsAttribute = UsdMesh.GetFaceVertexCountsAttr();
	if (!FaceCountsAttribute)
	{
		return EMeshTopologyVariance::Constant;
	}

	pxr::UsdAttribute FaceVertexIndicesAttribute = UsdMesh.GetFaceVertexIndicesAttr();
	if (!FaceVertexIndicesAttribute)
	{
		return EMeshTopologyVariance::Constant;
	}

	if (!FaceVertexIndicesAttribute.ValueMightBeTimeVarying() && !FaceCountsAttribute.ValueMightBeTimeVarying())
	{
		if (!Points.ValueMightBeTimeVarying())
		{
			return EMeshTopologyVariance::Constant;
		}
		else
		{
			return EMeshTopologyVariance::Homogenous;
		}
	}
	else
	{
		return EMeshTopologyVariance::Heterogenous;
	}
}

UsdUtils::EMeshTopologyVariance UsdUtils::GetMeshTopologyVariance(const UE::FUsdPrim& UsdPrim)
{
	pxr::UsdGeomMesh MeshPrim(UsdPrim);
	if (!MeshPrim)
	{
		return EMeshTopologyVariance::Constant;
	}
	return GetMeshTopologyVariance(MeshPrim);
}

uint64 UsdUtils::GetGprimVertexCount(const pxr::UsdGeomGprim& Gprim, double TimeCode)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UsdUtils::GetGprimVertexCount);

	if (pxr::UsdGeomMesh Mesh{Gprim})
	{
		if (pxr::UsdAttribute Points = Mesh.GetPointsAttr())
		{
			pxr::VtArray<pxr::GfVec3f> PointsArray;
			Points.Get(&PointsArray, pxr::UsdTimeCode(TimeCode));
			return PointsArray.size();
		}
	}
	else if (pxr::UsdGeomCapsule Capsule{Gprim})
	{
		// These numbers come from inspecting USD's implicitSurfaceMeshUtils.cpp
		// and comparing with the generated UStaticMesh vertex counts.
		// In practice it doesn't matter much though: These small Gprims are likely
		// never going to significantly affect whether a subtree should collapse or not
		return 82;
	}
	else if (pxr::UsdGeomCapsule_1 Capsule_1{Gprim})
	{
		return pxr::GeomUtilCapsuleMeshGenerator::ComputeNumPoints(
			UsdGeomMeshImpl::DefaultCapsuleNumRadialSegments,
			UsdGeomMeshImpl::DefaultCapsuleNumCapAxialSegments
		);
	}
	else if (pxr::UsdGeomCone Cone{Gprim})
	{
		return 31;
	}
	else if (pxr::UsdGeomCube Cube{Gprim})
	{
		return 8;
	}
	else if (pxr::UsdGeomCylinder Cylinder{Gprim})
	{
		return 42;
	}
	else if (pxr::UsdGeomCylinder_1 Cylinder_1{Gprim})
	{
		return pxr::GeomUtilCylinderMeshGenerator::ComputeNumPoints(UsdGeomMeshImpl::DefaultCylinderNumRadialSegments);
	}
	else if (pxr::UsdGeomSphere Sphere{Gprim})
	{
		return 92;
	}
	else if (pxr::UsdGeomPlane Plane{Gprim})
	{
		return 4;
	}

	return 0;
}

void UsdUtils::AuthorIdentityTransformGprimAttributes(const pxr::UsdPrim& UsdPrim, bool bDefaultValues, bool bTimeSampleValues)
{
	using namespace UE::UsdGeomMeshConversion::Private;

	pxr::UsdGeomGprim Gprim{UsdPrim};
	if (!Gprim)
	{
		return;
	}

	FScopedUsdAllocs Allocs;

	// We can't just "clear" these opinions because we may cause some weaker opinion
	// to pop up, and the caller will likely be relying on this function to make sure our
	// prim has attributes in such a way that its "primitive transform" is the identity.
	// In other words, after we call this function on UsdPrim, calling
	// ConvertGeomPrimitiveTransform on the same prim should generate the identity transform.
	auto SetAttrValue = [bDefaultValues, bTimeSampleValues](const pxr::UsdAttribute& Attr, auto Value)
	{
		if (!Attr)
		{
			return;
		}

		if (bDefaultValues)
		{
			Attr.Set(Value);
		}

		if (bTimeSampleValues)
		{
			UsdUtils::ClearAllTimeSamples(Attr);

			// Ideally we'd use pxr::UsdTimeCode::EarliestTime() but that seems to be -DBL_MAX, which
			// could look weird to a user when written on the USD file. Since this is going to be the
			// only timeSample it doesn't really matter anyway
			const pxr::UsdTimeCode TimeCode{0.0};
			Attr.Set(Value, TimeCode);
		}
	};	

	// In here we must author the attribute values that cause ConvertGeomPrimitive
	// to generate meshes in the [-0.5, 0.5] bounding box, as that will correspond to the
	// identity "primitive transform".
	// Note that these values *do not* correspond to the fallback values for the attributes.
	// For whatever reason the attribute fallback values all lead to a scaling factor of 2 instead.
	// If we want our primitives to end up with a scale of 2 when writing out to USD however,
	// we will put the scale of 2 directly on the Xform/component transform instead, and with this
	// function have the attributes generate a scale of 1 instead.
	if (pxr::UsdGeomCapsule Capsule = pxr::UsdGeomCapsule{UsdPrim})
	{
		SetAttrValue(Capsule.CreateRadiusAttr(), UsdGeomMeshImpl::DefaultCapsuleMeshRadius);
		SetAttrValue(Capsule.CreateHeightAttr(), UsdGeomMeshImpl::DefaultCapsuleMeshHeight);
	}
	else if (pxr::UsdGeomCapsule_1 Capsule_1 = pxr::UsdGeomCapsule_1{UsdPrim})
	{
		pxr::UsdAttribute RadiusTopAttr = Capsule_1.GetRadiusTopAttr();

		// For the two radii primitives we need to preserve the ratio between the radii to retain the right shape,
		// but we still need to restore RadiusTop to its default value as we'll author our primitive transform based on that fact
		const double RadiusTop = GetDefaultOrEarliest<double>(Capsule_1.GetRadiusTopAttr());
		const double RadiusBottom = GetDefaultOrEarliest<double>(Capsule_1.GetRadiusBottomAttr());
		const double RadiusRatio = RadiusTop != 0.0f ? (RadiusBottom / RadiusTop) : 1.0f;
		const double NewRadiusTop = UsdGeomMeshImpl::DefaultCapsuleMeshRadius;

		SetAttrValue(Capsule_1.CreateRadiusTopAttr(), NewRadiusTop);
		SetAttrValue(Capsule_1.CreateRadiusBottomAttr(), NewRadiusTop * RadiusRatio);
		SetAttrValue(Capsule_1.CreateHeightAttr(), UsdGeomMeshImpl::DefaultCapsuleMeshHeight);
	}
	else if (pxr::UsdGeomCone Cone = pxr::UsdGeomCone{UsdPrim})
	{
		SetAttrValue(Cone.CreateRadiusAttr(), UsdGeomMeshImpl::DefaultConeMeshRadius);
		SetAttrValue(Cone.CreateHeightAttr(), UsdGeomMeshImpl::DefaultConeMeshHeight);
	}
	else if (pxr::UsdGeomCube Cube = pxr::UsdGeomCube{UsdPrim})
	{
		SetAttrValue(Cube.CreateSizeAttr(), UsdGeomMeshImpl::DefaultCubeMeshSize);
	}
	else if (pxr::UsdGeomCylinder Cylinder = pxr::UsdGeomCylinder{UsdPrim})
	{
		SetAttrValue(Cylinder.CreateRadiusAttr(), UsdGeomMeshImpl::DefaultCylinderMeshRadius);
		SetAttrValue(Cylinder.CreateHeightAttr(), UsdGeomMeshImpl::DefaultCylinderMeshHeight);
	}
	else if (pxr::UsdGeomCylinder_1 Cylinder_1 = pxr::UsdGeomCylinder_1{UsdPrim})
	{
		// See the Capsule_1 case
		const double RadiusTop = GetDefaultOrEarliest<double>(Cylinder_1.GetRadiusTopAttr());
		const double RadiusBottom = GetDefaultOrEarliest<double>(Cylinder_1.GetRadiusBottomAttr());
		const double RadiusRatio = RadiusTop != 0.0f ? (RadiusBottom / RadiusTop) : 1.0f;
		const double NewRadiusTop = UsdGeomMeshImpl::DefaultCylinderMeshRadius;

		SetAttrValue(Cylinder_1.CreateRadiusTopAttr(), NewRadiusTop);
		SetAttrValue(Cylinder_1.CreateRadiusBottomAttr(), NewRadiusTop * RadiusRatio);
		SetAttrValue(Cylinder_1.CreateHeightAttr(), UsdGeomMeshImpl::DefaultCylinderMeshHeight);
	}
	else if (pxr::UsdGeomSphere Sphere = pxr::UsdGeomSphere{UsdPrim})
	{
		SetAttrValue(Sphere.CreateRadiusAttr(), UsdGeomMeshImpl::DefaultSphereMeshRadius);
	}
	else if (pxr::UsdGeomPlane Plane = pxr::UsdGeomPlane{UsdPrim})
	{
		SetAttrValue(Plane.CreateWidthAttr(), UsdGeomMeshImpl::DefaultPlaneMeshWidth);
		SetAttrValue(Plane.CreateLengthAttr(), UsdGeomMeshImpl::DefaultPlaneMeshLength);
	}
}

void UsdUtils::GetAnimatedMeshTimeCodes(const UE::FUsdStage& Stage, const FString& PrimPath, int32& OutStartFrame, int32& OutEndFrame)
{
	if (!Stage || PrimPath.IsEmpty())
	{
		return;
	}

	UE::FUsdPrim UsdPrim = (Stage.GetPrimAtPath(UE::FSdfPath(*PrimPath)));
	if (!UsdPrim)
	{
		return;
	}

	TArray<UE::FUsdAttribute> Attributes = UsdUtils::GetAnimatedMeshAttributes(UsdPrim);

	TOptional<double> MinStartTimeCode;
	TOptional<double> MaxEndTimeCode;

	for (const UE::FUsdAttribute& Attribute : Attributes)
	{
		TArray<double> TimeSamples;
		if (Attribute.GetTimeSamples(TimeSamples) && TimeSamples.Num() > 0)
		{
			MinStartTimeCode = FMath::Min(MinStartTimeCode.Get(TNumericLimits<double>::Max()), TimeSamples[0]);
			MaxEndTimeCode = FMath::Max(MaxEndTimeCode.Get(TNumericLimits<double>::Lowest()), TimeSamples[TimeSamples.Num() - 1]);
		}
	}

	if (MinStartTimeCode.IsSet() && MaxEndTimeCode.IsSet())
	{
		OutStartFrame = FMath::FloorToInt(MinStartTimeCode.GetValue());
		OutEndFrame = FMath::CeilToInt(MaxEndTimeCode.GetValue());
	}
}

TArray<UE::FUsdAttribute> UsdUtils::GetAnimatedMeshAttributes(const pxr::UsdPrim& Prim)
{
	FScopedUsdAllocs Allocs;

	if (!Prim)
	{
		return {};
	}

	constexpr bool bIncludeInherited = false;
	pxr::TfTokenVector GeomMeshAttributeNames = pxr::UsdGeomMesh::GetSchemaAttributeNames(bIncludeInherited);
	pxr::TfTokenVector GeomPointBasedAttributeNames = pxr::UsdGeomPointBased::GetSchemaAttributeNames(bIncludeInherited);

	GeomMeshAttributeNames.reserve(GeomMeshAttributeNames.size() + GeomPointBasedAttributeNames.size());
	GeomMeshAttributeNames.insert(GeomMeshAttributeNames.end(), GeomPointBasedAttributeNames.begin(), GeomPointBasedAttributeNames.end());

	TArray<UE::FUsdAttribute> AnimatedAttributes;
	AnimatedAttributes.Reserve(GeomMeshAttributeNames.size());

	for (const pxr::TfToken& AttributeName : GeomMeshAttributeNames)
	{
		if (const pxr::UsdAttribute& Attribute = Prim.GetAttribute(AttributeName))
		{
			std::vector<double> TimeSamples;
			if (Attribute.GetTimeSamples(&TimeSamples) && TimeSamples.size() > 0)
			{
				AnimatedAttributes.Add(UE::FUsdAttribute{Attribute});
			}
		}
	}

	return AnimatedAttributes;
}

#undef LOCTEXT_NAMESPACE

#endif	  // #if USE_USD_SDK
