// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "USDGroomConversion.h"

#include "GroomCacheData.h"
#include "HairDescription.h"
#include "USDMemory.h"
#include "USDPrimConversion.h"
#include "USDTypesConversion.h"
#include "UsdWrappers/UsdPrim.h"

#include "USDIncludesStart.h"
#include "pxr/usd/usd/prim.h"
#include "pxr/usd/usdGeom/basisCurves.h"
#include "pxr/usd/usdGeom/primvar.h"
#include "pxr/usd/usdGeom/primvarsAPI.h"
#include "pxr/usd/usdGeom/xform.h"
#include "USDIncludesEnd.h"

namespace UE::UsdGroomConversion::Private
{
	bool IsAttributeValid(const FString& AttributeName)
	{
		// Ignore the attributes that aren't prefixed with groom_
		return AttributeName.StartsWith(TEXT("groom_"));
	}

	// Default USD to UE type converter
	template<typename TAttributeType>
	struct TConverter
	{
		template<typename InType>
		static TAttributeType ConvertType(const InType& In)
		{
			return In;
		}

		static TAttributeType DefaultValue()
		{
			return TAttributeType();
		}
	};

	// String to FName converter
	template<>
	struct TConverter<FName>
	{
		template<typename InType>
		static FName ConvertType(const InType& In)
		{
			return UsdToUnreal::ConvertName(In);
		}

		static FName DefaultValue()
		{
			return NAME_None;
		}
	};

	// GfVec2f/Float2 to FVector2f converter
	template<>
	struct TConverter<FVector2f>
	{
		template<typename InType>
		static FVector2f ConvertType(const InType& In)
		{
			return FVector2f(In[0], In[1]);
		}

		static FVector2f DefaultValue()
		{
			return FVector2f::ZeroVector;
		}
	};

	// GfVec3f/Float3 to FVector3f converter
	template<>
	struct TConverter<FVector3f>
	{
		template<typename InType>
		static FVector3f ConvertType(const InType& In)
		{
			return FVector3f(In[0], In[1], In[2]);
		}

		static FVector3f DefaultValue()
		{
			return FVector3f::ZeroVector;
		}
	};

	/** Convert the given primvar to hair attribute in the proper scope */
	template<typename USDType, typename UEType>
	void ConvertPrimvar(
		const pxr::UsdGeomPrimvar& Primvar,
		const pxr::UsdTimeCode& TimeCode,
		int32 StartStrandID,
		int32 NumStrands,
		int32 StartVertexID,
		int32 NumVertices,
		FHairDescription& HairDescription
	)
	{
		FString PrimvarName = UsdToUnreal::ConvertToken(Primvar.GetPrimvarName());
		FName AttributeName(*PrimvarName);

		// ComputeFlattened can handle indexed primvar; for non-indexed primvar, this is equivalent to Get
		pxr::VtArray<USDType> Values;
		Primvar.ComputeFlattened(&Values, TimeCode);
		int32 NumValues = Values.size();

		if (NumValues == 0)
		{
			return;
		}

		// Check the supported attribute scopes by checking the primvar interpolation
		// and the number of values as a fallback
		pxr::TfToken Interpolation = Primvar.GetInterpolation();
		if (Interpolation == pxr::UsdGeomTokens->uniform || Interpolation == pxr::UsdGeomTokens->constant || NumValues == NumStrands)
		{
			if (Interpolation == pxr::UsdGeomTokens->uniform && NumValues != NumStrands)
			{
				// Attribute has a mismatch between the expected and actual number of values so skip it
				return;
			}

			TStrandAttributesRef<UEType> StrandAttributeRef = HairDescription.StrandAttributes().GetAttributesRef<UEType>(AttributeName);
			if (!StrandAttributeRef.IsValid())
			{
				HairDescription.StrandAttributes().RegisterAttribute<UEType>(AttributeName, 1, TConverter<UEType>::DefaultValue());
				StrandAttributeRef = HairDescription.StrandAttributes().GetAttributesRef<UEType>(AttributeName);
			}

			// Constant scope is converted to uniform scope by setting the single value over all strands
			bool bIsConstantValue = Interpolation == pxr::UsdGeomTokens->constant;
			for (int32 StrandIndex = 0; StrandIndex < NumStrands; ++StrandIndex)
			{
				StrandAttributeRef[FStrandID(StartStrandID + StrandIndex)] = TConverter<UEType>::ConvertType(
					Values[bIsConstantValue ? 0 : StrandIndex]
				);
			}
		}
		else if (Interpolation == pxr::UsdGeomTokens->vertex || NumValues == NumVertices)
		{
			TVertexAttributesRef<UEType> VertexAttributeRef = HairDescription.VertexAttributes().GetAttributesRef<UEType>(AttributeName);
			if (!VertexAttributeRef.IsValid())
			{
				HairDescription.VertexAttributes().RegisterAttribute<UEType>(AttributeName, 1, TConverter<UEType>::DefaultValue());
				VertexAttributeRef = HairDescription.VertexAttributes().GetAttributesRef<UEType>(AttributeName);
			}

			for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
			{
				VertexAttributeRef[FVertexID(StartVertexID + VertexIndex)] = TConverter<UEType>::ConvertType(Values[VertexIndex]);
			}
		}
	}

	/** Convert the given primvars to hair attributes */
	void ConvertPrimvars(
		const std::vector<pxr::UsdGeomPrimvar>& Attributes,
		const pxr::UsdTimeCode& TimeCode,
		int32 StartStrandID,
		int32 NumStrands,
		int32 StartVertexID,
		int32 NumVertices,
		FHairDescription& HairDescription
	)
	{
		for (const pxr::UsdGeomPrimvar& Primvar : Attributes)
		{
			// Process only groom attributes
			FString UEPrimvarName = UsdToUnreal::ConvertToken(Primvar.GetPrimvarName());
			if (!IsAttributeValid(UEPrimvarName))
			{
				continue;
			}

			// Dispatch the USD to UE conversion by the primvar scalar type
			pxr::SdfValueTypeName ScalarTypeName = Primvar.GetTypeName().GetScalarType();
			if (ScalarTypeName == pxr::SdfValueTypeNames->Int)
			{
				ConvertPrimvar<int, int>(Primvar, TimeCode, StartStrandID, NumStrands, StartVertexID, NumVertices, HairDescription);
			}
			else if (ScalarTypeName == pxr::SdfValueTypeNames->Float)
			{
				ConvertPrimvar<float, float>(Primvar, TimeCode, StartStrandID, NumStrands, StartVertexID, NumVertices, HairDescription);
			}
			else if (ScalarTypeName == pxr::SdfValueTypeNames->Vector3f || ScalarTypeName == pxr::SdfValueTypeNames->Float3)
			{
				ConvertPrimvar<pxr::GfVec3f, FVector3f>(Primvar, TimeCode, StartStrandID, NumStrands, StartVertexID, NumVertices, HairDescription);
			}
			else if (ScalarTypeName == pxr::SdfValueTypeNames->TexCoord2f || ScalarTypeName == pxr::SdfValueTypeNames->Float2)
			{
				ConvertPrimvar<pxr::GfVec2f, FVector2f>(Primvar, TimeCode, StartStrandID, NumStrands, StartVertexID, NumVertices, HairDescription);
			}
			else if (ScalarTypeName == pxr::SdfValueTypeNames->Double2)
			{
				ConvertPrimvar<pxr::GfVec2d, FVector2f>(Primvar, TimeCode, StartStrandID, NumStrands, StartVertexID, NumVertices, HairDescription);
			}
			else if (ScalarTypeName == pxr::SdfValueTypeNames->String)
			{
				ConvertPrimvar<std::string, FName>(Primvar, TimeCode, StartStrandID, NumStrands, StartVertexID, NumVertices, HairDescription);
			}
		}
	}

	/** Convert the given USD attribute to groom-scope hair attribute (constant value that applies to the whole groom) */
	template<typename USDType, typename UEType>
	void ConvertGroomAttribute(const pxr::UsdAttribute& Attribute, const pxr::UsdTimeCode& TimeCode, FHairDescription& HairDescription)
	{
		FString PropertyName = UsdToUnreal::ConvertToken(Attribute.GetBaseName());
		FName AttributeName(*PropertyName);

		USDType Value;
		if (!Attribute.Get(&Value, TimeCode))
		{
			return;
		}

		TGroomAttributesRef<UEType> GroomAttributeRef = HairDescription.GroomAttributes().GetAttributesRef<UEType>(AttributeName);
		if (!GroomAttributeRef.IsValid())
		{
			HairDescription.GroomAttributes().RegisterAttribute<UEType>(AttributeName, 1, TConverter<UEType>::DefaultValue());
			GroomAttributeRef = HairDescription.GroomAttributes().GetAttributesRef<UEType>(AttributeName);
		}

		GroomAttributeRef[0] = TConverter<UEType>::ConvertType(Value);
	}

	/** Convert the given attributes to groom-scope hair attributes (constant value that applies to the whole groom) */
	void ConvertGroomAttributes(const std::vector<pxr::UsdProperty>& Properties, const pxr::UsdTimeCode& TimeCode, FHairDescription& HairDescription)
	{
		for (const pxr::UsdProperty& Property : Properties)
		{
			pxr::UsdAttribute Attribute = Property.As<pxr::UsdAttribute>();
			if (!Attribute || !Attribute.HasValue())
			{
				continue;
			}

			// Process only groom attributes
			FString PropertyName = UsdToUnreal::ConvertToken(Property.GetBaseName());
			if (!IsAttributeValid(PropertyName))
			{
				continue;
			}

			// Dispatch the USD to UE conversion by the attribute scalar type
			pxr::SdfValueTypeName ScalarTypeName = Attribute.GetTypeName().GetScalarType();
			if (ScalarTypeName == pxr::SdfValueTypeNames->Int)
			{
				ConvertGroomAttribute<int, int>(Attribute, TimeCode, HairDescription);
			}
			else if (ScalarTypeName == pxr::SdfValueTypeNames->Float)
			{
				ConvertGroomAttribute<float, float>(Attribute, TimeCode, HairDescription);
			}
			else if (ScalarTypeName == pxr::SdfValueTypeNames->Vector3f || ScalarTypeName == pxr::SdfValueTypeNames->Float3)
			{
				ConvertGroomAttribute<pxr::GfVec3f, FVector3f>(Attribute, TimeCode, HairDescription);
			}
			else if (ScalarTypeName == pxr::SdfValueTypeNames->String)
			{
				ConvertGroomAttribute<std::string, FName>(Attribute, TimeCode, HairDescription);
			}
		}
	}

	/** Fill the AnimInfo with the time data for the given attribute, if it's animated */
	void UpdateAnimInfo(const pxr::UsdAttribute& Attr, EGroomCacheAttributes Flag, FGroomAnimationInfo* AnimInfo)
	{
		if (AnimInfo)
		{
			std::vector<double> TimeSamples;
			if (Attr.GetTimeSamples(&TimeSamples))
			{
				if (TimeSamples.size() > 1)
				{
					AnimInfo->Attributes = AnimInfo->Attributes | Flag;
					AnimInfo->StartFrame = FMath::Min(AnimInfo->StartFrame, TimeSamples[0]);
					AnimInfo->EndFrame = FMath::Max(AnimInfo->EndFrame, TimeSamples[TimeSamples.size() - 1]);
					AnimInfo->NumFrames = FMath::Max(AnimInfo->NumFrames, uint32(AnimInfo->EndFrame - AnimInfo->StartFrame + 1));
				}
			}
		}
	}
}	 // namespace UE::UsdGroomConversion::Private

namespace UsdToUnreal
{
	bool ConvertGroomHierarchy(
		const pxr::UsdPrim& Prim,
		const pxr::UsdTimeCode& TimeCode,
		const FTransform& ParentTransform,
		FHairDescription& HairDescription,
		FGroomAnimationInfo* AnimInfo,
		bool bCheckGroomAttributes
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UsdToUnreal::ConvertGroomHierarchy);

		FScopedUsdAllocs Allocs;

		// ref. AlembicHairTranslator ParseObject

		bool bSuccess = true;
		FTransform TransformToPropagate = ParentTransform;
		if (pxr::UsdGeomBasisCurves Curves = pxr::UsdGeomBasisCurves(Prim))
		{
			// Get curve and curve vextex count and vertex positions, the minimal data set required
			const int32 NumCurves = Curves.GetCurveCount(TimeCode);
			if (NumCurves == 0)
			{
				return false;
			}

			pxr::UsdAttribute NumVerticesAttr = Curves.GetCurveVertexCountsAttr();
			pxr::VtArray<int> NumVertices;
			if (!NumVerticesAttr || !NumVerticesAttr.Get(&NumVertices, TimeCode) || NumVertices.empty())
			{
				return false;
			}

			pxr::UsdAttribute PointsAttr = Curves.GetPointsAttr();
			pxr::VtArray<pxr::GfVec3f> Points;
			if (!PointsAttr || !PointsAttr.Get(&Points, TimeCode) || Points.empty())
			{
				return false;
			}

			UE::UsdGroomConversion::Private::UpdateAnimInfo(PointsAttr, EGroomCacheAttributes::Position, AnimInfo);

			// Get the starting strand and vertex IDs for this group of Curves
			int32 StartStrandID = HairDescription.GetNumStrands();
			int32 StartVertexID = HairDescription.GetNumVertices();

			bool bResetXformStack = false;
			FTransform PrimTransform = FTransform::Identity;
			bool bConverted = UsdToUnreal::ConvertXformable(Prim.GetStage(), Curves, PrimTransform, TimeCode.GetValue(), &bResetXformStack);
			if (bConverted)
			{
				TransformToPropagate = bResetXformStack ? PrimTransform : ParentTransform * PrimTransform;
			}

			uint32 PointIndex = 0;
			const FUsdStageInfo StageInfo(Prim.GetStage());
			for (int32 CurveIndex = 0; CurveIndex < NumCurves; ++CurveIndex)
			{
				const int CurveNumVertices = NumVertices[CurveIndex];

				FStrandID StrandID = HairDescription.AddStrand();

				SetHairStrandAttribute(HairDescription, StrandID, HairAttribute::Strand::VertexCount, CurveNumVertices);

				for (int32 CurveVertexIndex = 0; CurveVertexIndex < CurveNumVertices; ++CurveVertexIndex, ++PointIndex)
				{
					FVertexID VertexID = HairDescription.AddVertex();

					const pxr::GfVec3f& Point = Points[PointIndex];

					FVector Position = UsdToUnreal::ConvertVector(StageInfo, Point);
					FVector ConvertedPosition = TransformToPropagate.TransformPosition(Position);
					SetHairVertexAttribute(HairDescription, VertexID, HairAttribute::Vertex::Position, FVector3f(ConvertedPosition));
				}
			}

			// Set width values. Width attribute is not a generic primvar so must be processed separately
			int NumPoints = Points.size();
			pxr::UsdAttribute Widths = Curves.GetWidthsAttr();
			pxr::VtArray<float> WidthsArray;
			if (Widths && Widths.Get(&WidthsArray, TimeCode) && !WidthsArray.empty())
			{
				UE::UsdGroomConversion::Private::UpdateAnimInfo(Widths, EGroomCacheAttributes::Width, AnimInfo);

				// Determine the scope of the Widths attribute
				pxr::TfToken Interpolation = Curves.GetWidthsInterpolation();

				const double UEMetersPerUnit = 0.01;
				const float Scale = !FMath::IsNearlyEqual(StageInfo.MetersPerUnit, UEMetersPerUnit)
										? (float)(StageInfo.MetersPerUnit / UEMetersPerUnit)
										: 1.0f;
				if (Interpolation == pxr::UsdGeomTokens->uniform)
				{
					TStrandAttributesRef<float> WidthStrandAttributeRef = HairDescription.StrandAttributes().GetAttributesRef<float>(
						HairAttribute::Strand::Width
					);
					if (!WidthStrandAttributeRef.IsValid())
					{
						HairDescription.StrandAttributes().RegisterAttribute<float>(HairAttribute::Strand::Width);
						WidthStrandAttributeRef = HairDescription.StrandAttributes().GetAttributesRef<float>(HairAttribute::Strand::Width);
					}

					for (int32 Index = 0; Index < NumCurves; ++Index)
					{
						WidthStrandAttributeRef[FStrandID(StartStrandID + Index)] = WidthsArray[Index] * Scale;
					}
				}
				else if (Interpolation == pxr::UsdGeomTokens->vertex)
				{
					TVertexAttributesRef<float> VertexAttributeRef = HairDescription.VertexAttributes().GetAttributesRef<float>(
						HairAttribute::Vertex::Width
					);
					if (!VertexAttributeRef.IsValid())
					{
						HairDescription.VertexAttributes().RegisterAttribute<float>(HairAttribute::Vertex::Width);
						VertexAttributeRef = HairDescription.VertexAttributes().GetAttributesRef<float>(HairAttribute::Vertex::Width);
					}

					for (int32 Index = 0; Index < NumPoints; ++Index)
					{
						VertexAttributeRef[FVertexID(StartVertexID + Index)] = WidthsArray[Index] * Scale;
					}
				}
				else if (bCheckGroomAttributes && Interpolation == pxr::UsdGeomTokens->constant)
				{
					// This is a more compact way to store a constant width
					TGroomAttributesRef<float> GroomAttributeRef = HairDescription.GroomAttributes().GetAttributesRef<float>(HairAttribute::Groom::Width);
					if (!GroomAttributeRef.IsValid())
					{
						HairDescription.GroomAttributes().RegisterAttribute<float>(HairAttribute::Groom::Width);
						GroomAttributeRef = HairDescription.GroomAttributes().GetAttributesRef<float>(HairAttribute::Groom::Width);
					}

					GroomAttributeRef[0] = WidthsArray[0] * Scale;
				}
			}

			// Extract the groom attributes (at vertex and strand scope) from the generic primvars
			pxr::UsdGeomPrimvarsAPI PrimvarsAPI(Prim);
			std::vector<pxr::UsdGeomPrimvar> Primvars = PrimvarsAPI.GetPrimvarsWithValues();
			if (!Primvars.empty())
			{
				UE::UsdGroomConversion::Private::ConvertPrimvars(
					Primvars,
					TimeCode,
					StartStrandID,
					NumCurves,
					StartVertexID,
					NumPoints,
					HairDescription
				);

				// Check if color attribute is animated for GroomCache
				// Other animated primvars/attributes are not supported by the GroomCache
				static const pxr::TfToken GroomColorToken(UnrealToUsd::ConvertName(HairAttribute::Vertex::Color).Get());
				pxr::UsdGeomPrimvar ColorPrimvar = PrimvarsAPI.GetPrimvar(GroomColorToken);
				if (ColorPrimvar)
				{
					UE::UsdGroomConversion::Private::UpdateAnimInfo(ColorPrimvar, EGroomCacheAttributes::Color, AnimInfo);
				}
			}

			// Following the USD recommendation that gprims be not nested, a Curves prim is handled as a leaf
		}
		else
		{
			// Traverse any prim type
			// Extract the transform if it's a UsdGeomXformable
			if (pxr::UsdGeomXform Xform = pxr::UsdGeomXform(Prim))
			{
				// Propagate the prim transform to the children
				bool bResetXformStack = false;
				FTransform PrimTransform = FTransform::Identity;
				bool bConverted = UsdToUnreal::ConvertXformable(Prim.GetStage(), Xform, PrimTransform, TimeCode.GetValue(), &bResetXformStack);
				if (bConverted)
				{
					TransformToPropagate = bResetXformStack ? PrimTransform : ParentTransform * PrimTransform;
				}
			}

			for (const pxr::UsdPrim& Child : Prim.GetChildren())
			{
				const bool bCheckChildrenGroomAttributes = false;
				bSuccess &= ConvertGroomHierarchy(Child, TimeCode, TransformToPropagate, HairDescription, AnimInfo, bCheckChildrenGroomAttributes);
				if (!bSuccess)
				{
					break;
				}
			}
		}

		if (bCheckGroomAttributes)
		{
			// Groom-scoped attributes are checked only once, on the top-level prim
			// Attributes from Alembic are stored as attributes in the userProperties namespace
			// #ueent_todo: Will need to adjust once the USD groom schemas are defined
			std::vector<pxr::UsdProperty> UserProperties = Prim.GetAuthoredPropertiesInNamespace("userProperties");
			UE::UsdGroomConversion::Private::ConvertGroomAttributes(UserProperties, TimeCode, HairDescription);
		}

		return bSuccess;
	}
}	 // namespace UsdToUnreal

#endif	  // #if USE_USD_SDK
