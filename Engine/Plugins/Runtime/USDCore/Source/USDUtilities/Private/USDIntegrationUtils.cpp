// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDIntegrationUtils.h"

#include "UnrealUSDWrapper.h"
#include "USDAttributeUtils.h"
#include "USDMemory.h"
#include "USDTypesConversion.h"
#include "UsdWrappers/SdfChangeBlock.h"
#include "UsdWrappers/UsdAttribute.h"

#include "Animation/AnimInstance.h"
#include "Components/SkeletalMeshComponent.h"
#include "LiveLinkComponentController.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/usd/attribute.h"
#include "pxr/usd/usd/prim.h"
#include "pxr/usd/usd/tokens.h"
#include "USDIncludesEnd.h"

#define LOCTEXT_NAMESPACE "USDIntegrationUtils"

namespace UE::UsdIntegrationUtils::Private
{
	bool CanApplyOrRemoveSchema(pxr::UsdPrim Prim, pxr::TfToken SchemaName, bool bApplySchema)
	{
		if (!Prim || Prim.IsPseudoRoot())
		{
			return false;
		}

		FScopedUsdAllocs UsdAllocs;

		pxr::TfType Schema = pxr::UsdSchemaRegistry::GetTypeFromSchemaTypeName(SchemaName);
		if (!Schema)
		{
			return false;
		}

		// Query via this function first, because if we're given something like "Material", the following calls
		// will emit USD errors when it tries checking if it can apply a non-single-apply API schema
		if (!pxr::UsdSchemaRegistry::IsAppliedAPISchema(Schema))
		{
			return false;
		}

		// Check if the schema is compatible with this prim
		if (bApplySchema && !Prim.CanApplyAPI(Schema))
		{
			return false;
		}

		pxr::UsdStageRefPtr Stage = Prim.GetStage();
		pxr::SdfLayerRefPtr EditTarget = Stage->GetEditTarget().GetLayer();
		if (!Stage || !EditTarget)
		{
			return false;
		}

		bool bAlreadyHasSchema = false;
		if (EditTarget == Stage->GetRootLayer())
		{
			bAlreadyHasSchema = Prim.HasAPI(Schema);
		}
		else
		{
			// In the future we'll have better layer editing facilities. For now, lets make sure that
			// we only show that we can setup/remove a schema for a prim if it can be done *at that
			// particular edit target*, which is what will happen when adding/removing anyway.
			// For example, a user could:
			//  - Add the schema on a sublayer;
			//  - Remove the schema on the root layer;
			//  - Select the sublayer as the edit target;
			// If we always just queried the existing stage, the user would see that they can add the schema
			// (because the composed prim does not have the schema) but adding the schema would do nothing,
			// as the sublayer prim spec already has the schema...

			pxr::SdfPrimSpecHandle PrimSpecOnLayer = EditTarget->GetPrimAtPath(Prim.GetPath());
			if (!PrimSpecOnLayer)
			{
				// We can always just make a new 'over' on this layer
				return true;
			}

			// We can only add the schema if its not present on this layer, but we should let USD sort the
			// Ops out because we could have Ops to add/delete/prepend/append the same schema on the same prim...
			pxr::SdfListOp<pxr::TfToken> Ops = PrimSpecOnLayer->GetInfo(pxr::UsdTokens->apiSchemas).Get<pxr::SdfTokenListOp>();
			std::vector<pxr::TfToken> AppliedOps;
			Ops.ApplyOperations(&AppliedOps);
			bAlreadyHasSchema = std::find(AppliedOps.begin(), AppliedOps.end(), SchemaName) != AppliedOps.end();
		}

		return bApplySchema != bAlreadyHasSchema;
	}
}	 // namespace UE::UsdIntegrationUtils::Private

bool UsdUtils::PrimHasSchema(const pxr::UsdPrim& Prim, const pxr::TfToken& SchemaToken)
{
	if (!Prim)
	{
		return false;
	}

	FScopedUsdAllocs Allocs;

	pxr::TfType Schema = pxr::UsdSchemaRegistry::GetTypeFromSchemaTypeName(SchemaToken);
	ensure(static_cast<bool>(Schema));

	return Prim.HasAPI(Schema);
}

bool UsdUtils::CanApplySchema(pxr::UsdPrim Prim, pxr::TfToken SchemaName)
{
	const bool bApplySchema = true;
	return UE::UsdIntegrationUtils::Private::CanApplyOrRemoveSchema(Prim, SchemaName, bApplySchema);
}

bool UsdUtils::ApplySchema(const pxr::UsdPrim& Prim, const pxr::TfToken& SchemaToken)
{
	if (!Prim)
	{
		return false;
	}

	FScopedUsdAllocs Allocs;

	pxr::TfType Schema = pxr::UsdSchemaRegistry::GetTypeFromSchemaTypeName(SchemaToken);
	if (!Schema)
	{
		return false;
	}

	return Prim.ApplyAPI(Schema);
}

bool UsdUtils::CanRemoveSchema(pxr::UsdPrim Prim, pxr::TfToken SchemaName)
{
	const bool bApplySchema = false;
	return UE::UsdIntegrationUtils::Private::CanApplyOrRemoveSchema(Prim, SchemaName, bApplySchema);
}

bool UsdUtils::RemoveSchema(const pxr::UsdPrim& Prim, const pxr::TfToken& SchemaToken)
{
	if (!Prim)
	{
		return false;
	}

	FScopedUsdAllocs Allocs;

	pxr::TfType Schema = pxr::UsdSchemaRegistry::GetTypeFromSchemaTypeName(SchemaToken);
	if (!Schema)
	{
		return false;
	}

	return Prim.RemoveAPI(Schema);
}

void UnrealToUsd::ConvertLiveLinkProperties(const UActorComponent* InComponent, pxr::UsdPrim& InOutPrim)
{
#if WITH_EDITOR
	if (!InOutPrim || !UsdUtils::PrimHasSchema(InOutPrim, UnrealIdentifiers::LiveLinkAPI))
	{
		return;
	}

	FScopedUsdAllocs Allocs;

	UE::FSdfChangeBlock ChangeBlock;

	// Skeletal LiveLink case
	if (const USkeletalMeshComponent* SkeletalComponent = Cast<USkeletalMeshComponent>(InComponent))
	{
		if (pxr::UsdAttribute Attr = InOutPrim.CreateAttribute(UnrealIdentifiers::UnrealAnimBlueprintPath, pxr::SdfValueTypeNames->String))
		{
			FString AnimBPPath = SkeletalComponent->AnimClass && SkeletalComponent->AnimClass->ClassGeneratedBy
									 ? SkeletalComponent->AnimClass->ClassGeneratedBy->GetPathName()
									 : FString();

			Attr.Set(UnrealToUsd::ConvertString(*AnimBPPath).Get(), pxr::UsdTimeCode::Default());
			UsdUtils::NotifyIfOverriddenOpinion(Attr);
		}

		if (pxr::UsdAttribute Attr = InOutPrim.CreateAttribute(UnrealIdentifiers::UnrealLiveLinkEnabled, pxr::SdfValueTypeNames->Bool))
		{
			const bool bEnabled = SkeletalComponent->GetAnimationMode() == EAnimationMode::AnimationBlueprint;
			Attr.Set(bEnabled, pxr::UsdTimeCode::Default());
			UsdUtils::NotifyIfOverriddenOpinion(Attr);
		}
	}
	// Non-skeletal LiveLink case
	else if (const ULiveLinkComponentController* InController = Cast<ULiveLinkComponentController>(InComponent))
	{
		if (pxr::UsdAttribute Attr = InOutPrim.CreateAttribute(UnrealIdentifiers::UnrealLiveLinkSubjectName, pxr::SdfValueTypeNames->String))
		{
			Attr.Set(UnrealToUsd::ConvertString(*InController->SubjectRepresentation.Subject.Name.ToString()).Get(), pxr::UsdTimeCode::Default());
			UsdUtils::NotifyIfOverriddenOpinion(Attr);
		}

		if (pxr::UsdAttribute Attr = InOutPrim.CreateAttribute(UnrealIdentifiers::UnrealLiveLinkEnabled, pxr::SdfValueTypeNames->Bool))
		{
			Attr.Set(InController->bEvaluateLiveLink, pxr::UsdTimeCode::Default());
			UsdUtils::NotifyIfOverriddenOpinion(Attr);
		}
	}
#endif	  // WITH_EDITOR
}

#undef LOCTEXT_NAMESPACE
#endif	  // USE_USD_SDK
