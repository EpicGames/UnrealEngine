// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDLayerUtils.h"

#include "UnrealUSDWrapper.h"
#include "USDAttributeUtils.h"
#include "USDErrorUtils.h"
#include "USDMemory.h"
#include "USDTypesConversion.h"

#include "UsdWrappers/SdfChangeBlock.h"
#include "UsdWrappers/SdfLayer.h"
#include "UsdWrappers/UsdAttribute.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdStage.h"

#include "Framework/Application/SlateApplication.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"

#if WITH_EDITOR
#include "DesktopPlatformModule.h"
#endif	  // WITH_EDITOR

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/pcp/layerStack.h"
#include "pxr/usd/sdf/fileFormat.h"
#include "pxr/usd/sdf/layer.h"
#include "pxr/usd/sdf/layerUtils.h"
#include "pxr/usd/sdf/usdaFileFormat.h"
#include "pxr/usd/sdf/zipFile.h"
#include "pxr/usd/usd/editContext.h"
#include "pxr/usd/usd/modelAPI.h"
#include "pxr/usd/usd/primCompositionQuery.h"
#include "pxr/usd/usd/stage.h"
#include "pxr/usd/usdGeom/xform.h"
#include "pxr/usd/usdUtils/flattenLayerStack.h"
#include "USDIncludesEnd.h"

#define LOCTEXT_NAMESPACE "USDLayerUtils"

namespace UE::UsdLayerUtilsImpl::Private
{
	/**
	 * Adapted from flattenUtils.cpp::_FixAssetPaths, except that we only handle actual AssetPaths here as layer/prim paths
	 * will be remapped via Layer.UpdateCompositionAssetDependency().
	 * Returns whether anything was remapped
	 */
	bool FixAssetPaths(const pxr::SdfLayerHandle& SourceLayer, pxr::VtValue* Value)
	{
		if (Value->IsHolding<pxr::SdfAssetPath>())
		{
			pxr::SdfAssetPath AssetPath;
			Value->Swap(AssetPath);
			AssetPath = pxr::SdfAssetPath(SourceLayer->ComputeAbsolutePath(AssetPath.GetAssetPath()));
			Value->Swap(AssetPath);
			return true;
		}
		else if (Value->IsHolding<pxr::VtArray<pxr::SdfAssetPath>>())
		{
			pxr::VtArray<pxr::SdfAssetPath> PathArray;
			Value->Swap(PathArray);
			for (pxr::SdfAssetPath& AssetPath : PathArray)
			{
				AssetPath = pxr::SdfAssetPath(SourceLayer->ComputeAbsolutePath(AssetPath.GetAssetPath()));
			}
			Value->Swap(PathArray);
			return true;
		}

		return false;
	}
}	 // namespace UE::UsdLayerUtilsImpl::Private

FText UsdUtils::ToText(ECanInsertSublayerResult Result)
{
	switch (Result)
	{
		default:
		case ECanInsertSublayerResult::Success:
			return FText::GetEmpty();
		case ECanInsertSublayerResult::ErrorSubLayerNotFound:
			return LOCTEXT("CanAddSubLayerNotFound", "SubLayer not found!");
		case ECanInsertSublayerResult::ErrorSubLayerInvalid:
			return LOCTEXT("CanAddSubLayerInvalid", "SubLayer is invalid!");
		case ECanInsertSublayerResult::ErrorSubLayerIsParentLayer:
			return LOCTEXT("CanAddSubLayerIsParent", "SubLayer is the same as the parent layer!");
		case ECanInsertSublayerResult::ErrorCycleDetected:
			return LOCTEXT("CanAddSubLayerCycle", "Cycles detected!");
	}
}

UsdUtils::ECanInsertSublayerResult UsdUtils::CanInsertSubLayer(const pxr::SdfLayerRefPtr& ParentLayer, const TCHAR* SubLayerIdentifier)
{
	if (!SubLayerIdentifier)
	{
		return ECanInsertSublayerResult::ErrorSubLayerNotFound;
	}

	FScopedUsdAllocs Allocs;

	pxr::SdfLayerRefPtr SubLayer = pxr::SdfLayer::FindOrOpen(UnrealToUsd::ConvertString(SubLayerIdentifier).Get());
	if (!SubLayer)
	{
		return ECanInsertSublayerResult::ErrorSubLayerNotFound;
	}

	if (SubLayer == ParentLayer)
	{
		return ECanInsertSublayerResult::ErrorSubLayerIsParentLayer;
	}

	// We can't climb through ancestors of ParentLayer, so we have to open sublayer and see if parentlayer is a
	// descendant of *it* in order to detect cycles
	TFunction<ECanInsertSublayerResult(const pxr::SdfLayerRefPtr&)> CanAddSubLayerRecursive;
	CanAddSubLayerRecursive = [ParentLayer, &CanAddSubLayerRecursive](const pxr::SdfLayerRefPtr& CurrentParent) -> ECanInsertSublayerResult
	{
		for (const std::string& SubLayerPath : CurrentParent->GetSubLayerPaths())
		{
			// This may seem expensive, but keep in mind the main use case for this (at least for now) is for checking
			// during layer drag and drop, where all of these layers are actually already open anyway
			pxr::SdfLayerRefPtr ChildSubLayer = pxr::SdfLayer::FindOrOpenRelativeToLayer(CurrentParent, SubLayerPath);

			if (!ChildSubLayer)
			{
				return ECanInsertSublayerResult::ErrorSubLayerInvalid;
			}

			ECanInsertSublayerResult RecursiveResult = ChildSubLayer == ParentLayer ? ECanInsertSublayerResult::ErrorCycleDetected
																					: CanAddSubLayerRecursive(ChildSubLayer);

			if (RecursiveResult != ECanInsertSublayerResult::Success)
			{
				return RecursiveResult;
			}
		}

		return ECanInsertSublayerResult::Success;
	};

	return CanAddSubLayerRecursive(SubLayer);
}

bool UsdUtils::InsertSubLayer(
	const pxr::SdfLayerRefPtr& ParentLayer,
	const TCHAR* SubLayerFile,
	int32 Index,
	double OffsetTimeCodes,
	double TimeCodesScale
)
{
	if (!ParentLayer)
	{
		return false;
	}

	FString RelativeSubLayerPath = SubLayerFile;
	MakePathRelativeToLayer(UE::FSdfLayer{ParentLayer}, RelativeSubLayerPath);

	// If the relative path is just the same as the clean name (e.g. Layer.usda wants to add Layer.usda as a sublayer) then
	// just stop here as that is always an error
	FString ParentLayerPath = UsdToUnreal::ConvertString(ParentLayer->GetRealPath());
	if (FPaths::GetCleanFilename(ParentLayerPath) == RelativeSubLayerPath)
	{
		USD_LOG_USERWARNING(
			FText::Format(LOCTEXT("SelfSublayer", "Tried to add layer '{0}' as a sublayer of itself!"), FText::FromString(ParentLayerPath))
		);
		return false;
	}

	FScopedUsdAllocs UsdAllocs;

	ParentLayer->InsertSubLayerPath(UnrealToUsd::ConvertString(*RelativeSubLayerPath).Get(), Index);

	if (!FMath::IsNearlyZero(OffsetTimeCodes) || !FMath::IsNearlyEqual(TimeCodesScale, 1.0))
	{
		if (Index == -1)
		{
			Index = ParentLayer->GetNumSubLayerPaths() - 1;
		}

		ParentLayer->SetSubLayerOffset(pxr::SdfLayerOffset{OffsetTimeCodes, TimeCodesScale}, Index);
	}

	return true;
}

#if WITH_EDITOR
TOptional<FString> UsdUtils::BrowseUsdFile(EBrowseFileMode Mode)
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();

	if (DesktopPlatform == nullptr)
	{
		return {};
	}

	TArray<FString> OutFiles;

	TArray<FString> NativeTextExtensions;
	TArray<FString> NativePossiblyBinaryExtensions;
	UnrealUSDWrapper::GetNativeFileFormats(NativeTextExtensions, NativePossiblyBinaryExtensions);

	TSet<FString> NativeTextExtensionsSet{NativeTextExtensions};
	TSet<FString> NativePossiblyBinaryExtensionsSet{NativePossiblyBinaryExtensions};

	// When browsing files for the purposes of opening a stage or saving layers,
	// we offer the native USD file formats as options. Browsing files in order
	// to use them as the targets of composition arcs (e.g. sublayers,
	// references, payloads, etc.) also offers any plugin file formats that are
	// registered.
	TArray<FString> SupportedExtensions;
	if (Mode == EBrowseFileMode::Composition)
	{
		SupportedExtensions = UnrealUSDWrapper::GetAllSupportedFileFormats();
	}
	else
	{
		SupportedExtensions = NativeTextExtensions;
		SupportedExtensions.Append(NativePossiblyBinaryExtensions);
	}

	if (SupportedExtensions.Num() == 0)
	{
		USD_LOG_ERROR(TEXT("No file extensions supported by the USD SDK!"));
		return {};
	}

	if (Mode == EBrowseFileMode::Save)
	{
		// USD 21.08 doesn't yet support saving to USDZ, so instead of allowing this option and leading to an error we'll just hide it
		SupportedExtensions.Remove(TEXT("usdz"));
	}

	// Show an option for "all supported extensions" at the same time, but only if not saving: For saving the user should have to pick one directly
	FString FileTypes;
	if (Mode != EBrowseFileMode::Save)
	{
		FString JoinedExtensions = FString::Join(SupportedExtensions, TEXT(";*."));	   // Combine "usd" and "usda" into "usd; *.usda"
		FileTypes = FString::Printf(TEXT("Universal Scene Description files (*.%s)|*.%s|"), *JoinedExtensions, *JoinedExtensions);
	}

	for (const FString& SupportedExtension : SupportedExtensions)
	{
		const bool bIsTextNative = NativeTextExtensionsSet.Contains(SupportedExtension);
		const bool bIsBinaryNative = NativePossiblyBinaryExtensionsSet.Contains(SupportedExtension);

		// The '(*.%s)' on the actual name (before the '|') is not optional: We need the name part to be different for each format
		// or else the options will overwrite each other on the Mac
		FileTypes += FString::Printf(
			TEXT("Universal Scene Description %sfile (*.%s)|*.%s|"),
			bIsTextNative	  ? TEXT("text ")
			: bIsBinaryNative ? TEXT("binary ")
							  : TEXT(""),
			*SupportedExtension,
			*SupportedExtension
		);
	}

	switch (Mode)
	{
		case EBrowseFileMode::Open:
		case EBrowseFileMode::Composition:
		{
			if (!DesktopPlatform->OpenFileDialog(
					FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
					LOCTEXT("ChooseFile", "Choose file").ToString(),
					TEXT(""),
					TEXT(""),
					*FileTypes,
					EFileDialogFlags::None,
					OutFiles
				))
			{
				return {};
			}
			break;
		}
		case EBrowseFileMode::Save:
		{
			if (!DesktopPlatform->SaveFileDialog(
					FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
					LOCTEXT("ChooseFile", "Choose file").ToString(),
					TEXT(""),
					TEXT(""),
					*FileTypes,
					EFileDialogFlags::None,
					OutFiles
				))
			{
				return {};
			}
			break;
		}
		default:
			break;
	}

	if (OutFiles.Num() > 0)
	{
		// Always make this an absolute path because it may try generating a relative path to the engine binary if it can
		return FPaths::ConvertRelativePathToFull(OutFiles[0]);
	}

	return {};
}

#endif	  // WITH_EDITOR

FString UsdUtils::MakePathRelativeToProjectDir(const FString& Path)
{
	FString PathConverted = FPaths::ConvertRelativePathToFull(Path);

	// Mirror behavior of RelativeToGameDir meta tag on the stage actor's RootLayer
	if (FPaths::IsUnderDirectory(PathConverted, FPaths::ProjectDir()))
	{
		FPaths::MakePathRelativeTo(PathConverted, *FPaths::ProjectDir());
	}

	return PathConverted;
}

bool UsdUtils::IsZipArchive(const FString& InUSDZFilePath)
{
	if (TUniquePtr<FArchive> Ar = TUniquePtr<FArchive>(IFileManager::Get().CreateFileReader(*InUSDZFilePath)))
	{
		uint32 Magic = 0;
		if (Ar->TotalSize() >= 4)
		{
			Ar->Serialize(&Magic, sizeof(Magic));
			// ZIP local file header: 0x04034B50 (Little Endian for "PK\x03\x04")
			return Magic == 0x04034B50;
		}
	}
	return false;
}

bool UsdUtils::SplitUSDZAssetPath(const FString& InFilePathIntoUSDZArchive, FString& OutUSDZFilePath, FString& OutInnerAssetPath)
{
	// We need at least an opening and closing bracket
	if (InFilePathIntoUSDZArchive.Len() < 3)
	{
		return false;
	}

	const int32 OpenBracketPos = InFilePathIntoUSDZArchive.Find(TEXT("["), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
	const int32 CloseBracketPos = InFilePathIntoUSDZArchive.Find(TEXT("]"), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
	if (OpenBracketPos != INDEX_NONE &&								 //
		CloseBracketPos == InFilePathIntoUSDZArchive.Len() - 1 &&	 //
		CloseBracketPos > OpenBracketPos - 1)
	{
		// Should be like "C:/MyFiles/scene.usdz"
		OutUSDZFilePath = InFilePathIntoUSDZArchive.Left(OpenBracketPos);

		// Should be like "0/texture.png"
		OutInnerAssetPath = InFilePathIntoUSDZArchive.Mid(OpenBracketPos + 1, CloseBracketPos - OpenBracketPos - 1);

		return true;
	}

	return false;
}

UsdUtils::EUSDZDecompressResult UsdUtils::DecompressUSDZFile(const FString& InUSDZFilePath, const FString& InOutputDir, FString* OutDecompressedRootLayer)
{
	if (!IsZipArchive(InUSDZFilePath))
	{
		return EUSDZDecompressResult::Failed;
	}

	FScopedUsdAllocs Allocs;

	pxr::SdfZipFile File = pxr::SdfZipFile::Open(UnrealToUsd::ConvertString(*InUSDZFilePath).Get());
	if (!File)
	{
		return EUSDZDecompressResult::Failed;
	}

	auto TryMakeSafeExtractionPath = [](const FString& InExtractionRoot, const FString& InArchiveEntryPath, FString& OutSafeAbsolutePath) -> bool
	{
		ensure(!InArchiveEntryPath.IsEmpty());

		// Normalize path separators first so we can consistently reason about segments
		FString ArchiveEntryPath = InArchiveEntryPath;
		FPaths::NormalizeFilename(ArchiveEntryPath);

		// Reject rooted / absolute / drive-qualified / UNC-like paths. Note we don't
		// automatically reject relative '..' path components here, but instead wait
		// for the normalized absolute path to escape the extraction directory.
		if (ArchiveEntryPath.StartsWith(TEXT("/")) ||
			ArchiveEntryPath.StartsWith(TEXT("\\")) ||
			ArchiveEntryPath.Contains(TEXT(":")))
		{
			return false;
		}

		// Build canonical absolute extraction root
		FString ExtractionRoot = FPaths::ConvertRelativePathToFull(InExtractionRoot);
		FPaths::NormalizeDirectoryName(ExtractionRoot);

		// Build canonical absolute candidate output path
		FString CandidatePath = FPaths::Combine(ExtractionRoot, ArchiveEntryPath);
		CandidatePath = FPaths::ConvertRelativePathToFull(CandidatePath);
		FPaths::NormalizeFilename(CandidatePath);

		// Check that CandidatePath remains under ExtractionRoot
		FString ExtractionRootPrefix = ExtractionRoot;
		FPaths::NormalizeFilename(ExtractionRootPrefix);
		if (!ExtractionRootPrefix.EndsWith(TEXT("/")))
		{
			ExtractionRootPrefix += TEXT("/");
		}

#if PLATFORM_WINDOWS
		constexpr ESearchCase::Type PathSearchCase = ESearchCase::IgnoreCase;
#else
		constexpr ESearchCase::Type PathSearchCase = ESearchCase::CaseSensitive;
#endif

		if (!CandidatePath.StartsWith(ExtractionRootPrefix, PathSearchCase))
		{
			return false;
		}

		OutSafeAbsolutePath = CandidatePath;
		return true;
	};

	TMap<int32, FString> SafeTargetPathsByArchiveIndex;
	TSet<FString> UnsafeArchivePaths;

	bool bSawFirstFile = false;
	FString ValidatedRootLayerPath;

	// Pass 1: Validate the entire archive and compute final extraction paths.
	int32 ArchiveIndex = 0;
	for (pxr::SdfZipFile::Iterator Iter = File.begin(), End = File.end(); Iter != End; ++Iter, ++ArchiveIndex)
	{
		const pxr::SdfZipFile::FileInfo Info = Iter.GetFileInfo();
		const FString UEInnerFilePath = UsdToUnreal::ConvertString(*Iter);

		if (UEInnerFilePath.IsEmpty())
		{
			continue;
		}

		FString SafeTargetPath;
		if (!TryMakeSafeExtractionPath(InOutputDir, UEInnerFilePath, SafeTargetPath))
		{
			UnsafeArchivePaths.Add(UEInnerFilePath);

			USD_LOG_USERERROR(FText::Format(
				LOCTEXT("UnsafeUSDZEntryPath", "Cannot decompress USDZ file '{0}': Archive entry '{1}' attempts to write outside the extraction directory."),
				FText::FromString(InUSDZFilePath),
				FText::FromString(UEInnerFilePath)
			));
			
			continue;
		}

		// According to USDZ spec, the first file must be the root USD layer
		if (!bSawFirstFile)
		{
			bSawFirstFile = true;

			const bool bIncludeDot = false;
			const FString Extension = FPaths::GetExtension(UEInnerFilePath, bIncludeDot).ToLower();
			if (Extension.StartsWith(TEXT("usd")))
			{
				ValidatedRootLayerPath = SafeTargetPath;
			}
			else
			{
				USD_LOG_USERWARNING(FText::Format(
					LOCTEXT("FailedToDecompress", "Failed to decompress {0}: First file '{1}' should always be a USD file!"),
					FText::FromString(InUSDZFilePath),
					FText::FromString(UEInnerFilePath)
				));
				return EUSDZDecompressResult::Failed;
			}
		}

		// Check for these, but it's very unlikely they differ from these values as USD never uses
		// compression/encryption, and Usd_UsdzResolver::OpenAsset also just errors out if it runs into these
		if (Info.compressionMethod != 0)
		{
			USD_LOG_USERWARNING(FText::Format(
				LOCTEXT("FailedToDecompressInternal", "Cannot open {0} in {1}: Compressed files are not supported!"),
				FText::FromString(UEInnerFilePath),
				FText::FromString(InUSDZFilePath)
			));
			continue;
		}
		if (Info.encrypted)
		{
			USD_LOG_USERWARNING(FText::Format(
				LOCTEXT("FailToDecrypt", "Cannot open {0} in {1}: Encrypted files are not supported!"),
				FText::FromString(UEInnerFilePath),
				FText::FromString(InUSDZFilePath)
			));
			continue;
		}

		SafeTargetPathsByArchiveIndex.Add(ArchiveIndex, SafeTargetPath);
	}

	if (!bSawFirstFile || ValidatedRootLayerPath.IsEmpty() || !UnsafeArchivePaths.IsEmpty())
	{
		return EUSDZDecompressResult::Failed;
	}

	IFileManager& FileManager = IFileManager::Get();

	// For now we don't support extracting to a directory that exists. Symlinks and junctions 
	// inside an existing directory can lead to unsafe extractions without additional checks.
	if (FileManager.DirectoryExists(*InOutputDir))
	{
		USD_LOG_USERERROR(FText::Format(
			LOCTEXT("USDZDirectoryExists", "Cannot decompress USDZ file '{0}': Target directory {1} already exists."),
			FText::FromString(InUSDZFilePath),
			FText::FromString(InOutputDir)
		));
		return EUSDZDecompressResult::Failed;
	}

	const bool bTree = true;
	if (!FileManager.MakeDirectory(*InOutputDir, bTree))
	{
		USD_LOG_USERERROR(FText::Format(
			LOCTEXT("USDZFailedToCreateDirectory", "Cannot decompress USDZ file '{0}': Failed to create target directory {1}."),
			FText::FromString(InUSDZFilePath),
			FText::FromString(InOutputDir)
		));
		return EUSDZDecompressResult::Failed;
	}

	EUSDZDecompressResult Result = SafeTargetPathsByArchiveIndex.Num() == ArchiveIndex
		? EUSDZDecompressResult::Success
		: EUSDZDecompressResult::SuccessWithSkippedEntries;

	// Pass 2: Extract using only pre-validated target paths.
	ArchiveIndex = 0;
	for (pxr::SdfZipFile::Iterator Iter = File.begin(), End = File.end(); Iter != End; ++Iter, ++ArchiveIndex)
	{
		const FString* SafeTargetPath = SafeTargetPathsByArchiveIndex.Find(ArchiveIndex);
		if (SafeTargetPath == nullptr)
		{
			continue;
		}

		const pxr::SdfZipFile::FileInfo Info = Iter.GetFileInfo();
		const char* FileStart = Iter.GetFile();
		const std::size_t TargetFileSizeBytes = Info.size;

		// Create any nested folder (usually a folder named "0")
		if (!IFileManager::Get().MakeDirectory(*FPaths::GetPath(*SafeTargetPath), bTree))
		{
			Result = EUSDZDecompressResult::Failed;
			break;
		}

		TUniquePtr<FArchive> Ar{ IFileManager::Get().CreateFileWriter(**SafeTargetPath) };
		if (!Ar)
		{
			Result = EUSDZDecompressResult::Failed;
			break;
		}

		Ar->Serialize((void*)FileStart, TargetFileSizeBytes);
		if(!Ar->Close())
		{
			Result = EUSDZDecompressResult::Failed;
			break;
		}
	}

	if (Result != EUSDZDecompressResult::Failed)
	{
		if (OutDecompressedRootLayer != nullptr)
		{
			*OutDecompressedRootLayer = ValidatedRootLayerPath;
		}
	}
	else
	{ 
		if (!FileManager.DeleteDirectory(*InOutputDir, /*RequireExists=*/ true, /*Tree=*/ true))
		{
			USD_LOG_USERWARNING(FText::Format(
				LOCTEXT("USDZFailedToCleanup", "Failed to clean up directory {0} while decompressing USDZ file '{1}'."),
				FText::FromString(InOutputDir),
				FText::FromString(InUSDZFilePath)
			));
		}
	}

	return Result;
}

TUsdStore<pxr::SdfLayerRefPtr> UsdUtils::CreateNewLayer(
	TUsdStore<pxr::UsdStageRefPtr> UsdStage,
	const TUsdStore<pxr::SdfLayerRefPtr>& ParentLayer,
	const TCHAR* LayerFilePath
)
{
	FScopedUsdAllocs UsdAllocs;

	std::string UsdLayerFilePath = UnrealToUsd::ConvertString(*FPaths::ConvertRelativePathToFull(LayerFilePath)).Get();

	pxr::SdfLayerRefPtr LayerRef = pxr::SdfLayer::CreateNew(UsdLayerFilePath);

	if (!LayerRef)
	{
		return {};
	}

	// New layer needs to be created and in the stage layer stack before we can edit it
	UsdUtils::InsertSubLayer(ParentLayer.Get(), LayerFilePath);

	FScopedUsdMessageLog ScopedLog;
	pxr::UsdEditContext UsdEditContext(UsdStage.Get(), LayerRef);

	// Create default prim
	FString PrimPath = TEXT("/") + FPaths::GetBaseFilename(UsdToUnreal::ConvertString(LayerRef->GetDisplayName()));

	pxr::SdfPath UsdPrimPath = UnrealToUsd::ConvertPath(*PrimPath).Get();
	pxr::UsdGeomXform DefaultPrim = pxr::UsdGeomXform::Define(UsdStage.Get(), UsdPrimPath);

	if (DefaultPrim)
	{
		// Set default prim
		LayerRef->SetDefaultPrim(DefaultPrim.GetPrim().GetName());
	}

	if (FUsdLogManager::HasAccumulatedErrors())
	{
		return {};
	}

	return TUsdStore<pxr::SdfLayerRefPtr>(LayerRef);
}

UE::FSdfLayer UsdUtils::FindLayerForPrim(const pxr::UsdPrim& Prim)
{
	if (!Prim)
	{
		return {};
	}

	FScopedUsdAllocs UsdAllocs;

	// Use this instead of UsdPrimCompositionQuery as that one can simply fail in some scenarios
	// (e.g. empty parent layer pointing at a sublayer with a prim, where it fails to provide the sublayer arc's layer)
	for (const pxr::SdfPrimSpecHandle& Handle : Prim.GetPrimStack())
	{
		if (Handle)
		{
			if (pxr::SdfLayerHandle Layer = Handle->GetLayer())
			{
				return UE::FSdfLayer(Layer);
			}
		}
	}

	return UE::FSdfLayer(Prim.GetStage()->GetRootLayer());
}

UE::FSdfLayer UsdUtils::FindLayerForAttribute(const pxr::UsdAttribute& Attribute, double TimeCode)
{
	if (!Attribute)
	{
		return {};
	}

	FScopedUsdAllocs UsdAllocs;

	for (const pxr::SdfPropertySpecHandle& PropertySpec : Attribute.GetPropertyStack(TimeCode))
	{
		if (PropertySpec)
		{
			return UE::FSdfLayer(PropertySpec->GetLayer());
		}
	}

	return {};
}

UE::FSdfLayer UsdUtils::FindLayerForAttributes(const TArray<UE::FUsdAttribute>& Attributes, double TimeCode, bool bIncludeSessionLayers)
{
	FScopedUsdAllocs UsdAllocs;

	TMap<FString, UE::FSdfLayer> IdentifierToLayers;
	IdentifierToLayers.Reserve(Attributes.Num());

	pxr::UsdStageRefPtr Stage;
	for (const UE::FUsdAttribute& Attribute : Attributes)
	{
		if (Attribute)
		{
			if (UE::FSdfLayer Layer = UsdUtils::FindLayerForAttribute(Attribute, TimeCode))
			{
				IdentifierToLayers.Add(Layer.GetIdentifier(), Layer);

				if (!Stage)
				{
					Stage = pxr::UsdStageRefPtr{Attribute.GetPrim().GetStage()};
				}
			}
		}
	}

	if (!Stage || IdentifierToLayers.Num() == 0)
	{
		return {};
	}

	if (IdentifierToLayers.Num() == 1)
	{
		return IdentifierToLayers.CreateIterator().Value();
	}

	// Iterate through the layer stack in strong to weak order, and return the first of those layers
	// that is actually one of the attribute layers
	for (const pxr::SdfLayerHandle& LayerHandle : Stage->GetLayerStack(bIncludeSessionLayers))
	{
		FString Identifier = UsdToUnreal::ConvertString(LayerHandle->GetIdentifier());
		if (UE::FSdfLayer* AttributeLayer = IdentifierToLayers.Find(Identifier))
		{
			return *AttributeLayer;
		}
	}

	return {};
}

UE::FSdfLayer UsdUtils::FindLayerForSubLayerPath(const UE::FSdfLayer& RootLayer, const FStringView& SubLayerPath)
{
	const FString RelativeLayerPath = UE::FSdfLayerUtils::SdfComputeAssetPathRelativeToLayer(RootLayer, SubLayerPath.GetData());

	return UE::FSdfLayer::FindOrOpen(*RelativeLayerPath);
}

bool UsdUtils::SetRefOrPayloadLayerOffset(pxr::UsdPrim& Prim, const UE::FSdfLayerOffset& LayerOffset)
{
	FScopedUsdAllocs UsdAllocs;

	pxr::UsdPrimCompositionQuery PrimCompositionQuery(Prim);
	std::vector<pxr::UsdPrimCompositionQueryArc> CompositionArcs = PrimCompositionQuery.GetCompositionArcs();

	for (const pxr::UsdPrimCompositionQueryArc& CompositionArc : CompositionArcs)
	{
		if (CompositionArc.GetArcType() == pxr::PcpArcTypeReference)
		{
			pxr::SdfReferenceEditorProxy ReferenceEditor;
			pxr::SdfReference OldReference;

			if (CompositionArc.GetIntroducingListEditor(&ReferenceEditor, &OldReference))
			{
				pxr::SdfReference NewReference = OldReference;
				NewReference.SetLayerOffset(pxr::SdfLayerOffset(LayerOffset.Offset, LayerOffset.Scale));

				ReferenceEditor.ReplaceItemEdits(OldReference, NewReference);

				return true;
			}
		}
		else if (CompositionArc.GetArcType() == pxr::PcpArcTypePayload)
		{
			pxr::SdfPayloadEditorProxy PayloadEditor;
			pxr::SdfPayload OldPayload;

			if (CompositionArc.GetIntroducingListEditor(&PayloadEditor, &OldPayload))
			{
				pxr::SdfPayload NewPayload = OldPayload;
				NewPayload.SetLayerOffset(pxr::SdfLayerOffset(LayerOffset.Offset, LayerOffset.Scale));

				PayloadEditor.ReplaceItemEdits(OldPayload, NewPayload);

				return true;
			}
		}
	}

	return false;
}

UE::FSdfLayerOffset UsdUtils::GetLayerToStageOffset(const pxr::UsdAttribute& Attribute)
{
	// Inspired by pxr::_GetLayerToStageOffset

	UE::FSdfLayer AttributeLayer = UsdUtils::FindLayerForAttribute(Attribute, pxr::UsdTimeCode::EarliestTime().GetValue());

	FScopedUsdAllocs UsdAllocs;

	pxr::UsdResolveInfo ResolveInfo = Attribute.GetResolveInfo(pxr::UsdTimeCode::EarliestTime());
	pxr::PcpNodeRef Node = ResolveInfo.GetNode();
	if (!Node)
	{
		return UE::FSdfLayerOffset();
	}

	const pxr::PcpMapExpression& MapToRoot = Node.GetMapToRoot();
	if (MapToRoot.IsNull())
	{
		return UE::FSdfLayerOffset();
	}

	pxr::SdfLayerOffset NodeToRootNodeOffset = MapToRoot.GetTimeOffset();

	pxr::SdfLayerOffset LocalOffset = NodeToRootNodeOffset;

	if (const pxr::SdfLayerOffset* LayerToRootLayerOffset = ResolveInfo.GetNode().GetLayerStack()->GetLayerOffsetForLayer(
			pxr::SdfLayerRefPtr(AttributeLayer)
		))
	{
		LocalOffset = LocalOffset * (*LayerToRootLayerOffset);
	}

	return UE::FSdfLayerOffset(LocalOffset.GetOffset(), LocalOffset.GetScale());
}

UE::FSdfLayerOffset UsdUtils::GetPrimToStageOffset(const UE::FUsdPrim& Prim)
{
	// In most cases all we care about is an offset from the prim's layer to the stage, but it is also possible for a prim
	// to directly reference another layer with an offset and scale as well, and this function will pick up on that. Example:
	//
	// def SkelRoot "Model" (
	//	  prepend references = @sublayer.usda@ ( offset = 15; scale = 2.0 )
	// )
	// {
	// }
	//
	// Otherwise, this function really has the same effect as GetLayerToStageOffset, but we need to use an actual prim to be able
	// to get USD to combine layer offsets and scales for us (via UsdPrimCompositionQuery).

	FScopedUsdAllocs Allocs;

	UE::FSdfLayer StrongestLayerForPrim = UsdUtils::FindLayerForPrim(Prim);

	pxr::UsdPrim UsdPrim{Prim};

	pxr::UsdPrimCompositionQuery PrimCompositionQuery(UsdPrim);
	pxr::UsdPrimCompositionQuery::Filter Filter;
	Filter.hasSpecsFilter = pxr::UsdPrimCompositionQuery::HasSpecsFilter::HasSpecs;
	PrimCompositionQuery.SetFilter(Filter);

	for (const pxr::UsdPrimCompositionQueryArc& CompositionArc : PrimCompositionQuery.GetCompositionArcs())
	{
		if (pxr::PcpNodeRef Node = CompositionArc.GetTargetNode())
		{
			pxr::SdfLayerOffset Offset;

			// This part of the offset will handle direct prim references
			const pxr::PcpMapExpression& MapToRoot = Node.GetMapToRoot();
			if (!MapToRoot.IsNull())
			{
				Offset = MapToRoot.GetTimeOffset();
			}

			if (const pxr::SdfLayerOffset* LayerOffset = Node.GetLayerStack()->GetLayerOffsetForLayer(pxr::SdfLayerRefPtr{StrongestLayerForPrim}))
			{
				Offset = Offset * (*LayerOffset);
			}

			return UE::FSdfLayerOffset{Offset.GetOffset(), Offset.GetScale()};
		}
	}

	return UE::FSdfLayerOffset{};
}

void UsdUtils::AddTimeCodeRangeToLayer(const pxr::SdfLayerRefPtr& Layer, double StartTimeCode, double EndTimeCode)
{
	FScopedUsdAllocs UsdAllocs;

	if (!Layer)
	{
		USD_LOG_USERWARNING(LOCTEXT("AddTimeCode_InvalidLayer", "Trying to set timecodes on an invalid layer."));
		return;
	}

	// The HasTimeCode check is needed or else we can't author anything with a StartTimeCode lower than the default of 0
	if (StartTimeCode < Layer->GetStartTimeCode() || !Layer->HasStartTimeCode())
	{
		Layer->SetStartTimeCode(StartTimeCode);
	}

	if (EndTimeCode > Layer->GetEndTimeCode() || !Layer->HasEndTimeCode())
	{
		Layer->SetEndTimeCode(EndTimeCode);
	}
}

void UsdUtils::MakePathRelativeToLayer(const UE::FSdfLayer& Layer, FString& Path)
{
#if USE_USD_SDK
	FScopedUsdAllocs UsdAllocs;

	if (pxr::SdfLayerRefPtr UsdLayer = (pxr::SdfLayerRefPtr)Layer)
	{
		std::string RepositoryPath = UsdLayer->GetRepositoryPath().empty() ? UsdLayer->GetRealPath() : UsdLayer->GetRepositoryPath();
		FString LayerAbsolutePath = UsdToUnreal::ConvertString(RepositoryPath);
		if (!LayerAbsolutePath.IsEmpty())
		{
			FPaths::MakePathRelativeTo(Path, *LayerAbsolutePath);
		}
	}
#endif	  // #if USE_USD_SDK
}

UE::FSdfLayer UsdUtils::GetUEPersistentStateSublayer(const UE::FUsdStage& Stage, bool bCreateIfNeeded)
{
	UE::FSdfLayer StateLayer;
	if (!Stage)
	{
		return StateLayer;
	}

	FScopedUsdAllocs Allocs;

	UE::FSdfChangeBlock ChangeBlock;

	FString PathPart;
	FString FilenamePart;
	FString ExtensionPart;
	FPaths::Split(Stage.GetRootLayer().GetRealPath(), PathPart, FilenamePart, ExtensionPart);

	FString ExpectedStateLayerPath = FPaths::Combine(PathPart, FString::Printf(TEXT("%s-UE-persistent-state.%s"), *FilenamePart, *ExtensionPart));
	FPaths::NormalizeFilename(ExpectedStateLayerPath);

	StateLayer = UE::FSdfLayer::FindOrOpen(*ExpectedStateLayerPath);

	if (!StateLayer && bCreateIfNeeded)
	{
		StateLayer = pxr::SdfLayer::New(
			pxr::SdfFileFormat::FindById(pxr::SdfUsdaFileFormatTokens->Id),
			UnrealToUsd::ConvertString(*ExpectedStateLayerPath).Get()
		);
	}

	// Add the layer as a sublayer of the session layer, in the right location
	// Always check this because we need to do this even if we just loaded an existing state layer from disk
	if (StateLayer)
	{
		UE::FSdfLayer SessionLayer = Stage.GetSessionLayer();

		// For consistency we always add the UEPersistentState sublayer as the weakest sublayer of the stage's session layer
		// Note that we intentionally only guarantee the UEPersistentLayer is weaker than the UESessionLayer when inserting,
		// so that the user may reorder these if they want, for whatever reason
		bool bNeedsToBeAdded = true;
		for (const FString& Path : SessionLayer.GetSubLayerPaths())
		{
			if (FPaths::IsSamePath(Path, ExpectedStateLayerPath))
			{
				bNeedsToBeAdded = false;
				break;
			}
		}

		if (bNeedsToBeAdded)
		{
			// Always add it at the back, so it's weaker than the session layer
			InsertSubLayer(static_cast<pxr::SdfLayerRefPtr&>(SessionLayer), *ExpectedStateLayerPath);
		}
	}

	return StateLayer;
}

UE::FSdfLayer UsdUtils::GetUESessionStateSublayer(const UE::FUsdStage& Stage, bool bCreateIfNeeded)
{
	UE::FSdfLayer StateLayer;
	if (!Stage)
	{
		return StateLayer;
	}

	FScopedUsdAllocs Allocs;

	const pxr::UsdStageRefPtr UsdStage{Stage};
	pxr::SdfLayerRefPtr UsdSessionLayer = UsdStage->GetSessionLayer();

	FString PathPart;
	FString FilenamePart;
	FString ExtensionPart;
	FPaths::Split(Stage.GetRootLayer().GetRealPath(), PathPart, FilenamePart, ExtensionPart);

	FString ExpectedStateLayerDisplayName = FString::Printf(TEXT("%s-UE-session-state.%s"), *FilenamePart, *ExtensionPart);
	FPaths::NormalizeFilename(ExpectedStateLayerDisplayName);

	std::string UsdExpectedStateLayerDisplayName = UnrealToUsd::ConvertString(*ExpectedStateLayerDisplayName).Get();

	// Check if we already have an existing utils layer in this stage
	std::string ExistingUESessionStateIdentifier;
	{
		std::unordered_set<std::string> SessionLayerSubLayerIdentifiers;
		for (const std::string& SubLayerIdentifier : UsdSessionLayer->GetSubLayerPaths())
		{
			SessionLayerSubLayerIdentifiers.insert(SubLayerIdentifier);
		}
		if (SessionLayerSubLayerIdentifiers.size() > 0)
		{
			const bool bIncludeSessionLayers = true;
			for (const pxr::SdfLayerHandle& Layer : UsdStage->GetLayerStack(bIncludeSessionLayers))
			{
				// All session layers always come before the root layer
				if (Layer == UsdStage->GetRootLayer())
				{
					break;
				}

				const std::string& Identifier = Layer->GetIdentifier();
				if (Layer->IsAnonymous() && Layer->GetDisplayName() == UsdExpectedStateLayerDisplayName
					&& SessionLayerSubLayerIdentifiers.count(Identifier) > 0)
				{
					ExistingUESessionStateIdentifier = Identifier;
					break;
				}
			}
		}
	}

	if (ExistingUESessionStateIdentifier.size() > 0)
	{
		StateLayer = UE::FSdfLayer::FindOrOpen(*UsdToUnreal::ConvertString(ExistingUESessionStateIdentifier));
	}

	// We only need to add as sublayer when creating the StateLayer layers, because they are always transient and never saved/loaded from disk
	// so if it exists already, it was created right here, where we add it as a sublayer
	if (!StateLayer && bCreateIfNeeded)
	{
		pxr::SdfLayerRefPtr UsdStateLayer = pxr::SdfLayer::CreateAnonymous(UsdExpectedStateLayerDisplayName);
		UsdSessionLayer->InsertSubLayerPath(UsdStateLayer->GetIdentifier(), 0);	   // Always add it at the front, so it's stronger than the persistent
																				   // layer

		StateLayer = UsdStateLayer;
	}

	return StateLayer;
}

UE::FSdfLayer UsdUtils::FindLayerForIdentifier(const TCHAR* Identifier, const UE::FUsdStage& Stage)
{
	FScopedUsdAllocs UsdAllocs;

	std::string IdentifierStr = UnrealToUsd::ConvertString(Identifier).Get();
	if (pxr::SdfLayer::IsAnonymousLayerIdentifier(IdentifierStr))
	{
		std::string DisplayName = pxr::SdfLayer::GetDisplayNameFromIdentifier(IdentifierStr);

		if (pxr::UsdStageRefPtr UsdStage = static_cast<pxr::UsdStageRefPtr>(Stage))
		{
			const bool bIncludeSessionLayers = true;
			for (const pxr::SdfLayerHandle& Layer : UsdStage->GetLayerStack(bIncludeSessionLayers))
			{
				if (Layer->GetDisplayName() == DisplayName)
				{
					return UE::FSdfLayer{Layer};
				}
			}
		}
	}
	else
	{
		if (pxr::SdfLayerRefPtr Layer = pxr::SdfLayer::FindOrOpen(IdentifierStr))
		{
			return UE::FSdfLayer{Layer};
		}
	}

	return UE::FSdfLayer{};
}

bool UsdUtils::IsSessionLayerWithinStage(const pxr::SdfLayerRefPtr& Layer, const pxr::UsdStageRefPtr& Stage)
{
	if (!Layer || !Stage)
	{
		return false;
	}

	pxr::SdfLayerRefPtr RootLayer = Stage->GetRootLayer();

	const bool bIncludeSessionLayers = true;
	for (const pxr::SdfLayerHandle& ExistingLayer : Stage->GetLayerStack(bIncludeSessionLayers))
	{
		// All session layers come before the root layer within the layer stack
		// Break before we compare with Layer because if Layer is the actual stage's RootLayer we want to return false
		if (ExistingLayer == RootLayer)
		{
			break;
		}

		if (ExistingLayer == Layer)
		{
			return true;
		}
	}

	return false;
}

UE::FSdfLayer UsdUtils::FlattenLayerStack(const pxr::UsdStageRefPtr& Stage)
{
	FScopedUsdAllocs Allocs;
	return UE::FSdfLayer{pxr::UsdUtilsFlattenLayerStack(Stage)};
}

void UsdUtils::ConvertAssetRelativePathsToAbsolute(UE::FSdfLayer& LayerToConvert, const UE::FSdfLayer& AnchorLayer)
{
	if (!LayerToConvert || !AnchorLayer)
	{
		return;
	}

	FScopedUsdAllocs Allocs;

	pxr::SdfLayerRefPtr UsdLayer{LayerToConvert};

	UsdLayer->Traverse(
		pxr::SdfPath::AbsoluteRootPath(),
		[&UsdLayer, &AnchorLayer](const pxr::SdfPath& Path)
		{
			pxr::SdfSpecType SpecType = UsdLayer->GetSpecType(Path);
			if (SpecType != pxr::SdfSpecTypeAttribute)
			{
				return;
			}

			pxr::VtValue LayerValue;
			if (!UsdLayer->HasField(Path, pxr::SdfFieldKeys->Default, &LayerValue))
			{
				return;
			}

			if (UE::UsdLayerUtilsImpl::Private::FixAssetPaths(pxr::SdfLayerRefPtr{AnchorLayer}, &LayerValue))
			{
				UsdLayer->SetField(Path, pxr::SdfFieldKeys->Default, LayerValue);
			}
		}
	);
}

int32 UsdUtils::GetSdfLayerNumFrames(const pxr::SdfLayerRefPtr& Layer)
{
	if (!Layer)
	{
		return 0;
	}

	// USD time code range is inclusive on both ends
	return FMath::Abs(FMath::CeilToInt32(Layer->GetEndTimeCode()) - FMath::FloorToInt32(Layer->GetStartTimeCode()) + 1);
}

#undef LOCTEXT_NAMESPACE

#endif	  // #if USE_USD_SDK
