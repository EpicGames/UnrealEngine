// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDLightConversion.h"

#include "USDAssetCache3.h"
#include "USDAssetUserData.h"
#include "USDAttributeUtils.h"
#include "USDClassesModule.h"
#include "USDConversionUtils.h"
#include "USDErrorUtils.h"
#include "USDLayerUtils.h"
#include "USDMemory.h"
#include "USDObjectUtils.h"
#include "USDShadeConversion.h"
#include "USDTypesConversion.h"

#include "UsdWrappers/UsdAttribute.h"
#include "UsdWrappers/UsdPrim.h"

#include "Components/DirectionalLightComponent.h"
#include "Components/LightComponent.h"
#include "Components/PointLightComponent.h"
#include "Components/RectLightComponent.h"
#include "Components/SkyLightComponent.h"
#include "Components/SpotLightComponent.h"
#include "EditorFramework/AssetImportData.h"
#include "Engine/TextureCube.h"
#include "Misc/Paths.h"
#include "RenderUtils.h"

#if USE_USD_SDK

#include "USDIncludesStart.h"
#include "pxr/usd/usdLux/diskLight.h"
#include "pxr/usd/usdLux/distantLight.h"
#include "pxr/usd/usdLux/domeLight.h"
#include "pxr/usd/usdLux/domeLight_1.h"
#include "pxr/usd/usdLux/lightAPI.h"
#include "pxr/usd/usdLux/rectLight.h"
#include "pxr/usd/usdLux/shadowAPI.h"
#include "pxr/usd/usdLux/shapingAPI.h"
#include "pxr/usd/usdLux/sphereLight.h"
#include "USDIncludesEnd.h"

#define LOCTEXT_NAMESPACE "USDLightConversion"

static float GMinLightSourceSize = 1.0f;
static FAutoConsoleVariableRef CVarMinLightSourceSize(
	TEXT("USD.LightExport.MinLightSourceSize"),
	GMinLightSourceSize,
	TEXT("Minimum source size (in centimeters) for point, spot, and rect lights when exporting to USD. "
		 "Lights with source dimensions below this value will have their dimensions clamped to this minimum "
		 "before writing to USD, preventing extreme intensity values from near-zero surface area. "
		 "For point/spot lights this clamps SourceRadius; for rect lights this clamps SourceWidth and SourceHeight. "
		 "Set to 0 to disable clamping. Default: 1.0")
);

namespace LightConversionImpl
{
	// Copied from USpotLightComponent::GetCosHalfConeAngle, so we don't need a component to do the same math
	float GetSpotLightCosHalfConeAngle(float OuterConeAngle, float InnerConeAngle)
	{
		const float ClampedInnerConeAngle = FMath::Clamp(InnerConeAngle, 0.0f, 89.0f) * (float)PI / 180.0f;
		const float HalfConeAngle = FMath::Clamp(
			OuterConeAngle * (float)PI / 180.0f,
			ClampedInnerConeAngle + 0.001f,
			89.0f * (float)PI / 180.0f + 0.001f
		);
		return FMath::Cos(HalfConeAngle);
	}
}	 // namespace LightConversionImpl

template<typename T>
T UsdUtils::GetLightAttrValueWithInputsFallback(const pxr::UsdAttribute& Attribute, double UsdTimeCode, bool* bOutIsAuthored)
{
	if (bOutIsAuthored)
	{
		*bOutIsAuthored = false;
	}

	if (!Attribute)
	{
		return {};
	}

	pxr::UsdTimeCode TimeCode{UsdTimeCode};
	pxr::UsdAttribute AttributeToUse = Attribute;
	bool bIsAuthored = Attribute.GetResolveInfo(TimeCode).GetSource() != pxr::UsdResolveInfoSourceFallback;

	if (!bIsAuthored)
	{
		std::string AttrName = Attribute.GetName();
		const static std::string Prefix = "inputs:";
		if (AttrName.starts_with(Prefix))
		{
			AttrName = AttrName.substr(Prefix.length());
			pxr::UsdAttribute FallbackAttr = Attribute.GetPrim().GetAttribute(pxr::TfToken{AttrName});
			if (FallbackAttr)
			{
				AttributeToUse = FallbackAttr;
				bIsAuthored = FallbackAttr.GetResolveInfo(TimeCode).GetSource() != pxr::UsdResolveInfoSourceFallback;
			}
		}
	}

	if (bOutIsAuthored)
	{
		*bOutIsAuthored = bIsAuthored;
	}

	T Result;
	if (AttributeToUse && AttributeToUse.Get(&Result, TimeCode))
	{
		return Result;
	}

	return {};
}

template USDUTILITIES_API float UsdUtils::GetLightAttrValueWithInputsFallback<float>(const pxr::UsdAttribute&, double, bool*);
template USDUTILITIES_API bool UsdUtils::GetLightAttrValueWithInputsFallback<bool>(const pxr::UsdAttribute&, double, bool*);
template USDUTILITIES_API pxr::GfVec3f UsdUtils::GetLightAttrValueWithInputsFallback<pxr::GfVec3f>(const pxr::UsdAttribute&, double, bool*);

bool UsdToUnreal::ConvertLight(const pxr::UsdPrim& Prim, ULightComponentBase& LightComponentBase, double UsdTimeCode)
{
	FScopedUsdAllocs Allocs;

	const pxr::UsdLuxLightAPI LightAPI(Prim);
	if (!LightAPI)
	{
		return false;
	}

	const float UsdIntensity = UsdUtils::GetLightAttrValueWithInputsFallback<float>(LightAPI.GetIntensityAttr(), UsdTimeCode);
	const float UsdExposure = UsdUtils::GetLightAttrValueWithInputsFallback<float>(LightAPI.GetExposureAttr(), UsdTimeCode);
	const pxr::GfVec3f UsdColor = UsdUtils::GetLightAttrValueWithInputsFallback<pxr::GfVec3f>(LightAPI.GetColorAttr(), UsdTimeCode);

	const bool bSRGB = true;
	LightComponentBase.LightColor = UsdToUnreal::ConvertColor(UsdColor).ToFColor(bSRGB);
	LightComponentBase.Intensity = UsdToUnreal::ConvertLightIntensity(UsdIntensity, UsdExposure);

	if (ULightComponent* LightComponent = Cast<ULightComponent>(&LightComponentBase))
	{
		LightComponent->bUseTemperature = UsdUtils::GetLightAttrValueWithInputsFallback<bool>(LightAPI.GetEnableColorTemperatureAttr(), UsdTimeCode);
		LightComponent->Temperature = UsdUtils::GetLightAttrValueWithInputsFallback<float>(LightAPI.GetColorTemperatureAttr(), UsdTimeCode);
	}

	if (const pxr::UsdLuxShadowAPI ShadowAPI{Prim})
	{
		LightComponentBase.SetCastShadows(UsdUtils::GetLightAttrValueWithInputsFallback<bool>(ShadowAPI.GetShadowEnableAttr(), UsdTimeCode));
	}

	return true;
}

bool UsdToUnreal::ConvertDistantLight(const pxr::UsdPrim& Prim, UDirectionalLightComponent& LightComponent, double UsdTimeCode)
{
	FScopedUsdAllocs Allocs;

	pxr::UsdLuxDistantLight DistantLight{Prim};
	if (!DistantLight)
	{
		return false;
	}

	LightComponent.LightSourceAngle = UsdUtils::GetLightAttrValueWithInputsFallback<float>(DistantLight.GetAngleAttr(), UsdTimeCode);

	return true;
}

bool UsdToUnreal::ConvertRectLight(const pxr::UsdPrim& Prim, URectLightComponent& LightComponent, double UsdTimeCode)
{
	FScopedUsdAllocs Allocs;

	pxr::UsdLuxRectLight RectLight{Prim};
	if (!RectLight)
	{
		return false;
	}

	const FUsdStageInfo StageInfo(Prim.GetStage());

	const float UsdIntensity = UsdUtils::GetLightAttrValueWithInputsFallback<float>(RectLight.GetIntensityAttr(), UsdTimeCode);
	const float UsdExposure = UsdUtils::GetLightAttrValueWithInputsFallback<float>(RectLight.GetExposureAttr(), UsdTimeCode);
	const float UsdWidth = UsdUtils::GetLightAttrValueWithInputsFallback<float>(RectLight.GetWidthAttr(), UsdTimeCode);
	const float UsdHeight = UsdUtils::GetLightAttrValueWithInputsFallback<float>(RectLight.GetHeightAttr(), UsdTimeCode);

	const double NitsIntensity = UsdToUnreal::ConvertLightIntensity(UsdIntensity, UsdExposure);

	LightComponent.SourceWidth = UsdToUnreal::ConvertDistance(StageInfo, UsdWidth);
	LightComponent.SourceHeight = UsdToUnreal::ConvertDistance(StageInfo, UsdHeight);
	LightComponent.Intensity = NitsIntensity;
	LightComponent.IntensityUnits = ELightUnits::Nits;

	return true;
}

bool UsdToUnreal::ConvertDiskLight(const pxr::UsdPrim& Prim, URectLightComponent& LightComponent, double UsdTimeCode)
{
	FScopedUsdAllocs Allocs;

	pxr::UsdLuxDiskLight DiskLight{Prim};
	if (!DiskLight)
	{
		return false;
	}

	const FUsdStageInfo StageInfo(Prim.GetStage());

	const float UsdIntensity = UsdUtils::GetLightAttrValueWithInputsFallback<float>(DiskLight.GetIntensityAttr(), UsdTimeCode);
	const float UsdExposure = UsdUtils::GetLightAttrValueWithInputsFallback<float>(DiskLight.GetExposureAttr(), UsdTimeCode);
	const float UsdRadius = UsdUtils::GetLightAttrValueWithInputsFallback<float>(DiskLight.GetRadiusAttr(), UsdTimeCode);

	const float RadiusUE = UsdToUnreal::ConvertDistance(StageInfo, UsdRadius);
	const float SquareSideUE = UsdUtils::GetRectLightSideFromDiskLight(RadiusUE);
	const double NitsIntensity = UsdToUnreal::ConvertLightIntensity(UsdIntensity, UsdExposure);

	LightComponent.SourceWidth = SquareSideUE;
	LightComponent.SourceHeight = SquareSideUE;
	LightComponent.Intensity = NitsIntensity;
	LightComponent.IntensityUnits = ELightUnits::Nits;

	return true;
}

bool UsdToUnreal::ConvertSphereLight(const pxr::UsdPrim& Prim, UPointLightComponent& LightComponent, double UsdTimeCode)
{
	FScopedUsdAllocs Allocs;

	pxr::UsdLuxSphereLight SphereLight{Prim};
	if (!SphereLight)
	{
		return false;
	}

	const FUsdStageInfo StageInfo(Prim.GetStage());

	const float UsdIntensity = UsdUtils::GetLightAttrValueWithInputsFallback<float>(SphereLight.GetIntensityAttr(), UsdTimeCode);
	const float UsdExposure = UsdUtils::GetLightAttrValueWithInputsFallback<float>(SphereLight.GetExposureAttr(), UsdTimeCode);
	const float UsdRadius = UsdUtils::GetLightAttrValueWithInputsFallback<float>(SphereLight.GetRadiusAttr(), UsdTimeCode);
	const double TreatAsPoint = UsdUtils::GetUsdValue<bool>(SphereLight.GetTreatAsPointAttr(), UsdTimeCode);

	const float RadiusUE = UsdToUnreal::ConvertDistance(StageInfo, UsdRadius);
	const double NitsIntensity = UsdToUnreal::ConvertLightIntensity(UsdIntensity, UsdExposure);

	LightComponent.SourceRadius = RadiusUE;
	LightComponent.Intensity = NitsIntensity;
	LightComponent.IntensityUnits = ELightUnits::Nits;

	return true;
}

bool UsdToUnreal::ConvertLuxShapingAPI(const pxr::UsdPrim& Prim, USpotLightComponent& LightComponent, double UsdTimeCode)
{
	FScopedUsdAllocs Allocs;

	if (!Prim.HasAPI<pxr::UsdLuxShapingAPI>())
	{
		return false;
	}

	pxr::UsdLuxShapingAPI ShapingAPI{Prim};
	if (!ShapingAPI)
	{
		return false;
	}

	const float UsdConeAngle = UsdUtils::GetLightAttrValueWithInputsFallback<float>(ShapingAPI.GetShapingConeAngleAttr(), UsdTimeCode);
	const float UsdConeSoftness = UsdUtils::GetLightAttrValueWithInputsFallback<float>(ShapingAPI.GetShapingConeSoftnessAttr(), UsdTimeCode);

	float InnerConeAngle = 0.0f;
	float OuterConeAngle = 1.0f;
	UsdToUnreal::ConvertSpotLightConeAngles(UsdConeAngle, UsdConeSoftness, OuterConeAngle, InnerConeAngle);

	LightComponent.SetInnerConeAngle(InnerConeAngle);
	LightComponent.SetOuterConeAngle(OuterConeAngle);

	return true;
}

float UsdToUnreal::ConvertLightIntensity(float UsdIntensity, float UsdExposure)
{
	return UsdIntensity * FMath::Exp2(UsdExposure);
}

void UsdToUnreal::ConvertSpotLightConeAngles(float InConeAngle, float InConeSoftness, float& OutOuterConeAngle, float& OutInnerConeAngle)
{
	OutOuterConeAngle = InConeAngle;
	OutInnerConeAngle = InConeAngle * (1.0f - InConeSoftness);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
// Deprecated
float UsdToUnreal::ConvertLightIntensityAttr(float UsdIntensity, float UsdExposure)
{
	return ConvertLightIntensity(UsdIntensity, UsdExposure);
}

// Deprecated
float UsdToUnreal::ConvertDistantLightIntensityAttr(float UsdIntensity, float UsdExposure)
{
	return UsdToUnreal::ConvertLightIntensity(UsdIntensity, UsdExposure);
}

// Deprecated
float UsdToUnreal::ConvertRectLightIntensityAttr(
	float UsdIntensity,
	float UsdExposure,
	float UsdWidth,
	float UsdHeight,
	const FUsdStageInfo& StageInfo
)
{
	float UEWidth = UsdToUnreal::ConvertDistance(StageInfo, UsdWidth);
	float UEHeight = UsdToUnreal::ConvertDistance(StageInfo, UsdHeight);

	const float AreaInSqMeters = (UEWidth / 100.f) * (UEHeight / 100.f);

	// Only use PI instead of 2PI because URectLightComponent::SetLightBrightness will use just PI and not 2PI for lumen conversions, due to a cosine
	// distribution c.f. UActorFactoryRectLight::PostSpawnActor, and the PI factor between candela and lumen for rect lights on
	// https://docs.unrealengine.com/en-US/BuildingWorlds/LightingAndShadows/PhysicalLightUnits/index.html#point,spot,andrectlights
	return ConvertLightIntensity(UsdIntensity, UsdExposure) * PI * AreaInSqMeters;	  // Lumen = Nits * (PI sr for area light) * Area
}

// Deprecated
float UsdToUnreal::ConvertDiskLightIntensityAttr(float UsdIntensity, float UsdExposure, float UsdRadius, const FUsdStageInfo& StageInfo)
{
	const float Radius = UsdToUnreal::ConvertDistance(StageInfo, UsdRadius);

	const float AreaInSqMeters = PI * FMath::Square(Radius / 100.f);

	// Only use PI instead of 2PI because URectLightComponent::SetLightBrightness will use just PI and not 2PI for lumen conversions, due to a cosine
	// distribution c.f. UActorFactoryRectLight::PostSpawnActor, and the PI factor between candela and lumen for rect lights on
	// https://docs.unrealengine.com/en-US/BuildingWorlds/LightingAndShadows/PhysicalLightUnits/index.html#point,spot,andrectlights
	return ConvertLightIntensity(UsdIntensity, UsdExposure) * PI * AreaInSqMeters;	  // Lumen = Nits * (PI sr for area light) * Area
}

// Deprecated
float UsdToUnreal::ConvertSphereLightIntensityAttr(float UsdIntensity, float UsdExposure, float UsdRadius, const FUsdStageInfo& StageInfo)
{
	float Radius = UsdToUnreal::ConvertDistance(StageInfo, UsdRadius);

	float SolidAngle = 4.f * PI;

	const float AreaInSqMeters = FMath::Max(4.f * PI * FMath::Square(Radius / 100.f), KINDA_SMALL_NUMBER);

	return ConvertLightIntensity(UsdIntensity, UsdExposure) * SolidAngle * AreaInSqMeters;	  // Lumen = Nits * SolidAngle * Area
}

// Deprecated
float UsdToUnreal::ConvertLuxShapingAPIIntensityAttr(
	float UsdIntensity,
	float UsdExposure,
	float UsdRadius,
	float UsdConeAngle,
	float UsdConeSoftness,
	const FUsdStageInfo& StageInfo
)
{
	float Radius = UsdToUnreal::ConvertDistance(StageInfo, UsdRadius);

	float InnerConeAngle = 0.0f;
	float OuterConeAngle = ConvertConeAngleSoftnessAttr(UsdConeAngle, UsdConeSoftness, InnerConeAngle);

	// c.f. USpotLightComponent::ComputeLightBrightness
	float SolidAngle = 2.f * PI * (1.0f - LightConversionImpl::GetSpotLightCosHalfConeAngle(OuterConeAngle, InnerConeAngle));

	const float AreaInSqMeters = FMath::Max(4.f * PI * FMath::Square(Radius / 100.f), KINDA_SMALL_NUMBER);

	return ConvertLightIntensity(UsdIntensity, UsdExposure) * SolidAngle * AreaInSqMeters;	  // Lumen = Nits * SolidAngle * Area
}

// Deprecated
float UsdToUnreal::ConvertConeAngleSoftnessAttr(float UsdConeAngle, float UsdConeSoftness, float& OutInnerConeAngle)
{
	OutInnerConeAngle = UsdConeAngle * (1.0f - UsdConeSoftness);
	float OuterConeAngle = UsdConeAngle;
	return OuterConeAngle;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool UnrealToUsd::ConvertLightComponent(const ULightComponentBase& LightComponent, pxr::UsdPrim& Prim, double UsdTimeCode)
{
	FScopedUsdAllocs UsdAllocs;

	if (!Prim)
	{
		return false;
	}

	pxr::UsdLuxLightAPI LightAPI(Prim);
	if (!LightAPI)
	{
		return false;
	}

	if (pxr::UsdAttribute Attr = LightAPI.CreateIntensityAttr())
	{
		Attr.Set<float>(LightComponent.Intensity, UsdTimeCode);
		UsdUtils::NotifyIfOverriddenOpinion(Attr);
	}

	// When converting into UE we multiply intensity and exposure together, so when writing back we just
	// put everything in intensity. USD also multiplies those two together, meaning it should end up the same
	if (pxr::UsdAttribute Attr = LightAPI.CreateExposureAttr())
	{
		Attr.Set<float>(0.0f, UsdTimeCode);
		UsdUtils::NotifyIfOverriddenOpinion(Attr);
	}

	if (const ULightComponent* DerivedLightComponent = Cast<ULightComponent>(&LightComponent))
	{
		if (pxr::UsdAttribute Attr = LightAPI.CreateEnableColorTemperatureAttr())
		{
			Attr.Set<bool>(DerivedLightComponent->bUseTemperature, UsdTimeCode);
			UsdUtils::NotifyIfOverriddenOpinion(Attr);
		}

		if (pxr::UsdAttribute Attr = LightAPI.CreateColorTemperatureAttr())
		{
			Attr.Set<float>(DerivedLightComponent->Temperature, UsdTimeCode);
			UsdUtils::NotifyIfOverriddenOpinion(Attr);
		}
	}

	if (pxr::UsdAttribute Attr = LightAPI.CreateColorAttr())
	{
		pxr::GfVec4f LinearColor = UnrealToUsd::ConvertColor(LightComponent.LightColor);
		Attr.Set<pxr::GfVec3f>(pxr::GfVec3f(LinearColor[0], LinearColor[1], LinearColor[2]), UsdTimeCode);
		UsdUtils::NotifyIfOverriddenOpinion(Attr);
	}

	// Only author shadow stuff if we need to, as it involves applying an API schema. We don't want to
	// open up an USD stage -> Change a light intensity and save -> End up adding she shadow API schema and attribute
	// just to put the default value of true
	bool bPrimCastsShadows = true;
	if (pxr::UsdLuxShadowAPI ExistingShadowAPI{Prim})
	{
		if (pxr::UsdAttribute Attr = ExistingShadowAPI.GetShadowEnableAttr())
		{
			bool bEnable = true;
			if (Attr.Get(&bEnable, UsdTimeCode))
			{
				bPrimCastsShadows = bEnable;
			}
		}
	}
	const bool bComponentCastsShadows = static_cast<bool>(LightComponent.CastShadows);
	if (bComponentCastsShadows != bPrimCastsShadows)
	{
		pxr::UsdLuxShadowAPI ShadowAPI = pxr::UsdLuxShadowAPI::Apply(Prim);
		if (pxr::UsdAttribute Attr = ShadowAPI.CreateShadowEnableAttr())
		{
			Attr.Set(bComponentCastsShadows, UsdTimeCode);
			UsdUtils::NotifyIfOverriddenOpinion(Attr);
		}
	}

	return true;
}

bool UnrealToUsd::ConvertDirectionalLightComponent(const UDirectionalLightComponent& LightComponent, pxr::UsdPrim& Prim, double UsdTimeCode)
{
	FScopedUsdAllocs UsdAllocs;

	if (!Prim)
	{
		return false;
	}

	pxr::UsdLuxDistantLight Light{Prim};
	if (!Light)
	{
		return false;
	}

	if (pxr::UsdAttribute Attr = Light.CreateAngleAttr())
	{
		Attr.Set<float>(LightComponent.LightSourceAngle, UsdTimeCode);
		UsdUtils::NotifyIfOverriddenOpinion(Attr);
	}

	return true;
}

bool UnrealToUsd::ConvertRectLightComponent(const URectLightComponent& LightComponent, pxr::UsdPrim& Prim, double UsdTimeCode)
{
	FScopedUsdAllocs UsdAllocs;

	if (!Prim)
	{
		return false;
	}

	pxr::UsdLuxLightAPI LightAPI(Prim);
	if (!LightAPI)
	{
		return false;
	}

	FUsdStageInfo StageInfo(Prim.GetStage());

	const float ClampedWidth = FMath::Max(LightComponent.SourceWidth, GMinLightSourceSize);
	const float ClampedHeight = FMath::Max(LightComponent.SourceHeight, GMinLightSourceSize);
	const float WidthMeters = ClampedWidth / 100.0f;
	const float HeightMeters = ClampedHeight / 100.0f;
	const float AreaSqMeters = UsdUtils::GetRectLightArea(WidthMeters, HeightMeters);

	float SolidAngle = 1.0f;

	// Disk light
	if (pxr::UsdLuxDiskLight DiskLight{Prim})
	{
		SolidAngle = UsdUtils::GetDiskLightSolidAngle();

		if (pxr::UsdAttribute Attr = DiskLight.CreateRadiusAttr())
		{
			const float DiskRadiusUEUnits = UsdUtils::GetDiskLightRadiusFromRectLight(ClampedWidth, ClampedHeight);
			Attr.Set<float>(UnrealToUsd::ConvertDistance(StageInfo, DiskRadiusUEUnits), UsdTimeCode);
			UsdUtils::NotifyIfOverriddenOpinion(Attr);
		}
	}
	// Rect light
	else if (pxr::UsdLuxRectLight RectLight{Prim})
	{
		SolidAngle = UsdUtils::GetRectLightSolidAngle();

		if (pxr::UsdAttribute Attr = RectLight.CreateWidthAttr())
		{
			Attr.Set<float>(UnrealToUsd::ConvertDistance(StageInfo, ClampedWidth), UsdTimeCode);
			UsdUtils::NotifyIfOverriddenOpinion(Attr);
		}

		if (pxr::UsdAttribute Attr = RectLight.CreateHeightAttr())
		{
			Attr.Set<float>(UnrealToUsd::ConvertDistance(StageInfo, ClampedHeight), UsdTimeCode);
			UsdUtils::NotifyIfOverriddenOpinion(Attr);
		}
	}
	else
	{
		return false;
	}

	// Common for both
	if (pxr::UsdAttribute Attr = LightAPI.CreateIntensityAttr())
	{
		float OldUsdIntensity = UsdUtils::GetUsdValue<float>(Attr, UsdTimeCode);

		// Area light with no area probably shouldn't emit any light?
		// It's not possible to set width/height less than 1 via the Details panel anyway, but just in case
		if (FMath::IsNearlyZero(AreaSqMeters))
		{
			OldUsdIntensity = 0.0f;
		}

		const float FinalIntensityNits = UsdUtils::ConvertIntensityToNits(OldUsdIntensity, SolidAngle, AreaSqMeters, LightComponent.IntensityUnits);

		Attr.Set<float>(FinalIntensityNits, UsdTimeCode);
		UsdUtils::NotifyIfOverriddenOpinion(Attr);
	}

	return true;
}

bool UnrealToUsd::ConvertPointLightComponent(const UPointLightComponent& LightComponent, pxr::UsdPrim& Prim, double UsdTimeCode)
{
	FScopedUsdAllocs Allocs;

	if (!Prim)
	{
		return false;
	}

	pxr::UsdLuxSphereLight Light{Prim};
	if (!Light)
	{
		return false;
	}

	FUsdStageInfo StageInfo(Prim.GetStage());

	const float RadiusUE = FMath::Max(LightComponent.SourceRadius, GMinLightSourceSize);
	const float RadiusMeters = RadiusUE / 100.0f;

	if (pxr::UsdAttribute Attr = Light.CreateRadiusAttr())
	{
		Attr.Set<float>(UnrealToUsd::ConvertDistance(StageInfo, RadiusUE), UsdTimeCode);
		UsdUtils::NotifyIfOverriddenOpinion(Attr);
	}

	if (pxr::UsdAttribute Attr = Light.CreateTreatAsPointAttr())
	{
		Attr.Set<bool>(FMath::IsNearlyZero(LightComponent.SourceRadius), UsdTimeCode);
		UsdUtils::NotifyIfOverriddenOpinion(Attr);
	}

	float SolidAngle = 1.0f;
	float AreaSqMeters = 1.0f;
	if (const USpotLightComponent* SpotLightComponent = Cast<const USpotLightComponent>(&LightComponent))
	{
		SolidAngle = UsdUtils::GetSpotLightSolidAngle(SpotLightComponent->OuterConeAngle, SpotLightComponent->InnerConeAngle);
		AreaSqMeters = UsdUtils::GetSpotLightArea(RadiusMeters);
	}
	else
	{
		SolidAngle = UsdUtils::GetPointLightSolidAngle();
		AreaSqMeters = UsdUtils::GetPointLightArea(RadiusMeters);
	}

	if (pxr::UsdAttribute Attr = Light.CreateIntensityAttr())
	{
		const float OldUsdIntensity = UsdUtils::GetUsdValue<float>(Attr, UsdTimeCode);
		const float FinalIntensityNits = UsdUtils::ConvertIntensityToNits(OldUsdIntensity, SolidAngle, AreaSqMeters, LightComponent.IntensityUnits);

		Attr.Set<float>(FinalIntensityNits, UsdTimeCode);
		UsdUtils::NotifyIfOverriddenOpinion(Attr);
	}

	return true;
}

bool UnrealToUsd::ConvertSkyLightComponent(const USkyLightComponent& LightComponent, pxr::UsdPrim& Prim, double UsdTimeCode)
{
	FScopedUsdAllocs Allocs;

	if (!Prim)
	{
		return false;
	}

	pxr::UsdAttribute TextureFileAttr;
	if (pxr::UsdLuxDomeLight DomeLight{Prim})
	{
		TextureFileAttr = DomeLight.GetTextureFileAttr();
	}
	else if (pxr::UsdLuxDomeLight_1 DomeLight1{Prim})
	{
		TextureFileAttr = DomeLight1.GetTextureFileAttr();
	}

#if WITH_EDITORONLY_DATA
	FUsdStageInfo StageInfo(Prim.GetStage());

	if (TextureFileAttr)
	{
		if (UTextureCube* TextureCube = LightComponent.Cubemap)
		{
			if (UAssetImportData* AssetImportData = TextureCube->AssetImportData)
			{
				FString FilePath = AssetImportData->GetFirstFilename();
				if (!FPaths::FileExists(FilePath))
				{
					USD_LOG_USERWARNING(FText::Format(
						LOCTEXT(
							"SourceCubemapDoesntExist",
							"Used '{0}' as cubemap when converting SkyLightComponent '{1}' onto prim '{2}', but the cubemap does not exist on the filesystem!"
						),
						FText::FromString(FilePath),
						FText::FromString(LightComponent.GetPathName()),
						FText::FromString(UsdToUnreal::ConvertPath(Prim.GetPrimPath()))
					));
				}

				UsdUtils::MakePathRelativeToLayer(UE::FSdfLayer{Prim.GetStage()->GetEditTarget().GetLayer()}, FilePath);
				TextureFileAttr.Set<pxr::SdfAssetPath>(pxr::SdfAssetPath{UnrealToUsd::ConvertString(*FilePath).Get()}, UsdTimeCode);
				UsdUtils::NotifyIfOverriddenOpinion(TextureFileAttr);
			}
		}
	}
#endif	  //  #if WITH_EDITORONLY_DATA

	return true;
}

bool UnrealToUsd::ConvertSpotLightComponent(const USpotLightComponent& LightComponent, pxr::UsdPrim& Prim, double UsdTimeCode)
{
	FScopedUsdAllocs Allocs;

	if (!Prim)
	{
		return false;
	}

	pxr::UsdLuxShapingAPI ShapingAPI = pxr::UsdLuxShapingAPI::Apply(Prim);
	if (!ShapingAPI)
	{
		return false;
	}

	float UsdConeAngle = 1.0f;
	float UsdConeSoftness = 1.0f;
	UnrealToUsd::ConvertSpotLightConeAngles(LightComponent.OuterConeAngle, LightComponent.InnerConeAngle, UsdConeAngle, UsdConeSoftness);

	if (pxr::UsdAttribute ConeAngleAttr = ShapingAPI.CreateShapingConeAngleAttr())
	{
		ConeAngleAttr.Set<float>(UsdConeAngle, UsdTimeCode);
		UsdUtils::NotifyIfOverriddenOpinion(ConeAngleAttr);
	}

	if (pxr::UsdAttribute SoftnessAttr = ShapingAPI.CreateShapingConeSoftnessAttr())
	{
		SoftnessAttr.Set<float>(UsdConeSoftness, UsdTimeCode);
		UsdUtils::NotifyIfOverriddenOpinion(SoftnessAttr);
	}

	return true;
}

void UnrealToUsd::ConvertSpotLightConeAngles(float InOuterConeAngle, float InInnerConeAngle, float& OutConeAngle, float& OutConeSoftness)
{
	// As of March 2021 there doesn't seem to be a consensus on what softness means, according to https://groups.google.com/g/usd-interest/c/A6bc4OZjSB0 
	// We approximate the best look here by trying to convert from inner/outer cone angle to softness according to the renderman docs

	OutConeAngle = InOuterConeAngle;
	OutConeSoftness = FMath::IsNearlyZero(InOuterConeAngle) ? 0.0 : 1.0f - (InInnerConeAngle / InOuterConeAngle);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
// Deprecated
float UnrealToUsd::ConvertLightIntensityProperty(float Intensity)
{
	return Intensity;
}

// Deprecated
float UnrealToUsd::ConvertRectLightIntensityProperty(
	float Intensity,
	float Width,
	float Height,
	const FUsdStageInfo& StageInfo,
	ELightUnits SourceUnits
)
{
	float UsdIntensity = UnrealToUsd::ConvertLightIntensityProperty(Intensity);

	float AreaInSqMeters = (Width / 100.0f) * (Height / 100.0f);

	if (FMath::IsNearlyZero(AreaInSqMeters))
	{
		UsdIntensity = 0.0f;
	}

	AreaInSqMeters = FMath::Max(AreaInSqMeters, KINDA_SMALL_NUMBER);

	// For area lights sr is technically 2PI, but we cancel that with an
	// extra factor of 2.0 here because URectLightComponent::SetLightBrightness uses just PI and not 2PI as steradian
	// due to some cosine distribution. This also matches the PI factor between candelas and lumen for rect lights on
	// https://docs.unrealengine.com/en-US/Engine/Rendering/LightingAndShadows/PhysicalLightUnits/index.html#point,spot,andrectlights
	return UsdUtils::ConvertIntensityToNits(UsdIntensity, PI, AreaInSqMeters, SourceUnits);
}

// Deprecated
float UnrealToUsd::ConvertRectLightIntensityProperty(float Intensity, float Radius, const FUsdStageInfo& StageInfo, ELightUnits SourceUnits)
{
	float UsdIntensity = UnrealToUsd::ConvertLightIntensityProperty(Intensity);

	float AreaInSqMeters = PI * FMath::Square(Radius / 100.0f);

	if (FMath::IsNearlyZero(AreaInSqMeters))
	{
		UsdIntensity = 0.0f;
	}

	AreaInSqMeters = FMath::Max(AreaInSqMeters, KINDA_SMALL_NUMBER);

	// For area lights sr is technically 2PI, but we cancel that with an
	// extra factor of 2.0 here because URectLightComponent::SetLightBrightness uses just PI and not 2PI as steradian
	// due to some cosine distribution. This also matches the PI factor between candelas and lumen for rect lights on
	// https://docs.unrealengine.com/en-US/Engine/Rendering/LightingAndShadows/PhysicalLightUnits/index.html#point,spot,andrectlights
	return UsdUtils::ConvertIntensityToNits(UsdIntensity, PI, AreaInSqMeters, SourceUnits);
}

// Deprecated
float UnrealToUsd::ConvertPointLightIntensityProperty(float Intensity, float SourceRadius, const FUsdStageInfo& StageInfo, ELightUnits SourceUnits)
{
	const float UsdIntensity = UnrealToUsd::ConvertLightIntensityProperty(Intensity);

	const float SolidAngle = 4.f * PI;

	const float AreaInSqMeters = FMath::Max(4.f * PI * FMath::Square(SourceRadius / 100.f), KINDA_SMALL_NUMBER);

	return UsdUtils::ConvertIntensityToNits(UsdIntensity, SolidAngle, AreaInSqMeters, SourceUnits);
}

// Deprecated
float UnrealToUsd::ConvertSpotLightIntensityProperty(
	float Intensity,
	float OuterConeAngle,
	float InnerConeAngle,
	float SourceRadius,
	const FUsdStageInfo& StageInfo,
	ELightUnits SourceUnits
)
{
	const float UsdIntensity = UnrealToUsd::ConvertLightIntensityProperty(Intensity);

	// c.f. USpotLightComponent::ComputeLightBrightness
	const float SolidAngle = 2.f * PI * (1.0f - LightConversionImpl::GetSpotLightCosHalfConeAngle(OuterConeAngle, InnerConeAngle));

	const float AreaInSqMeters = FMath::Max(4.f * PI * FMath::Square(SourceRadius / 100.f), KINDA_SMALL_NUMBER);

	return UsdUtils::ConvertIntensityToNits(UsdIntensity, SolidAngle, AreaInSqMeters, SourceUnits);
}

// Deprecated
float UnrealToUsd::ConvertOuterConeAngleProperty(float OuterConeAngle)
{
	return OuterConeAngle;
}

// Deprecated
float UnrealToUsd::ConvertInnerConeAngleProperty(float InnerConeAngle, float OuterConeAngle)
{
	// Keep in [0, 1] range, where 1 is maximum softness, i.e. inner cone angle is zero
	return FMath::IsNearlyZero(OuterConeAngle) ? 0.0 : 1.0f - InnerConeAngle / OuterConeAngle;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

float UsdUtils::GetRectLightSideFromDiskLight(float DiskLightRadius)
{
	// For area lights the important is the actual area, so when converting the disk lights into unreal rect lights we'll produce a
	// square rect light that provides the same area as the USD Disk.
	//
	// This is needed because it ensures the engine produces the correct intensity value whenever it converts light units,
	// as it has no idea this is a fake disk light area instead

	const double DiskArea = PI * DiskLightRadius * DiskLightRadius;
	const double SquareSide = FMath::Sqrt(DiskArea);
	return SquareSide;
}

float UsdUtils::GetDiskLightRadiusFromRectLight(float RectLightWidth, float RectLightHeight)
{
	const double RectArea = RectLightWidth * RectLightHeight;
	const float DiskRadius = FMath::Sqrt(RectArea / PI);
	return DiskRadius;
}

float UsdUtils::GetRectLightArea(float Width, float Height)
{
	return Width * Height;
}

float UsdUtils::GetDiskLightArea(float Radius)
{
	return PI * Radius * Radius;
}

float UsdUtils::GetPointLightArea(float Radius)
{
	return 4.0f * PI * Radius * Radius;
}

float UsdUtils::GetSpotLightArea(float Radius)
{
	// The "focusing" caused by the spot light is accounted for in the solid angle, not on the area
	return UsdUtils::GetPointLightArea(Radius);
}

float UsdUtils::GetRectLightSolidAngle()
{
	// For area lights sr is technically 2PI, but we cancel that with an
	// extra factor of 2.0 here because URectLightComponent::SetLightBrightness uses just PI and not 2PI as steradian
	// due to some cosine distribution. This also matches the PI factor between candelas and lumen for rect lights on
	// https://docs.unrealengine.com/en-US/Engine/Rendering/LightingAndShadows/PhysicalLightUnits/index.html#point,spot,andrectlights
	return PI;
}

float UsdUtils::GetDiskLightSolidAngle()
{
	// For area lights sr is technically 2PI, but we cancel that with an
	// extra factor of 2.0 here because URectLightComponent::SetLightBrightness uses just PI and not 2PI as steradian
	// due to some cosine distribution. This also matches the PI factor between candelas and lumen for rect lights on
	// https://docs.unrealengine.com/en-US/Engine/Rendering/LightingAndShadows/PhysicalLightUnits/index.html#point,spot,andrectlights
	return PI;
}

float UsdUtils::GetPointLightSolidAngle()
{
	return 4.0f * PI;
}

float UsdUtils::GetSpotLightSolidAngle(float OuterConeAngle, float InnerConeAngle)
{
	// Reference: USpotLightComponent::ComputeLightBrightness
	return 2.0f * PI * (1.0f - LightConversionImpl::GetSpotLightCosHalfConeAngle(OuterConeAngle, InnerConeAngle));
}

float UsdUtils::ConvertIntensityToNits(float Intensity, float Steradians, float AreaInSqMeters, ELightUnits SourceUnits)
{
	AreaInSqMeters = FMath::Max(AreaInSqMeters, KINDA_SMALL_NUMBER);
	Steradians = FMath::Max(Steradians, KINDA_SMALL_NUMBER);

	switch (SourceUnits)
	{
		case ELightUnits::Candelas:
			// Nit = candela / area
			return Intensity / AreaInSqMeters;
		case ELightUnits::Lumens:
			// Nit = lumen / ( sr * area );
			// https://docs.unrealengine.com/en-US/Engine/Rendering/LightingAndShadows/PhysicalLightUnits/index.html#point,spot,andrectlights
			return Intensity / (Steradians * AreaInSqMeters);
		case ELightUnits::EV:
			// Nit = luminance (cd/m2)
			// It is not clear why dividing by the area is necessary here units-wise, but the UE components seem to do this.
			// For example check UPointLightComponent::ComputeLightBrightness() and UPointLightComponent::SetLightBrightness()
			return EV100ToLuminance(Intensity) / AreaInSqMeters;
		case ELightUnits::Nits:
			return Intensity;
		case ELightUnits::Unitless:
			// Nit = ( unitless / 625 ) / area = candela / area
			// https://docs.unrealengine.com/en-US/Engine/Rendering/LightingAndShadows/PhysicalLightUnits/index.html#point,spot,andrectlights
			return (Intensity / 625.0f) / AreaInSqMeters;
		default:
			break;
	}

	return Intensity;
}

float UsdUtils::ConvertIntensityFromNits(float Intensity, float Steradians, float AreaInSqMeters, ELightUnits DestUnits)
{
	switch (DestUnits)
	{
		case ELightUnits::Candelas:
			return Intensity * AreaInSqMeters;
		case ELightUnits::Lumens:
			return Intensity * (Steradians * AreaInSqMeters);
		case ELightUnits::EV:
			return LuminanceToEV100(Intensity * AreaInSqMeters);
		case ELightUnits::Nits:
			return Intensity;
		case ELightUnits::Unitless:
			return Intensity * (625.0f * AreaInSqMeters);
		default:
			break;
	}

	return Intensity;
}

#undef LOCTEXT_NAMESPACE

#endif	  // #if USE_USD_SDK
