// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDMaterialXShaderGraph.h"

#if USE_USD_SDK && ENABLE_USD_MATERIALX

#include "Misc/Paths.h"
#include "USDErrorUtils.h"
#include "USDLayerUtils.h"
#include "USDMemory.h"
#include "USDTypesConversion.h"
#include "UsdWrappers/SdfLayer.h"

#include "USDIncludesStart.h"
#include "pxr/base/tf/token.h"
#include "pxr/usd/sdf/layerUtils.h"
#include "pxr/usd/usd/prim.h"
#include "pxr/usd/usdShade/connectableAPI.h"
#include "pxr/usd/usdShade/input.h"
#include "pxr/usd/usdShade/material.h"
#include "pxr/usd/usdShade/nodeGraph.h"
#include "pxr/usd/usdShade/shader.h"
#include "USDIncludesEnd.h"

#define LOCTEXT_NAMESPACE "USDMaterialXShaderGraph"

namespace mx = MaterialX;

FUsdMaterialXShaderGraph::FUsdMaterialXShaderGraph(const pxr::UsdPrim& UsdShadeMaterialPrim, const TCHAR* RenderContext)
	: MaterialXTypes {
		"boolean",
		"integer",
		"float",
		"color3",
		"color4",
		"vector2",
		"vector3",
		"vector4",
		"matrix33",
		"matrix44",
		"string",
		"filename",
		"geomname",
		"surfaceshader",
		"displacementshader",
		"volumeshader",
		"lightshader",
		"material",
		"none",
		"integerarray",
		"floatarray",
		"color3array",
		"color4array",
		"vector2array",
		"vector3array",
		"vector4array",
		"stringarray",
		"geomnamearray",
	}
	, UsdToMaterialXTypes {
		{"color3f",  "color3"},
		{"color4f",  "color4"},
		{"float2",   "vector2"},
		{"float3",   "vector3"},
		{"vector3f", "vector3"},
		{"vector4f", "vector4"},
		{"int",      "integer"},
		{"bool",     "boolean"},
	}
	, TangentSpaceInputs{
	    //UsdPreviewSurface
		"normal",
		//Standard Surface
		"coat_normal",
		"tangent",
		//Openpbr Surface
	    "geometry_normal",
		"geometry_coat_normal",
		"geometry_tangent",
		"geometry_coat_tangent"
	}
{
	pxr::UsdShadeMaterial UsdShadeMaterial{UsdShadeMaterialPrim};

	if (!UsdShadeMaterial)
	{
		USD_LOG_USERERROR(FText::Format(LOCTEXT("MaterialXShaderGraph_NoUsdShadeMaterial", "Couldn't create a UsdShadeMaterial for the prim {0}"),
						  FText::FromString(UsdToUnreal::ConvertToken(UsdShadeMaterialPrim.GetName()))));
		return;
	}

	try
	{
		Document = mx::createDocument();
		mx::DocumentPtr MaterialXLibrary = mx::createDocument();
		mx::FileSearchPath MaterialXFolder{ TCHAR_TO_UTF8(*FPaths::Combine(FPaths::EngineDir(), TEXT("Binaries"), TEXT("ThirdParty"), TEXT("MaterialX"))) };
		mx::StringSet LoadedLibs = mx::loadLibraries({ "libraries" }, MaterialXFolder, MaterialXLibrary);
		Document->importLibrary(MaterialXLibrary);

		// The path to the folder containing the custom node nodedefs should also be indicated with the PXR_MTLX_PLUGIN_SEARCH_PATHS environment variable 
		std::string PxrMtlxPluginSearchPath = TCHAR_TO_UTF8(*FPlatformMisc::GetEnvironmentVariable(TEXT("PXR_MTLX_PLUGIN_SEARCH_PATHS")));
		if(!PxrMtlxPluginSearchPath.empty())
		{
			mx::DocumentPtr PluginLibrary = mx::createDocument();
			mx::FileSearchPath PluginFolder{ PxrMtlxPluginSearchPath };
			mx::StringSet PluginLibs = mx::loadLibraries({}, PluginFolder, PluginLibrary);
			Document->importLibrary(PluginLibrary);
		}

		CreateMaterial(UsdShadeMaterial, RenderContext);
	}
	catch (const std::exception& Exception)
	{
		USD_LOG_USERERROR(FText::Format(LOCTEXT("MaterialXShaderGraph_GraphFailed", "MaterialX: {0}\n shader graph creation aborted...\n"),
						  FText::FromString(Exception.what())));
	}
}

mx::DocumentPtr FUsdMaterialXShaderGraph::GetDocument() const
{
	return Document;
}

const TArray<FUsdMaterialXShaderGraph::FGeomProp>& FUsdMaterialXShaderGraph::GetGeomPropValueNames() const
{
	return GeomPropValueNames;
}

void FUsdMaterialXShaderGraph::CreateMaterial(const pxr::UsdShadeMaterial& UsdShadeMaterial, const TCHAR* RenderContext)
{
	pxr::TfToken RenderContextToken = pxr::UsdShadeTokens->universalRenderContext;
	if (RenderContext)
	{
		pxr::TfToken ProvidedRenderContextToken = UnrealToUsd::ConvertToken(RenderContext).Get();
		RenderContextToken = ProvidedRenderContextToken;
	}

	pxr::UsdShadeShader SurfaceSource = UsdShadeMaterial.ComputeSurfaceSource(RenderContextToken);
	pxr::UsdShadeShader DisplacementSource = UsdShadeMaterial.ComputeDisplacementSource(RenderContextToken);
	pxr::UsdShadeShader VolumeSource = UsdShadeMaterial.ComputeVolumeSource(RenderContextToken);

	if (!SurfaceSource && !VolumeSource)
	{
		USD_LOG_ERROR(
			TEXT("Couldn't find any surface or volume shaders for the UsdShadeMaterial: '%s'."),
			*UsdToUnreal::ConvertToken(UsdShadeMaterial.GetPrim().GetName())
		);
		return;
	}

	auto CreateShaderGraph = [this](const pxr::UsdShadeShader& ShadeShader, mx::DocumentPtr InDocument) -> mx::NodePtr
	{
		pxr::TfToken NodeDefTokenId;
		ShadeShader.GetShaderId(&NodeDefTokenId);
		mx::NodeDefPtr NodeDef = InDocument->getNodeDef(NodeDefTokenId.GetString());

		// no node definition, just return
		if (!NodeDef)
		{
			return nullptr;
		}

		mx::NodePtr Shader = InDocument->addNodeInstance(NodeDef, ShadeShader.GetPrim().GetName().GetString());

		// if the shadergraph is not valid (probably caused by a missing nodedef or wrong type), just invalidate it all
		if (!ComputeShaderGraph(Shader, ShadeShader, Document))
		{
			Shader = nullptr;
		}

		return Shader;
	};

	// Create and store the input interface names
	GetInterfaceNameInputs(UsdShadeMaterial, Document);

	// Create the shader graph for the surface/volume/displacement shader
	// for the following source materials since we create them after the shader graph, we need to make sure that their name is unique
	// unlike MaterialX, USD allows a same name between a parent and its child
	std::string MaterialName = UsdShadeMaterial.GetPrim().GetName().GetString();
	mx::NodePtr SurfaceMaterial;
	if (SurfaceSource)
	{
		mx::NodePtr SurfaceShader = CreateShaderGraph(SurfaceSource, Document);
		if (SurfaceShader)
		{
			// create the surfacematerial first
			mx::NodeDefPtr SurfaceMaterialNodeDef = Document->getNodeDef("ND_surfacematerial");
			std::string SurfaceMaterialName = Document->getChild(MaterialName) ? Document->createValidChildName(MaterialName) : MaterialName;
			SurfaceMaterial = Document->addNodeInstance(SurfaceMaterialNodeDef, SurfaceMaterialName);

			SurfaceMaterial->setConnectedNode("surfaceshader", SurfaceShader);
		}
	}

	// a displacement shader is connected to a surfacematerial, we have to check if we created it before
	if (DisplacementSource)
	{
		if (mx::NodePtr DisplacementShader = CreateShaderGraph(DisplacementSource, Document))
		{
			if (!SurfaceMaterial)
			{
				mx::NodeDefPtr SurfaceMaterialNodeDef = Document->getNodeDef("ND_surfacematerial");
				std::string DisplacementMaterialName = Document->getChild(MaterialName) ? Document->createValidChildName(MaterialName) : MaterialName;
				SurfaceMaterial = Document->addNodeInstance(SurfaceMaterialNodeDef, DisplacementMaterialName);
			}
			SurfaceMaterial->setConnectedNode("displacementshader", DisplacementShader);
		}
	}

	if (SurfaceMaterial)
	{
		std::string ErrorMessage;
		SurfaceMaterial->validate(&ErrorMessage);
		if (!ErrorMessage.empty())
		{
			USD_LOG_ERROR(TEXT("USD MaterialX: %s."), UTF8_TO_TCHAR(ErrorMessage.c_str()));
			Document->removeNode(UsdShadeMaterial.GetPrim().GetName().GetString());
			SurfaceMaterial = nullptr;
		}
	}

	// a volume material is another type of material different from a surface material, it's unclear if we can have both a surface and volume in the same Material
	// if that's the case let's just suffix the name of the volume material to avoid collision during the creation of the node
	if (VolumeSource)
	{
		TGuardValue<bool> GuardSparseVolume{ bConvertGeomPropValueToSparseVolume, true };
		if (mx::NodePtr VolumeShader = CreateShaderGraph(VolumeSource, Document))
		{
			mx::NodeDefPtr VolumeMaterialNodeDef = Document->getNodeDef("ND_volumematerial");
			std::string VolumeMaterialName = Document->getChild(MaterialName) ? Document->createValidChildName(MaterialName) : MaterialName;
			mx::NodePtr VolumeMaterial = Document->addNodeInstance(VolumeMaterialNodeDef, VolumeMaterialName);

			VolumeMaterial->setConnectedNode("volumeshader", VolumeShader);

			std::string ErrorVolumeMessage;
			VolumeMaterial->validate(&ErrorVolumeMessage);
			if (!ErrorVolumeMessage.empty())
			{
				USD_LOG_ERROR(TEXT("USD MaterialX: %s."), UTF8_TO_TCHAR(ErrorVolumeMessage.c_str()));
				Document->removeNode(VolumeMaterialName);
				VolumeMaterial = nullptr;
			}
		}
	}

}

bool FUsdMaterialXShaderGraph::ComputeShaderGraph(mx::NodePtr Node, const pxr::UsdShadeShader& Shader, mx::GraphElementPtr Graph)
{
	if (!Node)
	{
		return false;
	}

	TUsdStore<std::vector<pxr::UsdShadeInput>> UsdInputs = Shader.GetInputs();

	for (const pxr::UsdShadeInput& UsdInput : *UsdInputs)
	{
		// Basically the idea here is to traverse the entire graph coming from a tangent space input of a surface shader (e.g: 'normal' of
		// <standard_surface>) if we come from a TS input, all geompropvalue along the path needs to be set in TS since we can only set at init-time
		// the value of a TGuardValue and we're in a recursive state, if bTangentSpaceInput is false we set its value to IsTangentSpace otherwise we
		// keep it as is (meaning true)
		TGuardValue<bool> GuardInput(bTangentSpaceInput, bTangentSpaceInput || IsTangentSpaceInput(UsdInput));

		mx::InputPtr Input = GetInput(Node, UsdInput);

		bool bHasValue = true;
		TUsdStore<pxr::UsdShadeInput::SourceInfoVector> ConnectedSources = UsdInput.GetConnectedSources();

		// At this point an input should end up in either of these 3 cases, with the precedence in that order:
		// 1. it has an interfaceName, if that's the case we continue to the next input (it has no value, no nodename)
		// 2. it is connected to a nodename with a valid nodedef (it has no value)
		// 3. it has a value, if not defined by the input, the default one from the nodedef is taken

		// Let's loop over the connected sources, even though we should expect only one
		for (const pxr::UsdShadeConnectionSourceInfo& Source : *ConnectedSources)
		{
			// we'll process the rest of the shader graph inside the nodegraph
			if (mx::NodeGraphPtr NodeGraph = GetNodeGraph(Source, Graph))
			{
				Input->setNodeGraphString(NodeGraph->getName());
				Input->setOutputString(Source.sourceName);
				bHasValue = false;
				continue;
			}

			if (SetInterfaceName(Input, Source, Graph))
			{
				bHasValue = false;
				break;
			}

			bool bIsNodeAlreadyExist;
			mx::NodePtr ConnectedNode = GetNode(Source, Graph, bIsNodeAlreadyExist);
			// Let's set if the input has an interfaceName
			if (!ConnectedNode)
			{
				break;
			}

			Input->setConnectedNode(ConnectedNode);
			std::string OutputName = Source.sourceName.GetString();

			// we can only assume at this point that the output for a sparse volume is the 'Attributes A'
			if (ConnectedNode->getCategory() == "sparse_volume")
			{
				OutputName = "outA";
			}

			// no need to add the default output
			if (OutputName != "out")
			{
				Input->setOutputString(OutputName);
			}

			// recurse if it's a newly created node, otherwise the work has already been done
			if (!bIsNodeAlreadyExist)
			{
				bHasValue = ComputeShaderGraph(ConnectedNode, Source.source, Graph);
			}

			bHasValue = false;
		}

		if (bHasValue)
		{
			SetInputValue(Input, UsdInput);
		}
	}

	return true;
}

bool FUsdMaterialXShaderGraph::IsGeomColor(const pxr::UsdShadeShader& GeomPropValueShader)
{
	if (pxr::UsdShadeInput GeomPropInput = GeomPropValueShader.GetInput(pxr::TfToken{"geomprop"}))
	{
		if (TUsdStore<std::string> Value; GeomPropInput.Get<std::string>(&Value.Get()))
		{
			if (*Value == "displayColor" || *Value == "displayOpacity")
			{
				return true;
			}
		}
	}
	return false;
}

void FUsdMaterialXShaderGraph::SetInputValue(mx::InputPtr Input, const pxr::UsdShadeInput& UsdInput)
{
	auto SetInputValue = [&UsdInput, Input](auto Type)
	{
		using MtlxType = decltype(Type);

		using FPairingUsdMtlxType = FPairingUsdMtlxType_t<MtlxType>;

		if (FPairingUsdMtlxType Value; UsdInput.Get<FPairingUsdMtlxType>(&Value))
		{
			if constexpr (std::is_same_v<FPairingUsdMtlxType, pxr::GfVec4f> ||
						  std::is_same_v<FPairingUsdMtlxType, pxr::GfVec3f> ||
						  std::is_same_v<FPairingUsdMtlxType, pxr::GfVec2f>)
			{
				Input->setValue(MtlxType{Value.data(), Value.data() + FPairingUsdMtlxType::dimension});
			}
			else
			{
				Input->setValue(Value);
			}
		}
		else if (mx::NodePtr Node = Input->getParent()->asA<mx::Node>())
		{
			mx::NodeDefPtr NodeDef = Node->getNodeDef();
			mx::InputPtr ActiveInput = NodeDef->getActiveInput(Input->getName());
			if (ActiveInput->hasDefaultGeomPropString())
			{
				Node->removeInput(Input->getName());
			}
			else
			{
				MtlxType DefaultValue = ActiveInput->getDefaultValue()->asA<MtlxType>();
				Input->setValue(DefaultValue);
			}
		}
	};

	const std::string& InputType = Input->getType();
	if (InputType == "float")
	{
		SetInputValue(float{});
	}
	else if (InputType == "integer")
	{
		SetInputValue(int32{});
	}
	else if (InputType == "boolean")
	{
		SetInputValue(bool{});
	}
	else if (InputType == "vector2")
	{
		SetInputValue(mx::Vector2{});
	}
	else if (InputType == "vector3")
	{
		SetInputValue(mx::Vector3{});
	}
	else if (InputType == "vector4")
	{
		SetInputValue(mx::Vector4{});
	}
	else if (InputType == "color3")
	{
		SetInputValue(mx::Color3{});
	}
	else if (InputType == "color4")
	{
		SetInputValue(mx::Color4{});
	}
	else if (InputType == "string")
	{
		if (TUsdStore<std::string> Value; UsdInput.Get<std::string>(&Value.Get()))
		{
			Input->setValue(Value.Get());
		}
		else if (mx::NodePtr Node = Input->getParent()->asA<mx::Node>())
		{
			mx::NodeDefPtr NodeDef = Node->getNodeDef();
			mx::InputPtr ActiveInput = NodeDef->getActiveInput(Input->getName());
			std::string DefaultValue = ActiveInput->getDefaultValue()->asA<std::string>();
			Input->setValue(DefaultValue);
		}
	}
	else if (InputType == "filename")
	{
		FString PathToResolve;
		if (pxr::SdfAssetPath AssetPath; UsdInput.Get<pxr::SdfAssetPath>(&AssetPath))
		{
			TUsdStore<UE::FSdfLayer> Layer = UsdUtils::FindLayerForAttribute(UsdInput.GetAttr(), pxr::UsdTimeCode::Default().GetValue());

			PathToResolve = Layer.Get()
								? UsdToUnreal::ConvertString(
									  TUsdStore<std::string>(pxr::SdfComputeAssetPathRelativeToLayer(Layer.Get(), AssetPath.GetAssetPath())).Get()
								  )
								: UsdToUnreal::ConvertString(AssetPath.GetAssetPath());
		}
		Input->setValueString(std::string{TCHAR_TO_UTF8(*PathToResolve)});
	}
	else
	{
		USD_LOG_WARNING(TEXT("Couldn't find a value type for (%s)."), UTF8_TO_TCHAR(InputType.c_str()));
	}
}

bool FUsdMaterialXShaderGraph::SetInterfaceName(mx::InputPtr Input, const pxr::UsdShadeConnectionSourceInfo& Source, mx::GraphElementPtr Graph)
{
	bool bIsSet = false;
	const std::string& SourceName = Source.sourceName.GetString();

	// the test might not be needed since we always add at least the Document inside the InterfaceNames
	if (auto It = InterfaceNames.find(Graph); It != InterfaceNames.end())
	{
		if (It->second.find(SourceName) != It->second.end())
		{
			if (mx::InputPtr InputInterfaceName = It->first->getInput(SourceName))
			{
				Input->setInterfaceName(InputInterfaceName->getName());
				bIsSet = true;
			}
		}
	}
	return bIsSet;
}

mx::NodePtr FUsdMaterialXShaderGraph::GetNode(const pxr::UsdShadeConnectionSourceInfo& Source, mx::GraphElementPtr Graph, bool& bIsNodeAlreadyExist)
{
	bool bIsGeomPropValue;
	bIsNodeAlreadyExist = true;
	mx::NodeDefPtr NodeDef = GetNodeDef(Source, bIsGeomPropValue);

	if (!NodeDef)
	{
		return nullptr;
	}

	std::string ConnectedShaderName = Source.source.GetPrim().GetName().GetString();

	mx::NodePtr ConnectedNode = Graph->getNode(ConnectedShaderName);

	// if the node is not present in the graph, we create it and notify that it's a new node
	if (!ConnectedNode)
	{
		ConnectedNode = Graph->addNodeInstance(NodeDef, ConnectedShaderName);
		if (bIsGeomPropValue)
		{
			ConnectedNode->setTypedAttribute("UE:GeomPropImage", bIsGeomPropValue);
		}
		else if (ConnectedNode->getCategory() == "sparse_volume")
		{
			// the name of the geompropvalue (which is the name of a volume primvar), in that case the name that the SparseVolumeAsset will be named at
			// we'll use that name later in the ExecutePostFactoryPipeline to assign the asset to the SparseVolumeTextureSample
			TUsdStore<std::string> GeomPropName;
			pxr::UsdShadeShader{ Source.source.GetPrim() }.GetInput(pxr::TfToken{ "geomprop" }).Get<std::string>(&(*GeomPropName));
			ConnectedNode->setAttribute("UE:GeomPropName", *GeomPropName);
		}
		bIsNodeAlreadyExist = false;
	}

	return ConnectedNode;
}

mx::InputPtr FUsdMaterialXShaderGraph::GetInput(mx::NodePtr Node, const pxr::UsdShadeInput& UsdInput)
{
	std::string InputName = UsdInput.GetBaseName().GetString();
	std::string InputType = UsdInput.GetTypeName().GetAsToken().GetString();

	bool bMatch = GetMatchingType(InputName, InputType);

	if (!bMatch && InputName == "geomprop")
	{
		if (Node->getCategory() == "geomcolor")
		{
			InputName = "index";
			InputType = "integer";
		}
		else
		{
			InputName = "file";
			InputType = "filename";
			TUsdStore<std::string> Value;
			UsdInput.Get<std::string>(&Value.Get());
			// Stash the geomprop name (e.g. "specmask", "mask") on the rewritten <image> MaterialX
			// node. The MaterialX node's file input value gets discarded later by SetInputValue's
			// "filename" branch, so we'd otherwise lose the only handle on which primvar this
			// node is meant to consume. Same "UE:GeomPropName" key the <sparse_volume> path uses
			Node->setAttribute("UE:GeomPropName", *Value);
			if (!GeomPropValueNames.FindByPredicate(
					[&Value](const FGeomProp& Other)
					{
						return Other.Name == Value->c_str();
					}
				))
			{
				// we only set the the input as being in Tangent Space if also its type is a vec3
				GeomPropValueNames.Emplace(Value->c_str(), bTangentSpaceInput && Node->getType() == "vector3" ? bTangentSpaceInput : false);
			}
		}
	}
	else // take the input given by the nodedef
	{
		if (mx::NodeDefPtr NodeDef = Node->getNodeDef())
		{
			if (mx::InputPtr Input = NodeDef->getActiveInput(InputName))
			{
				InputType = Input->getType();
			}
		}
	}

	mx::InputPtr Input = Node->addInput(InputName, InputType);

	// retrieve the 'ue_displacement_attribute' field needed by MaterialX, in order to set these values in the detail panel instead of scalars in the shader graph
	// see Engine\Binaries\ThirdParty\MaterialX\libraries\Interchange\Interchange_defs.mtlx for full definition of the attribute
	if (const pxr::UsdAttribute& Attr = UsdInput.GetAttr())
	{
		pxr::UsdMetadataValueMap MetadataValueMap = Attr.GetAllMetadata();
		if (auto CustomData = MetadataValueMap.find(pxr::TfToken("customData")); CustomData != MetadataValueMap.end())
		{
			if (CustomData->second.IsHolding<pxr::VtDictionary>())
			{
				pxr::VtDictionary Dictionary = CustomData->second.Get<pxr::VtDictionary>();
				if (auto DisplacementAttribute = Dictionary.find("ue_displacement_attribute"); DisplacementAttribute != Dictionary.end())
				{
					Input->setTypedAttribute(DisplacementAttribute->first, DisplacementAttribute->second.Get<bool>());
				}
			}
		}
	}

	return Input;
}

mx::NodeDefPtr FUsdMaterialXShaderGraph::GetNodeDef(const pxr::UsdShadeConnectionSourceInfo& Source, bool& bIsGeomPropValue)
{
	pxr::UsdShadeShader UsdConnectedShader{ Source.source.GetPrim() };

	if (!UsdConnectedShader)
	{
		USD_LOG_WARNING(TEXT("The '%s' connected source is not a valid USD shader."), *UsdToUnreal::ConvertString(Source.source.GetPrim().GetName()));
	}

	pxr::TfToken NodeDefTokenId;
	UsdConnectedShader.GetShaderId(&NodeDefTokenId);

	std::string NodeDefString = NodeDefTokenId.GetString();
	bIsGeomPropValue = false;
	if (std::size_t Pos = NodeDefString.find("geompropvalue"); Pos != std::string::npos)
	{
		// we convert a geomprop either to an image or a geomcolor in case if the name is displayColor
		std::string NodeName = "image";
		if (IsGeomColor(UsdConnectedShader))
		{
			NodeName = "geomcolor";
		}
		else
		{
			if (bConvertGeomPropValueToSparseVolume)
			{
				// we convert the geompropvalue to a sparse_volume node
				// at this point we can only assume that the output is attached to the 'Attributes A' of the SparseVolumeTexture
				NodeName = "sparse_volume_A";
			}
			else
			{
				bIsGeomPropValue = true;
			}
			// we replace the nodedef of a <geompropvalue> by either an <image> or a <sparse_volume> 
			// e.g: ND_geompropvalue_vector3
			//		ND_image_vector3
			// or   ND_sparse_volume_A_vector3
			// for integers/bool let's just convert them to float

			if (std::size_t pos = NodeDefString.rfind("integer"); pos != std::string::npos)
			{
				NodeDefString.replace(pos, sizeof("integer") - 1, "float");
			}
			else if (pos = NodeDefString.rfind("boolean"); pos != std::string::npos)
			{
				NodeDefString.replace(pos, sizeof("boolean") - 1, "float");
			}
			else if (pos = NodeDefString.rfind("string"); pos != std::string::npos)
			{
				USD_LOG_WARNING(
					TEXT("'%s': '%s' are not supported"),
					*UsdToUnreal::ConvertString(Source.source.GetPrim().GetName()),
					*UsdToUnreal::ConvertString(NodeDefString)
				);
				return nullptr;
			}
		}
		NodeDefString.replace(Pos, sizeof("geompropvalue") - 1, NodeName);
	}

	mx::NodeDefPtr NodeDef = Document->getNodeDef(NodeDefString);

	if (!NodeDef)
	{
		USD_LOG_WARNING(TEXT("Couldn't find a nodedef for (%s)."), UTF8_TO_TCHAR(NodeDefTokenId.GetString().c_str()));
	}

	return NodeDef;
}

mx::NodeGraphPtr FUsdMaterialXShaderGraph::GetNodeGraph(const pxr::UsdShadeConnectionSourceInfo& Source, mx::GraphElementPtr Graph)
{
	// we only process the nodegraph, if it's connected to a UsdShadeMaterial then we'll process it as a nodename or a value
	if (pxr::UsdShadeMaterial{ Source.source.GetPrim() })
	{
		return nullptr;
	}

	pxr::UsdShadeNodeGraph UsdConnectedNodeGraph{ Source.source.GetPrim() };

	if (!UsdConnectedNodeGraph)
	{
		return nullptr;
	}

	std::string NodeGraphName = UsdConnectedNodeGraph.GetPrim().GetName().GetString();
	mx::NodeGraphPtr NodeGraph = Document->getNodeGraph(NodeGraphName);

	if (NodeGraph)
	{
		return NodeGraph;
	}

	NodeGraph = Document->addNodeGraph(NodeGraphName);
	GetInterfaceNameInputs(UsdConnectedNodeGraph, NodeGraph);

	std::vector<pxr::UsdShadeOutput> UsdShadeOutputs = UsdConnectedNodeGraph.GetOutputs();
	for (const pxr::UsdShadeOutput& UsdShadeOutput : UsdShadeOutputs)
	{
		std::string OutputName = UsdShadeOutput.GetBaseName().GetString();
		std::string OutputType = UsdShadeOutput.GetTypeName().GetAsToken().GetString();

		GetMatchingType(OutputName, OutputType);
		mx::OutputPtr Output = NodeGraph->addOutput(OutputName, OutputType);

		TUsdStore<pxr::UsdShadeOutput::SourceInfoVector> ConnectedSources = UsdShadeOutput.GetConnectedSources();

		for (const pxr::UsdShadeConnectionSourceInfo& ConnectedSource : *ConnectedSources)
		{
			bool bIsNodeAlreadyExist;
			mx::NodePtr ConnectedNode = GetNode(ConnectedSource, NodeGraph, bIsNodeAlreadyExist);
			if (!ConnectedNode)
			{
				break;
			}
			Output->setNodeName(ConnectedNode->getName());
			ComputeShaderGraph(ConnectedNode, ConnectedSource.source, NodeGraph);
		}
	}

	return NodeGraph;
}

bool FUsdMaterialXShaderGraph::GetMatchingType(const std::string& Name, std::string& InOutType)
{
	bool bMatch = false;

	if (MaterialXTypes.find(InOutType) == MaterialXTypes.end())
	{
		if (auto It = UsdToMaterialXTypes.find(InOutType); It != UsdToMaterialXTypes.end())
		{
			InOutType = It->second;
			bMatch = true;
		}
		else if (Name == "file")	 // we're dealing with a texture, USD calls the type "asset" for the input
		{
			InOutType = "filename";
			bMatch = true;
		}
	}

	return bMatch;
}

bool FUsdMaterialXShaderGraph::IsTangentSpaceInput(const pxr::UsdShadeInput& UsdInput) const
{
	return TangentSpaceInputs.find(UsdInput.GetBaseName().GetString()) != TangentSpaceInputs.cend();
}

void FUsdMaterialXShaderGraph::GetInterfaceNameInputs(const pxr::UsdShadeNodeGraph& UsdShadeGraph, mx::GraphElementPtr Graph)
{
	TUsdStore<std::vector<pxr::UsdShadeInput>> InterfaceInputs = UsdShadeGraph.GetInterfaceInputs();
	std::unordered_set<std::string> InterfaceInputNames;
	for (const pxr::UsdShadeInput& InterfaceInput : *InterfaceInputs)
	{
		std::string InterfaceNameInput = InterfaceInput.GetBaseName().GetString();
		std::string InterfaceNameType = InterfaceInput.GetTypeName().GetAsToken().GetString();
		GetMatchingType(InterfaceNameInput, InterfaceNameType);

		mx::InputPtr InputInterfaceName = Graph->addInput(InterfaceNameInput, InterfaceNameType);
		SetInputValue(InputInterfaceName, InterfaceInput);
		InterfaceInputNames.emplace(InputInterfaceName->getName());
	}
	InterfaceNames.emplace(Graph, InterfaceInputNames);
}

#undef LOCTEXT_NAMESPACE 
#endif // USE_USD_SDK && ENABLE_USD_MATERIALX
