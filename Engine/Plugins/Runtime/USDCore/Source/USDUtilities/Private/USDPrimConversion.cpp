// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDPrimConversion.h"

#include "UnrealUSDWrapper.h"
#include "USDAssetUserData.h"
#include "USDAttributeUtils.h"
#include "USDConversionUtils.h"
#include "USDDrawModeComponent.h"
#include "USDErrorUtils.h"
#include "USDLayerUtils.h"
#include "USDLightConversion.h"
#include "USDMemory.h"
#include "USDObjectUtils.h"
#include "USDShadeConversion.h"
#include "USDSkeletalDataConversion.h"
#include "USDTypesConversion.h"
#include "USDValueConversion.h"
#include "UsdWrappers/UsdAttribute.h"
#include "UsdWrappers/UsdGeomBBoxCache.h"
#include "UsdWrappers/UsdGeomXformable.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdStage.h"

#include "Channels/MovieSceneBoolChannel.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "CineCameraActor.h"
#include "CineCameraComponent.h"
#include "Components/AudioComponent.h"
#include "Components/BrushComponent.h"
#include "Components/DirectionalLightComponent.h"
#include "Components/HeterogeneousVolumeComponent.h"
#include "Components/HierarchicalInstancedStaticMeshComponent.h"
#include "Components/LightComponent.h"
#include "Components/LocalLightComponent.h"
#include "Components/MeshComponent.h"
#include "Components/PointLightComponent.h"
#include "Components/RectLightComponent.h"
#include "Components/SceneComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/SkinnedMeshComponent.h"
#include "Components/SpotLightComponent.h"
#include "EditorFramework/AssetImportData.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture2D.h"
#include "GeometryCache.h"
#include "GeometryCacheComponent.h"
#include "InstancedFoliageActor.h"
#include "MovieScene.h"
#include "MovieSceneTimeHelpers.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Sections/MovieScene3DTransformSection.h"
#include "Sections/MovieSceneAudioSection.h"
#include "Sections/MovieSceneBoolSection.h"
#include "Sections/MovieSceneColorSection.h"
#include "Sections/MovieSceneFloatSection.h"
#include "Sections/MovieSceneVectorSection.h"
#include "Sections/MovieSceneVisibilitySection.h"
#include "Sound/SoundBase.h"
#include "Sound/SoundWave.h"
#include "SparseVolumeTexture/SparseVolumeTexture.h"
#include "Tracks/MovieScene3DTransformTrack.h"
#include "Tracks/MovieSceneBoolTrack.h"
#include "Tracks/MovieSceneColorTrack.h"
#include "Tracks/MovieSceneFloatTrack.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "Tracks/MovieSceneVectorTrack.h"
#include "Tracks/MovieSceneVisibilityTrack.h"

#if USE_USD_SDK

#include "USDIncludesStart.h"
#include "pxr/base/tf/stringUtils.h"
#include "pxr/base/vt/value.h"
#include "pxr/usd/kind/registry.h"
#include "pxr/usd/sdf/changeBlock.h"
#include "pxr/usd/sdf/path.h"
#include "pxr/usd/sdf/types.h"
#include "pxr/usd/usd/attribute.h"
#include "pxr/usd/usd/prim.h"
#include "pxr/usd/usd/primRange.h"
#include "pxr/usd/usd/stage.h"
#include "pxr/usd/usd/timeCode.h"
#include "pxr/usd/usdGeom/camera.h"
#include "pxr/usd/usdGeom/capsule.h"
#include "pxr/usd/usdGeom/capsule_1.h"
#include "pxr/usd/usdGeom/cone.h"
#include "pxr/usd/usdGeom/cube.h"
#include "pxr/usd/usdGeom/cylinder.h"
#include "pxr/usd/usdGeom/cylinder_1.h"
#include "pxr/usd/usdGeom/imageable.h"
#include "pxr/usd/usdGeom/mesh.h"
#include "pxr/usd/usdGeom/modelAPI.h"
#include "pxr/usd/usdGeom/plane.h"
#include "pxr/usd/usdGeom/pointInstancer.h"
#include "pxr/usd/usdGeom/scope.h"
#include "pxr/usd/usdGeom/sphere.h"
#include "pxr/usd/usdGeom/tokens.h"
#include "pxr/usd/usdGeom/xform.h"
#include "pxr/usd/usdGeom/xformable.h"
#include "pxr/usd/usdGeom/xformCommonAPI.h"
#include "pxr/usd/usdLux/diskLight.h"
#include "pxr/usd/usdLux/distantLight.h"
#include "pxr/usd/usdLux/domeLight.h"
#include "pxr/usd/usdLux/domeLight_1.h"
#include "pxr/usd/usdLux/lightAPI.h"
#include "pxr/usd/usdLux/rectLight.h"
#include "pxr/usd/usdLux/shadowAPI.h"
#include "pxr/usd/usdLux/shapingAPI.h"
#include "pxr/usd/usdLux/sphereLight.h"
#include "pxr/usd/usdLux/tokens.h"
#include "pxr/usd/usdMedia/spatialAudio.h"
#include "pxr/usd/usdShade/connectableAPI.h"
#include "pxr/usd/usdShade/material.h"
#include "pxr/usd/usdShade/materialBindingAPI.h"
#include "pxr/usd/usdShade/shader.h"
#include "pxr/usd/usdSkel/animation.h"
#include "pxr/usd/usdSkel/binding.h"
#include "pxr/usd/usdSkel/root.h"
#include "pxr/usd/usdSkel/skeleton.h"
#include "pxr/usd/usdSkel/skeletonQuery.h"
#include "pxr/usd/usdUtils/stageCache.h"
#include "pxr/usd/usdVol/openVDBAsset.h"
#include "pxr/usd/usdVol/volume.h"
#include "USDIncludesEnd.h"

#define LOCTEXT_NAMESPACE "USDPrimConversion"

static bool GConsiderAllPrimsHaveAnimatedBounds = false;
static FAutoConsoleVariableRef CVarConsiderAllPrimsHaveAnimatedBounds(
	TEXT("USD.Bounds.ConsiderAllPrimsHaveAnimatedBounds"),
	GConsiderAllPrimsHaveAnimatedBounds,
	TEXT("When active prevents USD from caching computed bounds between timeSamples for any prim, which allows us to force it to recompute accurate "
		 "bounds for cases it does not naturally consider animated (e.g. for animated Mesh points, skeletal animation, etc.). Warning: This can be "
		 "extremely expensive!")
);

static bool GSkipConstantValues = true;
static FAutoConsoleVariableRef CVarSkipConstantValues(
	TEXT("USD.LevelSequenceExport.SkipConstantValues"),
	GSkipConstantValues,
	TEXT(
		"Whether to prevent the exporter from writing out a timeSample when it has the same value as the previous timeSample. Enable this (default) if you want your generated files to have less timeSamples when possible. Disable this if you want each animated attribute to have an authored value for each timeSample of the animation"
	)
);

namespace UE::USDPrimConversion::Private
{
	// On the current edit target, will set the Xformable's op order to a single "xformOp:transform",
	// create the corresponding attribute, and return the op
	pxr::UsdGeomXformOp ForceMatrixXform(pxr::UsdGeomXformable& Xformable)
	{
		FScopedUsdAllocs Allocs;

		// Note: We don't use Xformable.MakeMatrixXform() here because while it can clear the
		// xform op order on the current edit target just fine, it will later try to AddTransformOp(),
		// which calls AddXformOp. Internally, it will read the *composed* prim and if it finds that it already
		// has an op of that type it will early out and not author anything. This means that if our stage
		// has a strong opinion for an e.g. "xformOp:transform" already on the layer stack, it's not possible
		// to author that same op on a weaker layer. We want to do this here, to ensure this prim's transform
		// works as expected even if this weaker layer is used standalone, so we must do the analogous ourselves

		// References: private constructor for UsdGeomXformOp that can receive a UsdPrim and UsdGeomXformable::AddXformOp

		// Find details about the transform attribute related to the default transform type xform op
		pxr::TfToken TransformAttrName = pxr::UsdGeomXformOp::GetOpName(pxr::UsdGeomXformOp::TypeTransform);
		const pxr::SdfValueTypeName& TransformAttrTypeName = pxr::UsdGeomXformOp::GetValueTypeName(
			pxr::UsdGeomXformOp::TypeTransform,
			pxr::UsdGeomXformOp::PrecisionDouble
		);
		if (TransformAttrName.IsEmpty() || !TransformAttrTypeName)
		{
			return {};
		}

		// Create the transform attribute that would match the default transform type xform op
		const bool bCustom = false;
		pxr::UsdPrim UsdPrim = Xformable.GetPrim();
		pxr::UsdAttribute TransformAttr = UsdPrim.CreateAttribute(TransformAttrName, TransformAttrTypeName, bCustom);
		if (!TransformAttr)
		{
			return {};
		}

		// Now that the attribute is created, use it to create the corresponding pxr::UsdGeomXformOp
		const bool bIsInverseOp = false;
		pxr::UsdGeomXformOp NewOp{TransformAttr, bIsInverseOp};
		if (!NewOp)
		{
			return {};
		}

		// If we have other xformOps, let's convert them to matrix xforms as well.
		// This is important as we can end up in edge cases like UE-221328, where we may have some timeSamples for different
		// xformOps that are fully lost when we author a single default opinion for a matrix op, as we'll also force the
		// xformOpOrder to just use our matrix op
		bool bResetsXformStack = false;
		std::vector<pxr::UsdGeomXformOp> ExistingOps = Xformable.GetOrderedXformOps(&bResetsXformStack);
		if (ExistingOps.size() > 0)
		{
			// If there's exactly one op and it's already xformOp:transform, then we don't need to do any conversion
			bool bNeedConversion = false;
			if (ExistingOps.size() > 1)
			{
				bNeedConversion = true;
			}
			else if (ExistingOps[0].GetOpType() != pxr::UsdGeomXformOp::TypeTransform || ExistingOps[0].IsInverseOp())
			{
				bNeedConversion = true;
			}

			if (bNeedConversion)
			{
				// Convert default opinion if any of the Ops have one
				bool bHasDefault = false;
				for (const pxr::UsdGeomXformOp& Op : ExistingOps)
				{
					const pxr::UsdAttribute& OpAttr = Op;
					bHasDefault = OpAttr.GetResolveInfo(pxr::UsdTimeCode::Default()).HasAuthoredValue();
					if (bHasDefault)
					{
						break;
					}
				}
				if (bHasDefault)
				{
					// TODO: Proper handling of bResetsXformStack
					pxr::GfMatrix4d Transform;
					bool bResetsXformStackDefault = false;
					if (Xformable.GetLocalTransformation(&Transform, &bResetsXformStackDefault, pxr::UsdTimeCode::Default()))
					{
						TransformAttr.Set(Transform, pxr::UsdTimeCode::Default());
					}
				}

				// Convert timeSamples
				std::vector<double> TimeSamples;
				if (Xformable.GetTimeSamples(&TimeSamples) && TimeSamples.size() > 0)
				{
					for (double TimeSample : TimeSamples)
					{
						pxr::GfMatrix4d Transform;
						bool bResetsXformStackTimeSample = false;
						if (Xformable.GetLocalTransformation(&Transform, &bResetsXformStackTimeSample, TimeSample))
						{
							TransformAttr.Set(Transform, TimeSample);
						}
					}
				}
			}
		}

		// Clear the existing xform op order for this prim on this layer
		Xformable.ClearXformOpOrder();

		// Store the Op name on an array that will be our new op order value
		pxr::VtTokenArray NewOps;
		NewOps.push_back(NewOp.GetOpName());
		Xformable.CreateXformOpOrderAttr().Set(NewOps);

		return NewOp;
	}

	// Turns OutTransform into the UE-space relative (local to parent) transform for Xformable, paying attention to if it
	// or any of its ancestors has the '!resetXformStack!' xformOp.
	void GetPrimConvertedRelativeTransform(
		pxr::UsdGeomXformable Xformable,
		double UsdTimeCode,
		FTransform& OutTransform,
		bool bIgnoreLocalTransform = false
	)
	{
		if (!Xformable)
		{
			return;
		}

		FScopedUsdAllocs Allocs;

		pxr::UsdPrim UsdPrim = Xformable.GetPrim();
		pxr::UsdStageRefPtr UsdStage = UsdPrim.GetStage();

		bool bResetTransformStack = false;
		if (bIgnoreLocalTransform)
		{
			FTransform Dummy;
			UsdToUnreal::ConvertXformable(UsdStage, Xformable, Dummy, UsdTimeCode, &bResetTransformStack);

			OutTransform = FTransform::Identity;
		}
		else
		{
			UsdToUnreal::ConvertXformable(UsdStage, Xformable, OutTransform, UsdTimeCode, &bResetTransformStack);
		}

		// If we have the resetXformStack op on this prim's xformOpOrder we have to essentially use its transform
		// as the world transform (i.e. we have to discard the parent transforms). We won't do this here, and will instead
		// keep relative transforms everywhere for consistency, which means we must manually invert the ParentToWorld transform
		// and compute our relative transform ourselves.
		//
		// Ideally we'd query the components for this for performance reasons, but not only we don't have access to them here,
		// but neither the stage actor's PrimsToAnimate nor the sequencer guarantee a particular evaluation order anyway,
		// which means that if our parent is also animated, we could end up computing our relative transforms using the outdated
		// parent's transform instead. This means we must compute our relative transform using the actual prim hierarchy.
		//
		// Additionally, our parent prims may be animated, so we must query all of our ancestors for a new world matrix every frame.
		//
		// We could use UsdGeomXformCache for this, but given that we won't actually cache anything (since we'll have to resample
		// all ancestors every frame anyway) and that we would have to manually handle the camera/light compensation at least for
		// our immediate parent, it's simpler to just recursively call our own UsdToUnreal::ConvertXformable and concatenate the
		// results. Its not as fast, but we'll only do this on the initial read for prims with `resetXformStack`, so it should
		// be very rare. We don't ever write out the resetXformStack either, so after that initial read this op should just disappear.
		//
		// Note that, alternatively, we could also handle this whole situation by having the scene components specify their transforms
		// as absolute, and the Sequencer would work with that as well. However that would spread out the handling of
		// resetXformStack through all USD workflows, and mean we'd have to *write out* resetXformStack when writing/exporting
		// absolute transform components, and also convert between them when the user toggles between relative/absolute manually,
		// which is probably worse than just baking it as relative transforms on first read and forgetting about it.
		if (bResetTransformStack)
		{
			FTransform ParentToWorld = FTransform::Identity;

			pxr::UsdPrim AncestorPrim = UsdPrim.GetParent();
			while (AncestorPrim && !AncestorPrim.IsPseudoRoot())
			{
				FTransform AncestorTransform = FTransform::Identity;
				bool bAncestorResetTransformStack = false;
				UsdToUnreal::ConvertXformable(
					UsdStage,
					pxr::UsdGeomXformable{AncestorPrim},
					AncestorTransform,
					UsdTimeCode,
					&bAncestorResetTransformStack
				);

				ParentToWorld = ParentToWorld * AncestorTransform;

				// If we find a parent that also has the resetXformStack, then we're in luck: That transform value will be its world
				// transform already, so we can stop concatenating stuff. Yes, on the component-side of things we'd have done the same
				// thing of making a fake relative transform for it, but the end result would have been the same final world transform
				if (bAncestorResetTransformStack)
				{
					break;
				}

				AncestorPrim = AncestorPrim.GetParent();
			}

			const FVector& Scale = ParentToWorld.GetScale3D();
			if (!FMath::IsNearlyEqual(Scale.X, Scale.Y) || !FMath::IsNearlyEqual(Scale.X, Scale.Z))
			{
				USD_LOG_USERWARNING(FText::Format(
					LOCTEXT(
						"InvertNonUniformScaling",
						"Inverting transform with non-uniform scaling '{0}' when computing relative transform for prim '{1}'! Result will "
						"likely be incorrect, since FTransforms can't invert non-uniform scalings. You can work around this by baking your "
						"non-uniform scaling transform into the vertices, or by not using the !resetXformStack! Xform op."
					),
					FText::FromString(Scale.ToString()),
					FText::FromString(UsdToUnreal::ConvertPath(UsdPrim.GetPrimPath()))
				));
			}

			// Multiplying with matrices here helps mitigate the issues encountered with non-uniform scaling, however it will stil
			// never be perfect, as it is not possible to generate an FTransform that can properly invert a complex transform with non-uniform
			// scaling when just multiplying them (which is what downstream code within USceneComponent will do).
			OutTransform = FTransform{OutTransform.ToMatrixWithScale() * ParentToWorld.ToInverseMatrixWithScale()};
		}
	}

	void PrepareAttributeForAnimationBake(const pxr::UsdAttribute& Attr)
	{
		if (!Attr)
		{
			return;
		}

		// Weirdly enough GetTimeSamples() will return time codes with the offset and scale applied, while
		// ClearAtTime() expects time codes without offset and scale applied, so we must manually undo them here
		UE::FSdfLayerOffset CombinedOffset = UsdUtils::GetPrimToStageOffset(UE::FUsdPrim{Attr.GetPrim()});

		std::vector<double> TimeSamples;
		Attr.GetTimeSamples(&TimeSamples);
		for (double TimeSample : TimeSamples)
		{
			double LocalTime = (TimeSample - CombinedOffset.Offset) / CombinedOffset.Scale;
			Attr.ClearAtTime(LocalTime);
		}

		UsdUtils::NotifyIfOverriddenOpinion(Attr);
	}
}	 // namespace UE::USDPrimConversion::Private

bool UsdToUnreal::ConvertXformable(
	const pxr::UsdStageRefPtr& Stage,
	const pxr::UsdTyped& Schema,
	FTransform& OutTransform,
	double EvalTime,
	bool* bOutResetTransformStack
)
{
	pxr::UsdGeomXformable Xformable(Schema);
	if (!Xformable)
	{
		return false;
	}

	FScopedUsdAllocs UsdAllocs;

	OutTransform = FTransform::Identity;

	// If we're a primitive try extracting its transform as well, given that we'll always reuse the default, 1.0 size procedural meshes
	UsdToUnreal::ConvertGeomPrimitiveTransform(Xformable.GetPrim(), EvalTime, OutTransform);

	// Transform
	pxr::GfMatrix4d UsdMatrix;
	bool bResetXformStack = false;
	bool* bResetXformStackPtr = bOutResetTransformStack ? bOutResetTransformStack : &bResetXformStack;
	Xformable.GetLocalTransformation(&UsdMatrix, bResetXformStackPtr, EvalTime);

	FUsdStageInfo StageInfo(Stage);
	OutTransform = OutTransform * UsdToUnreal::ConvertMatrix(StageInfo, UsdMatrix);

	const bool bPrimIsLight = Xformable.GetPrim().HasAPI<pxr::UsdLuxLightAPI>();

	// Extra rotation to match different camera facing direction convention
	// Note: The camera space is always Y-up, yes, but this is not what this is: This is the camera's transform wrt the stage,
	// which follows the stage up axis
	if (Xformable.GetPrim().IsA<pxr::UsdGeomCamera>() || bPrimIsLight)
	{
		if (StageInfo.UpAxis == EUsdUpAxis::YAxis)
		{
			OutTransform = FTransform(FRotator(0.0f, -90.f, 0.0f)) * OutTransform;
		}
		else
		{
			OutTransform = FTransform(FRotator(-90.0f, -90.f, 0.0f)) * OutTransform;
		}
	}
	// Invert the compensation applied to our parents, in case they're a camera or a light
	if (pxr::UsdPrim Parent = Xformable.GetPrim().GetParent())
	{
		const bool bParentIsLight = Parent.HasAPI<pxr::UsdLuxLightAPI>();

		// If bResetXFormStack is true, then the prim's local transform will be used directly as the world transform, and we will
		// already invert the parent transform fully, regardless of what it is. This means it doesn't really matter if our parent
		// has a camera/light compensation or not, and so we don't have to have the explicit inverse compensation here anyway!
		if (!(*bResetXformStackPtr) && (Parent.IsA<pxr::UsdGeomCamera>() || bParentIsLight))
		{
			if (StageInfo.UpAxis == EUsdUpAxis::YAxis)
			{
				OutTransform = OutTransform * FTransform(FRotator(0.0f, -90.f, 0.0f).GetInverse());
			}
			else
			{
				OutTransform = OutTransform * FTransform(FRotator(-90.0f, -90.f, 0.0f).GetInverse());
			}
		}
	}

	return true;
}

void UsdToUnreal::PropagateTransform(
	const pxr::UsdStageRefPtr& Stage,
	const pxr::UsdPrim& Root,
	const pxr::UsdPrim& Leaf,
	double EvalTime,
	FTransform& OutTransform
)
{
	FScopedUsdAllocs UsdAllocs;

	bool bResetXformStack = false;
	FTransform CurrentTransform = FTransform::Identity;
	if (ConvertXformable(Stage, pxr::UsdTyped{Leaf}, CurrentTransform, EvalTime, &bResetXformStack))
	{
		if (!bResetXformStack)
		{
			OutTransform *= CurrentTransform;

			if (Leaf != Root)
			{
				if (!Leaf.IsPseudoRoot())
				{
					PropagateTransform(Stage, Root, Leaf.GetParent(), EvalTime, OutTransform);
				}
				else
				{
					// Leaf was not even in Root's subtree
					OutTransform = FTransform::Identity;
				}
			}
		}
		else
		{
			// The Xform stack was reset so that effectively stops the propagation
			OutTransform = CurrentTransform;
		}
	}
	else
	{
		// The Leaf is not an Xformable but we still want to propagate the transform to the Root
		PropagateTransform(Stage, Root, Leaf.GetParent(), EvalTime, OutTransform);
	}
}

bool UsdToUnreal::ConvertXformable(
	const pxr::UsdStageRefPtr& Stage,
	const pxr::UsdTyped& Schema,
	USceneComponent& SceneComponent,
	double EvalTime,
	bool bUsePrimTransform
)
{
	pxr::UsdGeomXformable Xformable(Schema);
	if (!Xformable)
	{
		return false;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(UsdToUnreal::ConvertXformable);

	FScopedUsdAllocs UsdAllocs;

	// Transform
	FTransform Transform;
	UE::USDPrimConversion::Private::GetPrimConvertedRelativeTransform(Xformable, EvalTime, Transform, !bUsePrimTransform);
	SceneComponent.SetRelativeTransform(Transform);

	SceneComponent.Modify();

	// Computed (effective) visibility
	const bool bIsHidden = (Xformable.ComputeVisibility(EvalTime) == pxr::UsdGeomTokens->invisible);
	SceneComponent.SetHiddenInGame(bIsHidden);

	// Per-prim visibility
	bool bIsInvisible = false;	  // Default to 'inherited'
	if (pxr::UsdAttribute VisibilityAttr = Xformable.GetVisibilityAttr())
	{
		pxr::TfToken Value;
		if (VisibilityAttr.Get(&Value, EvalTime))
		{
			bIsInvisible = Value == pxr::UsdGeomTokens->invisible;
		}
	}
	if (bIsInvisible)
	{
		SceneComponent.ComponentTags.AddUnique(UnrealIdentifiers::Invisible);
		SceneComponent.ComponentTags.Remove(UnrealIdentifiers::Inherited);
	}
	else
	{
		SceneComponent.ComponentTags.Remove(UnrealIdentifiers::Invisible);
		SceneComponent.ComponentTags.AddUnique(UnrealIdentifiers::Inherited);
	}

	return true;
}

bool UsdToUnreal::ConvertGeomCamera(const UE::FUsdPrim& Prim, UCineCameraComponent& CameraComponent, double UsdTimeCode)
{
	FScopedUsdAllocs Allocs;

	pxr::UsdPrim UsdPrim{Prim};
	pxr::UsdGeomCamera GeomCamera{UsdPrim};
	if (!GeomCamera)
	{
		return false;
	}

	UE::FUsdStage Stage = Prim.GetStage();
	FUsdStageInfo StageInfo(Stage);

	bool bNeedsManualFocus = false;
	bool bNeedsManualExposure = false;
	bool bDisableFocus = false;

	if (pxr::UsdAttribute Attr = GeomCamera.GetFocalLengthAttr(); Attr.HasAuthoredValue())
	{
		CameraComponent.SetCurrentFocalLength(UsdToUnreal::ConvertDistance(StageInfo, UsdUtils::GetUsdValue<float>(Attr, UsdTimeCode)));
	}

	if (pxr::UsdAttribute Attr = GeomCamera.GetFocusDistanceAttr(); Attr.HasAuthoredValue())
	{
		float UsdFocusDistance = UsdToUnreal::ConvertDistance(
			StageInfo,
			UsdUtils::GetUsdValue<float>(GeomCamera.GetFocusDistanceAttr(), UsdTimeCode)
		);

		CameraComponent.FocusSettings.ManualFocusDistance = UsdFocusDistance;

		if (!FMath::IsNearlyZero(UsdFocusDistance))
		{
			bNeedsManualFocus = true;
		}
	}

	// Regular 'fStop' is now meant to be used for DoF effects according to the latest schema
	if (pxr::UsdAttribute Attr = GeomCamera.GetFStopAttr(); Attr.HasAuthoredValue())
	{
		const float UsdFStop = UsdUtils::GetUsdValue<float>(Attr, UsdTimeCode);
		CameraComponent.PostProcessSettings.bOverride_DepthOfFieldFstop = true;
		CameraComponent.PostProcessSettings.DepthOfFieldFstop = UsdFStop;
		bNeedsManualFocus = true;

		if (FMath::IsNearlyZero(UsdFStop))
		{
			bDisableFocus = true;
		}
	}

	// 'exposure:fStop' is the the value that is meant to affect exposure
	if (pxr::UsdAttribute Attr = GeomCamera.GetExposureFStopAttr(); Attr.HasAuthoredValue())
	{
		const float UsdFStop = UsdUtils::GetUsdValue<float>(Attr, UsdTimeCode);
		CameraComponent.SetCurrentAperture(UsdFStop);
		bNeedsManualExposure = true;
	}

	if (pxr::UsdAttribute Attr = GeomCamera.GetHorizontalApertureAttr(); Attr.HasAuthoredValue())
	{
		CameraComponent.Filmback.SensorWidth = UsdToUnreal::ConvertDistance(StageInfo, UsdUtils::GetUsdValue<float>(Attr, UsdTimeCode));
	}

	if (pxr::UsdAttribute Attr = GeomCamera.GetVerticalApertureAttr(); Attr.HasAuthoredValue())
	{
		CameraComponent.Filmback.SensorHeight = UsdToUnreal::ConvertDistance(StageInfo, UsdUtils::GetUsdValue<float>(Attr, UsdTimeCode));
	}

	if (pxr::UsdAttribute Attr = GeomCamera.GetHorizontalApertureOffsetAttr(); Attr.HasAuthoredValue())
	{
		CameraComponent.Filmback.SensorHorizontalOffset = UsdToUnreal::ConvertDistance(StageInfo, UsdUtils::GetUsdValue<float>(Attr, UsdTimeCode));
	}

	if (pxr::UsdAttribute Attr = GeomCamera.GetVerticalApertureOffsetAttr(); Attr.HasAuthoredValue())
	{
		CameraComponent.Filmback.SensorVerticalOffset = UsdToUnreal::ConvertDistance(StageInfo, UsdUtils::GetUsdValue<float>(Attr, UsdTimeCode));
	}

	if (pxr::UsdAttribute Attr = GeomCamera.GetExposureAttr(); Attr.HasAuthoredValue())
	{
		CameraComponent.PostProcessSettings.bOverride_AutoExposureBias = true;
		CameraComponent.PostProcessSettings.AutoExposureBias = UsdUtils::GetUsdValue<float>(Attr, UsdTimeCode);

		// We don't necessarily want manual exposure just because we have this: It still works with auto exposure
	}

	if (pxr::UsdAttribute Attr = GeomCamera.GetProjectionAttr(); Attr.HasAuthoredValue())
	{
		CameraComponent.SetProjectionMode(
			UsdUtils::GetUsdValue<pxr::TfToken>(Attr, UsdTimeCode) == pxr::UsdGeomTokens->orthographic ? ECameraProjectionMode::Orthographic
																									   : ECameraProjectionMode::Perspective
		);
	}

	if (pxr::UsdAttribute Attr = GeomCamera.GetClippingRangeAttr(); Attr.HasAuthoredValue())
	{
		pxr::GfVec2f NearAndFarSceneUnits = UsdUtils::GetUsdValue<pxr::GfVec2f>(Attr, UsdTimeCode);
		float UENear = UsdToUnreal::ConvertDistance(StageInfo, NearAndFarSceneUnits[0]);
		float UEFar = UsdToUnreal::ConvertDistance(StageInfo, NearAndFarSceneUnits[1]);

		CameraComponent.SetOrthoNearClipPlane(UENear);
		CameraComponent.SetOrthoFarClipPlane(UEFar);

		CameraComponent.SetCustomNearClippingPlane(UENear);
		CameraComponent.bOverride_CustomNearClippingPlane = true;
	}

	if (pxr::UsdAttribute Attr = GeomCamera.GetExposureIsoAttr(); Attr.HasAuthoredValue())
	{
		CameraComponent.PostProcessSettings.bOverride_CameraISO = true;
		CameraComponent.PostProcessSettings.CameraISO = UsdUtils::GetUsdValue<float>(Attr, UsdTimeCode);
		bNeedsManualExposure = true;
	}

	if (pxr::UsdAttribute Attr = GeomCamera.GetExposureTimeAttr(); Attr.HasAuthoredValue())
	{
		float UsdTimeSeconds = UsdUtils::GetUsdValue<float>(Attr, UsdTimeCode);
		if (!FMath::IsNearlyZero(UsdTimeSeconds))
		{
			CameraComponent.PostProcessSettings.bOverride_CameraShutterSpeed = true;
			CameraComponent.PostProcessSettings.CameraShutterSpeed = 1.0f / UsdTimeSeconds;
		}
		bNeedsManualExposure = true;
	}

	if (bDisableFocus)
	{
		CameraComponent.FocusSettings.FocusMethod = ECameraFocusMethod::Disable;
	}
	else if (bNeedsManualFocus)
	{
		CameraComponent.FocusSettings.FocusMethod = ECameraFocusMethod::Manual;
	}
	else
	{
		CameraComponent.FocusSettings.FocusMethod = ECameraFocusMethod::DoNotOverride;
	}

	if (bNeedsManualExposure)
	{
		CameraComponent.PostProcessSettings.bOverride_AutoExposureMethod = true;
		CameraComponent.PostProcessSettings.AutoExposureMethod = EAutoExposureMethod::AEM_Manual;
	}

	return true;
}

bool UsdToUnreal::ConvertBoolTimeSamples(
	const UE::FUsdStage& Stage,
	const TArray<double>& UsdTimeSamples,
	const TFunction<bool(double)>& ReaderFunc,
	UMovieSceneBoolTrack& MovieSceneTrack,
	const FMovieSceneSequenceTransform& SequenceTransform
)
{
	if (!ReaderFunc)
	{
		return false;
	}

	const UMovieScene* MovieScene = MovieSceneTrack.GetTypedOuter<UMovieScene>();
	if (!MovieScene)
	{
		return false;
	}

	const FFrameRate Resolution = MovieScene->GetTickResolution();
	const FFrameRate DisplayRate = MovieScene->GetDisplayRate();

	FScopedUsdAllocs Allocs;

	pxr::UsdStageRefPtr UsdStage{Stage};
	FUsdStageInfo StageInfo{Stage};

	TArray<FFrameNumber> FrameNumbers;
	FrameNumbers.Reserve(UsdTimeSamples.Num());

	TArray<bool> SectionValues;
	SectionValues.Reserve(UsdTimeSamples.Num());

	const double StageTimeCodesPerSecond = UsdStage->GetTimeCodesPerSecond();
	const FFrameRate StageFrameRate(StageTimeCodesPerSecond, 1);

	double LastTimeSample = TNumericLimits<double>::Lowest();
	for (const double UsdTimeSample : UsdTimeSamples)
	{
		// We never want to evaluate the same time twice
		if (FMath::IsNearlyEqual(UsdTimeSample, LastTimeSample))
		{
			continue;
		}
		LastTimeSample = UsdTimeSample;

		int32 FrameNumber = FMath::FloorToInt(UsdTimeSample);
		float SubFrameNumber = UsdTimeSample - FrameNumber;

		FFrameTime FrameTime(FrameNumber, SubFrameNumber);

		FFrameTime KeyFrameTime = FFrameRate::TransformTime(FrameTime, StageFrameRate, Resolution);
		KeyFrameTime *= SequenceTransform;
		FrameNumbers.Add(KeyFrameTime.GetFrame());

		bool UEValue = ReaderFunc(UsdTimeSample);
		SectionValues.Emplace(UEValue);
	}

	bool bSectionAdded = false;
	UMovieSceneBoolSection* Section = Cast<UMovieSceneBoolSection>(MovieSceneTrack.FindOrAddSection(0, bSectionAdded));
	Section->EvalOptions.CompletionMode = EMovieSceneCompletionMode::KeepState;

	TMovieSceneChannelData<bool> Data = Section->GetChannel().GetData();
	Data.Reset();
	for (int32 KeyIndex = 0; KeyIndex < FrameNumbers.Num(); ++KeyIndex)
	{
		Data.AddKey(FrameNumbers[KeyIndex], SectionValues[KeyIndex]);
	}

	Section->SetRange(TRange<FFrameNumber>::All());

	return true;
}

bool UsdToUnreal::ConvertBoolTimeSamples(
	const UE::FUsdStage& Stage,
	const TArray<double>& UsdTimeSamples,
	const TFunction<bool(double)>& ReaderFunc,
	UMovieSceneVisibilityTrack& MovieSceneTrack,
	const FMovieSceneSequenceTransform& SequenceTransform
)
{
	if (!ReaderFunc)
	{
		return false;
	}

	const UMovieScene* MovieScene = MovieSceneTrack.GetTypedOuter<UMovieScene>();
	if (!MovieScene)
	{
		return false;
	}

	const FFrameRate Resolution = MovieScene->GetTickResolution();
	const FFrameRate DisplayRate = MovieScene->GetDisplayRate();

	FScopedUsdAllocs Allocs;

	pxr::UsdStageRefPtr UsdStage{Stage};
	FUsdStageInfo StageInfo{Stage};

	TArray<FFrameNumber> FrameNumbers;
	FrameNumbers.Reserve(UsdTimeSamples.Num());

	TArray<bool> SectionValues;
	SectionValues.Reserve(UsdTimeSamples.Num());

	const double StageTimeCodesPerSecond = UsdStage->GetTimeCodesPerSecond();
	const FFrameRate StageFrameRate(StageTimeCodesPerSecond, 1);

	double LastTimeSample = TNumericLimits<double>::Lowest();
	for (const double UsdTimeSample : UsdTimeSamples)
	{
		// We never want to evaluate the same time twice
		if (FMath::IsNearlyEqual(UsdTimeSample, LastTimeSample))
		{
			continue;
		}
		LastTimeSample = UsdTimeSample;

		int32 FrameNumber = FMath::FloorToInt(UsdTimeSample);
		float SubFrameNumber = UsdTimeSample - FrameNumber;

		FFrameTime FrameTime(FrameNumber, SubFrameNumber);

		FFrameTime KeyFrameTime = FFrameRate::TransformTime(FrameTime, StageFrameRate, Resolution);
		KeyFrameTime *= SequenceTransform;
		FrameNumbers.Add(KeyFrameTime.GetFrame());

		bool UEValue = ReaderFunc(UsdTimeSample);
		SectionValues.Emplace(UEValue);
	}

	bool bSectionAdded = false;
	UMovieSceneVisibilitySection* Section = Cast<UMovieSceneVisibilitySection>(MovieSceneTrack.FindOrAddSection(0, bSectionAdded));
	Section->EvalOptions.CompletionMode = EMovieSceneCompletionMode::KeepState;

	TMovieSceneChannelData<bool> Data = Section->GetChannel().GetData();
	Data.Reset();
	for (int32 KeyIndex = 0; KeyIndex < FrameNumbers.Num(); ++KeyIndex)
	{
		Data.AddKey(FrameNumbers[KeyIndex], SectionValues[KeyIndex]);
	}

	Section->SetRange(TRange<FFrameNumber>::All());

	return true;
}

bool UsdToUnreal::ConvertFloatTimeSamples(
	const UE::FUsdStage& Stage,
	const TArray<double>& UsdTimeSamples,
	const TFunction<float(double)>& ReaderFunc,
	FMovieSceneFloatChannel& FloatChannel,
	const UMovieScene& MovieSceneOuter,
	const FMovieSceneSequenceTransform& SequenceTransform,
	TOptional<ERichCurveInterpMode> InterpolationModeOverride
)
{
	if (!ReaderFunc)
	{
		return false;
	}

	const FFrameRate Resolution = MovieSceneOuter.GetTickResolution();
	const FFrameRate DisplayRate = MovieSceneOuter.GetDisplayRate();

	FScopedUsdAllocs Allocs;

	pxr::UsdStageRefPtr UsdStage{Stage};
	FUsdStageInfo StageInfo{Stage};

	TArray<FFrameNumber> FrameNumbers;
	FrameNumbers.Reserve(UsdTimeSamples.Num());

	TArray<FMovieSceneFloatValue> SectionValues;
	SectionValues.Reserve(UsdTimeSamples.Num());

	const double StageTimeCodesPerSecond = UsdStage->GetTimeCodesPerSecond();
	const FFrameRate StageFrameRate(StageTimeCodesPerSecond, 1);

	const ERichCurveInterpMode InterpMode = InterpolationModeOverride.IsSet() ? InterpolationModeOverride.GetValue()
											: (UsdStage->GetInterpolationType() == pxr::UsdInterpolationTypeLinear)
												? ERichCurveInterpMode::RCIM_Linear
												: ERichCurveInterpMode::RCIM_Constant;

	double LastTimeSample = TNumericLimits<double>::Lowest();
	for (const double UsdTimeSample : UsdTimeSamples)
	{
		// We never want to evaluate the same time twice
		if (FMath::IsNearlyEqual(UsdTimeSample, LastTimeSample))
		{
			continue;
		}
		LastTimeSample = UsdTimeSample;

		int32 FrameNumber = FMath::FloorToInt(UsdTimeSample);
		float SubFrameNumber = UsdTimeSample - FrameNumber;

		FFrameTime FrameTime(FrameNumber, SubFrameNumber);

		FFrameTime KeyFrameTime = FFrameRate::TransformTime(FrameTime, StageFrameRate, Resolution);
		KeyFrameTime *= SequenceTransform;
		FrameNumbers.Add(KeyFrameTime.GetFrame());

		float UEValue = ReaderFunc(UsdTimeSample);
		SectionValues.Emplace_GetRef(UEValue).InterpMode = InterpMode;
	}

	FloatChannel.Set(FrameNumbers, SectionValues);

	return true;
}

bool UsdToUnreal::ConvertFloatTimeSamples(
	const UE::FUsdStage& Stage,
	const TArray<double>& UsdTimeSamples,
	const TFunction<float(double)>& ReaderFunc,
	UMovieSceneFloatTrack& MovieSceneTrack,
	const FMovieSceneSequenceTransform& SequenceTransform,
	TOptional<ERichCurveInterpMode> InterpolationModeOverride
)
{
	if (!ReaderFunc)
	{
		return false;
	}

	const UMovieScene* MovieScene = MovieSceneTrack.GetTypedOuter<UMovieScene>();
	if (!MovieScene)
	{
		return false;
	}

	bool bSectionAdded = false;
	UMovieSceneFloatSection* Section = Cast<UMovieSceneFloatSection>(MovieSceneTrack.FindOrAddSection(0, bSectionAdded));
	Section->EvalOptions.CompletionMode = EMovieSceneCompletionMode::KeepState;

	TArrayView<FMovieSceneFloatChannel*> Channels = Section->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
	if (Channels.Num() > 0)
	{
		FMovieSceneFloatChannel* Channel = Channels[0];

		const bool bSuccess = UsdToUnreal::ConvertFloatTimeSamples(
			Stage,
			UsdTimeSamples,
			ReaderFunc,
			*Channel,
			*MovieScene,
			SequenceTransform,
			InterpolationModeOverride
		);

		if (!bSuccess)
		{
			return false;
		}
	}

	Section->SetRange(TRange<FFrameNumber>::All());

	return true;
}

bool UsdToUnreal::ConvertColorTimeSamples(
	const UE::FUsdStage& Stage,
	const TArray<double>& UsdTimeSamples,
	const TFunction<FLinearColor(double)>& ReaderFunc,
	UMovieSceneColorTrack& MovieSceneTrack,
	const FMovieSceneSequenceTransform& SequenceTransform
)
{
	if (!ReaderFunc)
	{
		return false;
	}

	const UMovieScene* MovieScene = MovieSceneTrack.GetTypedOuter<UMovieScene>();
	if (!MovieScene)
	{
		return false;
	}

	const FFrameRate Resolution = MovieScene->GetTickResolution();
	const FFrameRate DisplayRate = MovieScene->GetDisplayRate();

	FScopedUsdAllocs Allocs;

	pxr::UsdStageRefPtr UsdStage{Stage};
	FUsdStageInfo StageInfo{Stage};

	TArray<FFrameNumber> FrameNumbers;
	FrameNumbers.Reserve(UsdTimeSamples.Num());

	TArray<FMovieSceneFloatValue> RedValues;
	TArray<FMovieSceneFloatValue> GreenValues;
	TArray<FMovieSceneFloatValue> BlueValues;
	TArray<FMovieSceneFloatValue> AlphaValues;
	RedValues.Reserve(UsdTimeSamples.Num());
	GreenValues.Reserve(UsdTimeSamples.Num());
	BlueValues.Reserve(UsdTimeSamples.Num());
	AlphaValues.Reserve(UsdTimeSamples.Num());

	const double StageTimeCodesPerSecond = UsdStage->GetTimeCodesPerSecond();
	const FFrameRate StageFrameRate(StageTimeCodesPerSecond, 1);

	const ERichCurveInterpMode InterpMode = (UsdStage->GetInterpolationType() == pxr::UsdInterpolationTypeLinear)
												? ERichCurveInterpMode::RCIM_Linear
												: ERichCurveInterpMode::RCIM_Constant;

	double LastTimeSample = TNumericLimits<double>::Lowest();
	for (const double UsdTimeSample : UsdTimeSamples)
	{
		// We never want to evaluate the same time twice
		if (FMath::IsNearlyEqual(UsdTimeSample, LastTimeSample))
		{
			continue;
		}
		LastTimeSample = UsdTimeSample;

		int32 FrameNumber = FMath::FloorToInt(UsdTimeSample);
		float SubFrameNumber = UsdTimeSample - FrameNumber;

		FFrameTime FrameTime(FrameNumber, SubFrameNumber);

		FFrameTime KeyFrameTime = FFrameRate::TransformTime(FrameTime, StageFrameRate, Resolution);
		KeyFrameTime *= SequenceTransform;
		FrameNumbers.Add(KeyFrameTime.GetFrame());

		FLinearColor UEValue = ReaderFunc(UsdTimeSample);
		RedValues.Emplace_GetRef(UEValue.R).InterpMode = InterpMode;
		GreenValues.Emplace_GetRef(UEValue.G).InterpMode = InterpMode;
		BlueValues.Emplace_GetRef(UEValue.B).InterpMode = InterpMode;
		AlphaValues.Emplace_GetRef(UEValue.A).InterpMode = InterpMode;
	}

	bool bSectionAdded = false;
	UMovieSceneColorSection* Section = Cast<UMovieSceneColorSection>(MovieSceneTrack.FindOrAddSection(0, bSectionAdded));
	Section->EvalOptions.CompletionMode = EMovieSceneCompletionMode::KeepState;

	TArrayView<FMovieSceneFloatChannel*> Channels = Section->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
	if (Channels.Num() != 4)
	{
		return false;
	}

	Channels[0]->Set(FrameNumbers, RedValues);
	Channels[1]->Set(FrameNumbers, GreenValues);
	Channels[2]->Set(FrameNumbers, BlueValues);
	Channels[3]->Set(FrameNumbers, AlphaValues);

	Section->SetRange(TRange<FFrameNumber>::All());

	return true;
}

bool UsdToUnreal::ConvertTransformTimeSamples(
	const UE::FUsdStage& Stage,
	const TArray<double>& UsdTimeSamples,
	const TFunction<FTransform(double)>& ReaderFunc,
	UMovieScene3DTransformTrack& MovieSceneTrack,
	const FMovieSceneSequenceTransform& SequenceTransform
)
{
	if (!ReaderFunc)
	{
		return false;
	}

	const UMovieScene* MovieScene = MovieSceneTrack.GetTypedOuter<UMovieScene>();
	if (!MovieScene)
	{
		return false;
	}

	const FFrameRate Resolution = MovieScene->GetTickResolution();
	const FFrameRate DisplayRate = MovieScene->GetDisplayRate();

	FScopedUsdAllocs Allocs;

	pxr::UsdStageRefPtr UsdStage{Stage};
	FUsdStageInfo StageInfo{Stage};

	TArray<FFrameNumber> FrameNumbers;
	FrameNumbers.Reserve(UsdTimeSamples.Num());

	TArray<FMovieSceneDoubleValue> LocationXValues;
	TArray<FMovieSceneDoubleValue> LocationYValues;
	TArray<FMovieSceneDoubleValue> LocationZValues;

	TArray<FMovieSceneDoubleValue> RotationXValues;
	TArray<FMovieSceneDoubleValue> RotationYValues;
	TArray<FMovieSceneDoubleValue> RotationZValues;

	TArray<FMovieSceneDoubleValue> ScaleXValues;
	TArray<FMovieSceneDoubleValue> ScaleYValues;
	TArray<FMovieSceneDoubleValue> ScaleZValues;

	LocationXValues.Reserve(UsdTimeSamples.Num());
	LocationYValues.Reserve(UsdTimeSamples.Num());
	LocationZValues.Reserve(UsdTimeSamples.Num());

	RotationXValues.Reserve(UsdTimeSamples.Num());
	RotationYValues.Reserve(UsdTimeSamples.Num());
	RotationZValues.Reserve(UsdTimeSamples.Num());

	ScaleXValues.Reserve(UsdTimeSamples.Num());
	ScaleYValues.Reserve(UsdTimeSamples.Num());
	ScaleZValues.Reserve(UsdTimeSamples.Num());

	const double StageTimeCodesPerSecond = UsdStage->GetTimeCodesPerSecond();
	const FFrameRate StageFrameRate(StageTimeCodesPerSecond, 1);

	const ERichCurveInterpMode InterpMode = (UsdStage->GetInterpolationType() == pxr::UsdInterpolationTypeLinear)
												? ERichCurveInterpMode::RCIM_Linear
												: ERichCurveInterpMode::RCIM_Constant;

	double LastTimeSample = TNumericLimits<double>::Lowest();

	struct FHeadingHelper
	{
		FRotator Heading;
		bool bHeadingSet = false;
	};
	FHeadingHelper HeadingHelper;

	for (const double UsdTimeSample : UsdTimeSamples)
	{
		// We never want to evaluate the same time twice
		if (FMath::IsNearlyEqual(UsdTimeSample, LastTimeSample))
		{
			continue;
		}
		LastTimeSample = UsdTimeSample;

		int32 FrameNumber = FMath::FloorToInt(UsdTimeSample);
		float SubFrameNumber = UsdTimeSample - FrameNumber;

		FFrameTime FrameTime(FrameNumber, SubFrameNumber);

		FFrameTime KeyFrameTime = FFrameRate::TransformTime(FrameTime, StageFrameRate, Resolution);
		KeyFrameTime *= SequenceTransform;
		FrameNumbers.Add(KeyFrameTime.GetFrame());

		FTransform UEValue = ReaderFunc(UsdTimeSample);
		FVector Location = UEValue.GetLocation();
		FRotator Rotator = UEValue.Rotator();
		FVector Scale = UEValue.GetScale3D();

		if (HeadingHelper.bHeadingSet)
		{
			FMath::WindRelativeAnglesDegrees(HeadingHelper.Heading.Roll, Rotator.Roll);
			FMath::WindRelativeAnglesDegrees(HeadingHelper.Heading.Pitch, Rotator.Pitch);
			FMath::WindRelativeAnglesDegrees(HeadingHelper.Heading.Yaw, Rotator.Yaw);

			FRotator OtherChoice = Rotator.GetEquivalentRotator().GetNormalized();
			float FirstDiff = HeadingHelper.Heading.GetManhattanDistance(Rotator);
			float SecondDiff = HeadingHelper.Heading.GetManhattanDistance(OtherChoice);
			if (SecondDiff < FirstDiff)
			{
				Rotator = OtherChoice;
			}
		}
		else
		{
			HeadingHelper.bHeadingSet = true;
		}

		HeadingHelper.Heading = Rotator;

		LocationXValues.Emplace_GetRef(Location.X).InterpMode = InterpMode;
		LocationYValues.Emplace_GetRef(Location.Y).InterpMode = InterpMode;
		LocationZValues.Emplace_GetRef(Location.Z).InterpMode = InterpMode;

		RotationXValues.Emplace_GetRef(Rotator.Roll).InterpMode = InterpMode;
		RotationYValues.Emplace_GetRef(Rotator.Pitch).InterpMode = InterpMode;
		RotationZValues.Emplace_GetRef(Rotator.Yaw).InterpMode = InterpMode;

		ScaleXValues.Emplace_GetRef(Scale.X).InterpMode = InterpMode;
		ScaleYValues.Emplace_GetRef(Scale.Y).InterpMode = InterpMode;
		ScaleZValues.Emplace_GetRef(Scale.Z).InterpMode = InterpMode;
	}

	bool bSectionAdded = false;
	UMovieScene3DTransformSection* Section = Cast<UMovieScene3DTransformSection>(MovieSceneTrack.FindOrAddSection(0, bSectionAdded));
	Section->EvalOptions.CompletionMode = EMovieSceneCompletionMode::KeepState;
	Section->SetRange(TRange<FFrameNumber>::All());

	TArrayView<FMovieSceneDoubleChannel*> Channels = Section->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();
	if (Channels.Num() < 9)
	{
		return false;
	}

	Channels[0]->Set(FrameNumbers, LocationXValues);
	Channels[1]->Set(FrameNumbers, LocationYValues);
	Channels[2]->Set(FrameNumbers, LocationZValues);

	Channels[3]->Set(FrameNumbers, RotationXValues);
	Channels[4]->Set(FrameNumbers, RotationYValues);
	Channels[5]->Set(FrameNumbers, RotationZValues);

	Channels[6]->Set(FrameNumbers, ScaleXValues);
	Channels[7]->Set(FrameNumbers, ScaleYValues);
	Channels[8]->Set(FrameNumbers, ScaleZValues);

	return true;
}

bool UsdToUnreal::ConvertBoundsTimeSamples(
	const UE::FUsdPrim& InPrim,
	const TArray<double>& InUsdTimeSamples,
	const FMovieSceneSequenceTransform& InSequenceTransform,
	UMovieSceneDoubleVectorTrack& InOutMinTrack,
	UMovieSceneDoubleVectorTrack& InOutMaxTrack,
	UE::FUsdGeomBBoxCache* InOutBBoxCache
)
{
	if (!InPrim)
	{
		return false;
	}

	TOptional<FWriteScopeLock> BBoxLock;
	pxr::UsdGeomBBoxCache* BBoxCache = nullptr;
	if (InOutBBoxCache)
	{
		BBoxLock.Emplace(InOutBBoxCache->Lock);
		BBoxCache = &static_cast<pxr::UsdGeomBBoxCache&>(*InOutBBoxCache);
	}

	// Create an BBoxCache on-demand (don't need to lock this one as it's purely ours)
	TOptional<pxr::UsdGeomBBoxCache> TempBBoxCacheStorage;
	if (BBoxCache == nullptr)
	{
		const bool bUseExtentsHint = true;
		const bool bIgnoreVisibility = false;
		static std::vector<pxr::TfToken> DefaultTokenVector{pxr::UsdGeomTokens->proxy, pxr::UsdGeomTokens->render};
		TempBBoxCacheStorage.Emplace(pxr::UsdTimeCode::EarliestTime(), DefaultTokenVector, bUseExtentsHint, bIgnoreVisibility);
		BBoxCache = &TempBBoxCacheStorage.GetValue();
	}

	const UMovieScene* MovieScene = InOutMinTrack.GetTypedOuter<UMovieScene>();
	if (!MovieScene)
	{
		return false;
	}

	const FFrameRate Resolution = MovieScene->GetTickResolution();

	FScopedUsdAllocs Allocs;

	pxr::UsdStageRefPtr UsdStage = InPrim.GetStage();
	FUsdStageInfo StageInfo{UsdStage};

	TArray<FFrameNumber> FrameNumbers;
	FrameNumbers.Reserve(InUsdTimeSamples.Num());

	TArray<FMovieSceneDoubleValue> MinXValues;
	TArray<FMovieSceneDoubleValue> MinYValues;
	TArray<FMovieSceneDoubleValue> MinZValues;
	TArray<FMovieSceneDoubleValue> MaxXValues;
	TArray<FMovieSceneDoubleValue> MaxYValues;
	TArray<FMovieSceneDoubleValue> MaxZValues;
	MinXValues.Reserve(InUsdTimeSamples.Num());
	MinYValues.Reserve(InUsdTimeSamples.Num());
	MinZValues.Reserve(InUsdTimeSamples.Num());
	MaxXValues.Reserve(InUsdTimeSamples.Num());
	MaxYValues.Reserve(InUsdTimeSamples.Num());
	MaxZValues.Reserve(InUsdTimeSamples.Num());

	const double StageTimeCodesPerSecond = UsdStage->GetTimeCodesPerSecond();
	const FFrameRate StageFrameRate(StageTimeCodesPerSecond, 1);

	const ERichCurveInterpMode InterpMode = (UsdStage->GetInterpolationType() == pxr::UsdInterpolationTypeLinear)
												? ERichCurveInterpMode::RCIM_Linear
												: ERichCurveInterpMode::RCIM_Constant;

	double LastTimeSample = TNumericLimits<double>::Lowest();
	for (const double UsdTimeSample : InUsdTimeSamples)
	{
		// We never want to evaluate the same time twice
		if (FMath::IsNearlyEqual(UsdTimeSample, LastTimeSample))
		{
			continue;
		}
		LastTimeSample = UsdTimeSample;

		int32 FrameNumber = FMath::FloorToInt(UsdTimeSample);
		float SubFrameNumber = UsdTimeSample - FrameNumber;

		FFrameTime FrameTime(FrameNumber, SubFrameNumber);

		FFrameTime KeyFrameTime = FFrameRate::TransformTime(FrameTime, StageFrameRate, Resolution);
		KeyFrameTime *= InSequenceTransform;
		FrameNumbers.Add(KeyFrameTime.GetFrame());

		if (GConsiderAllPrimsHaveAnimatedBounds)
		{
			BBoxCache->Clear();
		}

		// It may seem like we're repeatedly invalidating the BBoxCache by doing this but actually it should retain some stuff,
		// like bounds from non-animated prims
		BBoxCache->SetTime(UsdTimeSample);

		// Note: This can be extremely expensive, as it may fallback to computing new bounds, traversing points and everything for the entire subtree
		pxr::GfBBox3d BoxAndMatrix = BBoxCache->ComputeUntransformedBound(InPrim);
		pxr::GfRange3d Box = BoxAndMatrix.ComputeAlignedRange();

		FBox UEBox;
		if (!Box.IsEmpty())
		{
			FVector UESpaceUSDMin = UsdToUnreal::ConvertVector(StageInfo, Box.GetMin());
			FVector UESpaceUSDMax = UsdToUnreal::ConvertVector(StageInfo, Box.GetMax());
			UEBox = FBox{
				TArray<FVector>{UESpaceUSDMin, UESpaceUSDMax}
			};
		}

		MinXValues.Emplace_GetRef(UEBox.Min.X).InterpMode = InterpMode;
		MinYValues.Emplace_GetRef(UEBox.Min.Y).InterpMode = InterpMode;
		MinZValues.Emplace_GetRef(UEBox.Min.Z).InterpMode = InterpMode;

		MaxXValues.Emplace_GetRef(UEBox.Max.X).InterpMode = InterpMode;
		MaxYValues.Emplace_GetRef(UEBox.Max.Y).InterpMode = InterpMode;
		MaxZValues.Emplace_GetRef(UEBox.Max.Z).InterpMode = InterpMode;
	}

	bool bSectionAdded = false;
	UMovieSceneDoubleVectorSection* MinSection = Cast<UMovieSceneDoubleVectorSection>(InOutMinTrack.FindOrAddSection(0, bSectionAdded));
	UMovieSceneDoubleVectorSection* MaxSection = Cast<UMovieSceneDoubleVectorSection>(InOutMaxTrack.FindOrAddSection(0, bSectionAdded));
	MinSection->EvalOptions.CompletionMode = EMovieSceneCompletionMode::KeepState;
	MaxSection->EvalOptions.CompletionMode = EMovieSceneCompletionMode::KeepState;

	TArrayView<FMovieSceneDoubleChannel*> MinChannels = MinSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();
	TArrayView<FMovieSceneDoubleChannel*> MaxChannels = MaxSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();
	if (!ensure(MinChannels.Num() == 3 && MaxChannels.Num() == 3))
	{
		return false;
	}

	MinChannels[0]->Set(FrameNumbers, MinXValues);
	MinChannels[1]->Set(FrameNumbers, MinYValues);
	MinChannels[2]->Set(FrameNumbers, MinZValues);

	MaxChannels[0]->Set(FrameNumbers, MaxXValues);
	MaxChannels[1]->Set(FrameNumbers, MaxYValues);
	MaxChannels[2]->Set(FrameNumbers, MaxZValues);

	MinSection->SetRange(TRange<FFrameNumber>::All());
	MaxSection->SetRange(TRange<FFrameNumber>::All());

	return true;
}

UsdToUnreal::FPropertyTrackReader UsdToUnreal::CreatePropertyTrackReader(
	const UE::FUsdPrim& Prim,
	const FName& PropertyPath,
	bool bIgnorePrimLocalTransform
)
{
	UsdToUnreal::FPropertyTrackReader Reader;

	FScopedUsdAllocs Allocs;

	pxr::UsdPrim UsdPrim{Prim};
	pxr::UsdStageRefPtr UsdStage = UsdPrim.GetStage();
	FUsdStageInfo StageInfo{UsdStage};

	if (pxr::UsdGeomXformable Xformable{UsdPrim})
	{
		if (PropertyPath == UnrealIdentifiers::TransformPropertyName)
		{
			FTransform Default = FTransform::Identity;
			UE::USDPrimConversion::Private::GetPrimConvertedRelativeTransform(
				Xformable,
				UsdUtils::GetDefaultTimeCode(),
				Default,
				bIgnorePrimLocalTransform
			);

			Reader.TransformReader = [UsdStage, Xformable, Default, bIgnorePrimLocalTransform](double UsdTimeCode)
			{
				FTransform Result = Default;
				UE::USDPrimConversion::Private::GetPrimConvertedRelativeTransform(Xformable, UsdTimeCode, Result, bIgnorePrimLocalTransform);
				return Result;
			};
			return Reader;
		}
	}

	if (pxr::UsdGeomImageable Imageable{UsdPrim})
	{
		if (PropertyPath == UnrealIdentifiers::HiddenInGamePropertyName)
		{
			if (pxr::UsdAttribute Attr = Imageable.GetVisibilityAttr())
			{
				pxr::TfToken Default = pxr::UsdGeomTokens->inherited;
				Attr.Get<pxr::TfToken>(&Default);

				Reader.BoolReader = [Imageable](double UsdTimeCode)
				{
					// The property is "HiddenInGame" but it will end up in a visibility track, which is just a bool track,
					// where true means visible
					return Imageable.ComputeVisibility(UsdTimeCode) == pxr::UsdGeomTokens->inherited;
				};
				return Reader;
			}
		}
	}

	if (pxr::UsdGeomCamera Camera{UsdPrim})
	{
		TOptional<int32> ClippingPlaneIndex;
		bool bConvertDistance = true;
		bool bInvertValue = false;
		pxr::UsdAttribute Attr;

		if (PropertyPath == UnrealIdentifiers::CurrentFocalLengthPropertyName)
		{
			Attr = Camera.GetFocalLengthAttr();
		}
		else if (PropertyPath == UnrealIdentifiers::ManualFocusDistancePropertyName)
		{
			Attr = Camera.GetFocusDistanceAttr();
		}
		else if (PropertyPath == UnrealIdentifiers::CurrentAperturePropertyName)
		{
			bConvertDistance = false;
			Attr = Camera.GetExposureFStopAttr();
		}
		else if (PropertyPath == UnrealIdentifiers::SensorWidthPropertyName)
		{
			Attr = Camera.GetHorizontalApertureAttr();
		}
		else if (PropertyPath == UnrealIdentifiers::SensorHeightPropertyName)
		{
			Attr = Camera.GetVerticalApertureAttr();
		}
		else if (PropertyPath == UnrealIdentifiers::SensorHorizontalOffsetPropertyName)
		{
			Attr = Camera.GetHorizontalApertureOffsetAttr();
		}
		else if (PropertyPath == UnrealIdentifiers::SensorVerticalOffsetPropertyName)
		{
			Attr = Camera.GetVerticalApertureOffsetAttr();
		}
		else if (PropertyPath == UnrealIdentifiers::ExposureCompensationPropertyName)
		{
			Attr = Camera.GetExposureAttr();
			bConvertDistance = false;
		}
		else if (PropertyPath == UnrealIdentifiers::OrthoFarClipPlanePropertyName)
		{
			Attr = Camera.GetClippingRangeAttr();
			ClippingPlaneIndex = 1;	   // Far plane
		}
		else if (PropertyPath == UnrealIdentifiers::OrthoNearClipPlanePropertyName)
		{
			Attr = Camera.GetClippingRangeAttr();
			ClippingPlaneIndex = 0;	   // Near plane
		}
		else if (PropertyPath == UnrealIdentifiers::CustomNearClipppingPlanePropertyName)
		{
			Attr = Camera.GetClippingRangeAttr();
			ClippingPlaneIndex = 0;	   // Near plane
		}
		else if (PropertyPath == UnrealIdentifiers::DepthOfFieldFstopPropertyName)
		{
			bConvertDistance = false;
			Attr = Camera.GetFStopAttr();
		}
		else if (PropertyPath == UnrealIdentifiers::CameraISOPropertyName)
		{
			bConvertDistance = false;
			Attr = Camera.GetExposureIsoAttr();
		}
		else if (PropertyPath == UnrealIdentifiers::CameraShutterSpeedPropertyName)
		{
			bInvertValue = true;
			bConvertDistance = false;
			Attr = Camera.GetExposureTimeAttr();
		}

		// TODO: Projection mode track reader? Would need new enum flag reader types

		if (Attr)
		{
			if (ClippingPlaneIndex.IsSet() && ClippingPlaneIndex.GetValue() < 2)
			{
				pxr::GfVec2f DefaultVec;
				Attr.Get<pxr::GfVec2f>(&DefaultVec);
				float DefaultValue = UsdToUnreal::ConvertDistance(StageInfo, DefaultVec[ClippingPlaneIndex.GetValue()]);

				Reader.FloatReader = [Attr, DefaultValue, StageInfo, Index = ClippingPlaneIndex.GetValue()](double UsdTimeCode)
				{
					float Result = DefaultValue;

					pxr::GfVec2f DefaultVec;
					if (Attr.Get<pxr::GfVec2f>(&DefaultVec, UsdTimeCode))
					{
						Result = UsdToUnreal::ConvertDistance(StageInfo, DefaultVec[Index]);
					}

					return Result;
				};
				return Reader;
			}
			else if (bConvertDistance)
			{
				float Default = 0.0;
				Attr.Get<float>(&Default);
				Default = UsdToUnreal::ConvertDistance(StageInfo, Default);

				Reader.FloatReader = [Attr, Default, StageInfo](double UsdTimeCode)
				{
					float Result = Default;
					if (Attr.Get<float>(&Result, UsdTimeCode))
					{
						Result = UsdToUnreal::ConvertDistance(StageInfo, Result);
					}

					return Result;
				};
				return Reader;
			}
			else
			{
				float Default = 0.0;
				Attr.Get<float>(&Default);

				if (bInvertValue)
				{
					Reader.FloatReader = [Attr, Default](double UsdTimeCode)
					{
						float Result = Default;
						Attr.Get<float>(&Result, UsdTimeCode);

						return Result != 0.0f ? (1.0f / Result) : 0.0f;
					};
				}
				else
				{
					Reader.FloatReader = [Attr, Default](double UsdTimeCode)
					{
						float Result = Default;
						Attr.Get<float>(&Result, UsdTimeCode);

						return Result;
					};
				}

				return Reader;
			}
		}
	}
	else if (const pxr::UsdLuxLightAPI LightAPI{Prim})
	{
		if (PropertyPath == UnrealIdentifiers::LightColorPropertyName)
		{
			if (pxr::UsdAttribute Attr = LightAPI.GetColorAttr())
			{
				pxr::GfVec3f UsdDefault;
				Attr.Get<pxr::GfVec3f>(&UsdDefault);
				FLinearColor Default = UsdToUnreal::ConvertColor(UsdDefault);

				Reader.ColorReader = [Attr, Default](double UsdTimeCode)
				{
					FLinearColor Result = Default;

					pxr::GfVec3f Value;
					if (Attr.Get<pxr::GfVec3f>(&Value, UsdTimeCode))
					{
						Result = UsdToUnreal::ConvertColor(Value);
					}

					return Result;
				};
				return Reader;
			}
		}
		else if (PropertyPath == UnrealIdentifiers::UseTemperaturePropertyName)
		{
			if (pxr::UsdAttribute Attr = LightAPI.GetEnableColorTemperatureAttr())
			{
				bool Default;
				Attr.Get<bool>(&Default);

				Reader.BoolReader = [Attr, Default](double UsdTimeCode)
				{
					bool Result = Default;
					Attr.Get<bool>(&Result, UsdTimeCode);
					return Result;
				};
				return Reader;
			}
		}
		else if (PropertyPath == UnrealIdentifiers::TemperaturePropertyName)
		{
			if (pxr::UsdAttribute Attr = LightAPI.GetColorTemperatureAttr())
			{
				float Default;
				Attr.Get<float>(&Default);

				Reader.FloatReader = [Attr, Default](double UsdTimeCode)
				{
					float Result = Default;
					Attr.Get<float>(&Result, UsdTimeCode);
					return Result;
				};
				return Reader;
			}
		}

		else if (pxr::UsdLuxSphereLight SphereLight{UsdPrim})
		{
			if (PropertyPath == UnrealIdentifiers::SourceRadiusPropertyName)
			{
				if (pxr::UsdAttribute Attr = SphereLight.GetRadiusAttr())
				{
					float Default = 0.0;
					Attr.Get<float>(&Default);
					Default = UsdToUnreal::ConvertDistance(StageInfo, Default);

					Reader.FloatReader = [Attr, Default, StageInfo](double UsdTimeCode)
					{
						float Result = Default;
						if (Attr.Get<float>(&Result, UsdTimeCode))
						{
							Result = UsdToUnreal::ConvertDistance(StageInfo, Result);
						}

						return Result;
					};
					return Reader;
				}
			}

			// Spot light
			else if (UsdPrim.HasAPI<pxr::UsdLuxShapingAPI>())
			{
				pxr::UsdLuxShapingAPI ShapingAPI{UsdPrim};

				if (PropertyPath == UnrealIdentifiers::IntensityPropertyName)
				{
					pxr::UsdAttribute IntensityAttr = SphereLight.GetIntensityAttr();
					pxr::UsdAttribute ExposureAttr = SphereLight.GetExposureAttr();

					if (IntensityAttr && ExposureAttr)
					{
						// Default values directly from pxr/usd/usdLux/schema.usda
						float DefaultUsdIntensity = 1.0f;
						float DefaultUsdExposure = 0.0f;

						IntensityAttr.Get<float>(&DefaultUsdIntensity);
						ExposureAttr.Get<float>(&DefaultUsdExposure);

						float Default = UsdToUnreal::ConvertLightIntensity(DefaultUsdIntensity, DefaultUsdExposure);

						Reader.FloatReader = [IntensityAttr, ExposureAttr, Default, StageInfo](double UsdTimeCode)
						{
							float Result = Default;

							float UsdIntensity = 1.0f;
							float UsdExposure = 0.0f;
							if (IntensityAttr.Get<float>(&UsdIntensity, UsdTimeCode) && ExposureAttr.Get<float>(&UsdExposure, UsdTimeCode))
							{
								Result = UsdToUnreal::ConvertLightIntensity(UsdIntensity, UsdExposure);
							}

							return Result;
						};
						return Reader;
					}
				}
				else if (PropertyPath == UnrealIdentifiers::OuterConeAnglePropertyName)
				{
					if (pxr::UsdAttribute Attr = ShapingAPI.GetShapingConeAngleAttr())
					{
						float Default;
						Attr.Get<float>(&Default);

						Reader.FloatReader = [Attr, Default](double UsdTimeCode)
						{
							float Result = Default;
							Attr.Get<float>(&Result, UsdTimeCode);
							return Result;
						};
						return Reader;
					}
				}
				else if (PropertyPath == UnrealIdentifiers::InnerConeAnglePropertyName)
				{
					pxr::UsdAttribute ConeAngleAttr = ShapingAPI.GetShapingConeAngleAttr();
					pxr::UsdAttribute ConeSoftnessAttr = ShapingAPI.GetShapingConeSoftnessAttr();

					if (ConeAngleAttr && ConeSoftnessAttr)
					{
						// Default values directly from pxr/usd/usdLux/schema.usda
						float DefaultUsdConeAngle = 90.0f;
						float DefaultUsdConeSoftness = 0.0f;

						ConeAngleAttr.Get<float>(&DefaultUsdConeAngle);
						ConeSoftnessAttr.Get<float>(&DefaultUsdConeSoftness);

						float DefaultInnerConeAngle = 0.0f;
						float DefaultOuterConeAngle = 0.0f;
						UsdToUnreal::ConvertSpotLightConeAngles(DefaultUsdConeAngle, DefaultUsdConeSoftness, DefaultOuterConeAngle, DefaultInnerConeAngle);

						Reader.FloatReader = [ConeAngleAttr, ConeSoftnessAttr, DefaultInnerConeAngle](double UsdTimeCode)
						{
							float Result = DefaultInnerConeAngle;

							float UsdConeAngle = 90.0f;
							float UsdConeSoftness = 0.0f;
							if (ConeAngleAttr.Get<float>(&UsdConeAngle, UsdTimeCode) && ConeSoftnessAttr.Get<float>(&UsdConeSoftness, UsdTimeCode))
							{
								float OuterConeAngle = 0.0f;
								UsdToUnreal::ConvertSpotLightConeAngles(UsdConeAngle, UsdConeSoftness, OuterConeAngle, Result);
							}

							return Result;
						};
						return Reader;
					}
				}
			}
			// Just a point light
			else
			{
				if (PropertyPath == UnrealIdentifiers::IntensityPropertyName)
				{
					pxr::UsdAttribute IntensityAttr = SphereLight.GetIntensityAttr();
					pxr::UsdAttribute ExposureAttr = SphereLight.GetExposureAttr();
					pxr::UsdAttribute RadiusAttr = SphereLight.GetRadiusAttr();

					if (IntensityAttr && ExposureAttr && RadiusAttr)
					{
						// Default values directly from pxr/usd/usdLux/schema.usda
						float DefaultUsdIntensity = 1.0f;
						float DefaultUsdExposure = 0.0f;

						IntensityAttr.Get<float>(&DefaultUsdIntensity);
						ExposureAttr.Get<float>(&DefaultUsdExposure);

						float Default = UsdToUnreal::ConvertLightIntensity(DefaultUsdIntensity, DefaultUsdExposure);

						Reader.FloatReader = [IntensityAttr, ExposureAttr, Default, StageInfo](double UsdTimeCode)
						{
							float Result = Default;

							float UsdIntensity = 1.0f;
							float UsdExposure = 0.0f;
							if (IntensityAttr.Get<float>(&UsdIntensity, UsdTimeCode) && ExposureAttr.Get<float>(&UsdExposure, UsdTimeCode))
							{
								Result = UsdToUnreal::ConvertLightIntensity(UsdIntensity, UsdExposure);
							}

							return Result;
						};
						return Reader;
					}
				}
			}
		}
		else if (pxr::UsdLuxRectLight RectLight{UsdPrim})
		{
			if (PropertyPath == UnrealIdentifiers::SourceWidthPropertyName)
			{
				if (pxr::UsdAttribute Attr = RectLight.GetWidthAttr())
				{
					float Default = 0.0;
					Attr.Get<float>(&Default);
					Default = UsdToUnreal::ConvertDistance(StageInfo, Default);

					Reader.FloatReader = [Attr, Default, StageInfo](double UsdTimeCode)
					{
						float Result = Default;
						if (Attr.Get<float>(&Result, UsdTimeCode))
						{
							Result = UsdToUnreal::ConvertDistance(StageInfo, Result);
						}

						return Result;
					};
					return Reader;
				}
			}
			else if (PropertyPath == UnrealIdentifiers::SourceHeightPropertyName)
			{
				if (pxr::UsdAttribute Attr = RectLight.GetHeightAttr())
				{
					float Default = 0.0;
					Attr.Get<float>(&Default);
					Default = UsdToUnreal::ConvertDistance(StageInfo, Default);

					Reader.FloatReader = [Attr, Default, StageInfo](double UsdTimeCode)
					{
						float Result = Default;
						if (Attr.Get<float>(&Result, UsdTimeCode))
						{
							Result = UsdToUnreal::ConvertDistance(StageInfo, Result);
						}

						return Result;
					};
					return Reader;
				}
			}
			else if (PropertyPath == UnrealIdentifiers::IntensityPropertyName)
			{
				pxr::UsdAttribute IntensityAttr = RectLight.GetIntensityAttr();
				pxr::UsdAttribute ExposureAttr = RectLight.GetExposureAttr();

				if (IntensityAttr && ExposureAttr)
				{
					// Default values directly from pxr/usd/usdLux/schema.usda
					float DefaultUsdIntensity = 1.0f;
					float DefaultUsdExposure = 0.0f;

					IntensityAttr.Get<float>(&DefaultUsdIntensity);
					ExposureAttr.Get<float>(&DefaultUsdExposure);

					float Default = UsdToUnreal::ConvertLightIntensity(DefaultUsdIntensity, DefaultUsdExposure);

					Reader.FloatReader = [IntensityAttr, ExposureAttr, Default, StageInfo](double UsdTimeCode)
					{
						float Result = Default;

						float UsdIntensity = 1.0f;
						float UsdExposure = 0.0f;
						if (IntensityAttr.Get<float>(&UsdIntensity, UsdTimeCode) && ExposureAttr.Get<float>(&UsdExposure, UsdTimeCode))
						{
							Result = UsdToUnreal::ConvertLightIntensity(UsdIntensity, UsdExposure);
						}

						return Result;
					};
					return Reader;
				}
			}
		}
		else if (pxr::UsdLuxDiskLight DiskLight{UsdPrim})
		{
			if (PropertyPath == UnrealIdentifiers::SourceWidthPropertyName || PropertyPath == UnrealIdentifiers::SourceHeightPropertyName)
			{
				if (pxr::UsdAttribute Attr = DiskLight.GetRadiusAttr())
				{
					// The convention is that we fake a disk light with a rect light. For area lights the most important thing
					// is to preserve the area though, so here we compute the side of the rect light that would produce the same
					// total area as the disk
					float Default = 0.0;
					Attr.Get<float>(&Default);
					float UsdArea = PI * Default * Default;
					float UsdSquareSide = FMath::Sqrt(UsdArea);
					Default = UsdToUnreal::ConvertDistance(StageInfo, UsdSquareSide);

					Reader.FloatReader = [Attr, Default, StageInfo](double UsdTimeCode)
					{
						float Result = Default;
						if (Attr.Get<float>(&Result, UsdTimeCode))
						{
							float UsdArea = PI * Result * Result;
							float UsdSquareSide = FMath::Sqrt(UsdArea);
							Result = UsdToUnreal::ConvertDistance(StageInfo, UsdSquareSide);
						}

						return Result;
					};
					return Reader;
				}
			}
			else if (PropertyPath == UnrealIdentifiers::IntensityPropertyName)
			{
				pxr::UsdAttribute IntensityAttr = DiskLight.GetIntensityAttr();
				pxr::UsdAttribute ExposureAttr = DiskLight.GetExposureAttr();

				if (IntensityAttr && ExposureAttr)
				{
					// Default values directly from pxr/usd/usdLux/schema.usda
					float DefaultUsdIntensity = 1.0f;
					float DefaultUsdExposure = 0.0f;

					IntensityAttr.Get<float>(&DefaultUsdIntensity);
					ExposureAttr.Get<float>(&DefaultUsdExposure);

					float Default = UsdToUnreal::ConvertLightIntensity(DefaultUsdIntensity, DefaultUsdExposure);

					Reader.FloatReader = [IntensityAttr, ExposureAttr, Default, StageInfo](double UsdTimeCode)
					{
						float Result = Default;

						float UsdIntensity = 1.0f;
						float UsdExposure = 0.0f;
						if (IntensityAttr.Get<float>(&UsdIntensity, UsdTimeCode) && ExposureAttr.Get<float>(&UsdExposure, UsdTimeCode))
						{
							Result = UsdToUnreal::ConvertLightIntensity(UsdIntensity, UsdExposure);
						}

						return Result;
					};
					return Reader;
				}
			}
		}
		else if (pxr::UsdLuxDistantLight DistantLight{UsdPrim})
		{
			if (PropertyPath == UnrealIdentifiers::LightSourceAnglePropertyName)
			{
				if (pxr::UsdAttribute Attr = DistantLight.GetAngleAttr())
				{
					float Default;
					Attr.Get<float>(&Default);

					Reader.FloatReader = [Attr, Default](double UsdTimeCode)
					{
						float Result = Default;
						Attr.Get<float>(&Result, UsdTimeCode);
						return Result;
					};
					return Reader;
				}
			}
			else if (PropertyPath == UnrealIdentifiers::IntensityPropertyName)
			{
				pxr::UsdAttribute IntensityAttr = DistantLight.GetIntensityAttr();
				pxr::UsdAttribute ExposureAttr = DistantLight.GetExposureAttr();

				if (IntensityAttr && ExposureAttr)
				{
					// Default values directly from pxr/usd/usdLux/schema.usda
					float DefaultUsdIntensity = 1.0f;
					float DefaultUsdExposure = 0.0f;

					IntensityAttr.Get<float>(&DefaultUsdIntensity);
					ExposureAttr.Get<float>(&DefaultUsdExposure);

					float Default = UsdToUnreal::ConvertLightIntensity(DefaultUsdIntensity, DefaultUsdExposure);

					Reader.FloatReader = [IntensityAttr, ExposureAttr, Default, StageInfo](double UsdTimeCode)
					{
						float Result = Default;

						float UsdIntensity = 1.0f;
						float UsdExposure = 0.0f;
						if (IntensityAttr.Get<float>(&UsdIntensity, UsdTimeCode) && ExposureAttr.Get<float>(&UsdExposure, UsdTimeCode))
						{
							Result = UsdToUnreal::ConvertLightIntensity(UsdIntensity, UsdExposure);
						}

						return Result;
					};
					return Reader;
				}
			}
		}
		else if (UsdPrim.IsA<pxr::UsdLuxDomeLight>() || UsdPrim.IsA<pxr::UsdLuxDomeLight_1>())
		{
			pxr::UsdLuxNonboundableLightBase LightBase{UsdPrim};

			if (PropertyPath == UnrealIdentifiers::IntensityPropertyName)
			{
				pxr::UsdAttribute IntensityAttr = LightBase.GetIntensityAttr();
				pxr::UsdAttribute ExposureAttr = LightBase.GetExposureAttr();

				if (IntensityAttr && ExposureAttr)
				{
					// Default values directly from pxr/usd/usdLux/schema.usda
					float DefaultUsdIntensity = 1.0f;
					float DefaultUsdExposure = 0.0f;

					IntensityAttr.Get<float>(&DefaultUsdIntensity);
					ExposureAttr.Get<float>(&DefaultUsdExposure);

					float Default = UsdToUnreal::ConvertLightIntensity(DefaultUsdIntensity, DefaultUsdExposure);

					Reader.FloatReader = [IntensityAttr, ExposureAttr, Default, StageInfo](double UsdTimeCode)
					{
						float Result = Default;

						float UsdIntensity = 1.0f;
						float UsdExposure = 0.0f;
						if (IntensityAttr.Get<float>(&UsdIntensity, UsdTimeCode) && ExposureAttr.Get<float>(&UsdExposure, UsdTimeCode))
						{
							Result = UsdToUnreal::ConvertLightIntensity(UsdIntensity, UsdExposure);
						}

						return Result;
					};
					return Reader;
				}
			}
		}
	}
	else if (const pxr::UsdMediaSpatialAudio Audio{Prim})
	{
		if (PropertyPath == TEXT("Volume"))
		{
			if (pxr::UsdAttribute Attr = Audio.GetGainAttr())
			{
				double Default = 1.0;
				Attr.Get<double>(&Default);

				// The VolumeMultiplier property is a float anyway, so we may as well convert
				// doubles to floats right here
				Reader.FloatReader = [Attr, Default](double UsdTimeCode)
				{
					double Result = Default;
					Attr.Get<double>(&Result, UsdTimeCode);
					return static_cast<float>(FMath::Max(Result, 0.0));
				};
				return Reader;
			}
		}
	}

	return Reader;
}

bool UsdToUnreal::ConvertDrawMode(
	const pxr::UsdPrim& Prim,
	UUsdDrawModeComponent* DrawModeComponent,
	double EvalTime,
	pxr::UsdGeomBBoxCache* BBoxCache
)
{
	// We're not going to check if Prim actually has the "bounds" draw mode or if it has "applyDrawMode" set
	// to true, as that can be expensive and this can get called from UpdateComponents, which can get called
	// every frame of Time animation. If we have a UUsdDrawModeComponent at all we'll assume we're OK here
	if (!DrawModeComponent)
	{
		return false;
	}

	FScopedUsdAllocs UsdAllocs;

	TOptional<pxr::UsdGeomBBoxCache> TempBBoxCacheStorage;
	if (BBoxCache == nullptr)
	{
		const bool bUseExtentsHint = true;
		const bool bIgnoreVisibility = false;
		static std::vector<pxr::TfToken> DefaultTokenVector{pxr::UsdGeomTokens->proxy, pxr::UsdGeomTokens->render};
		TempBBoxCacheStorage.Emplace(EvalTime, DefaultTokenVector, bUseExtentsHint, bIgnoreVisibility);
		BBoxCache = &TempBBoxCacheStorage.GetValue();
	}

	// BBoxCache only considers prims with animated transforms or visibility as needing animated computed bounds.
	// This doesn't include e.g. animated points, meaning it would try reusing animated mesh bounds across frames for those animations,
	// which can be very incorrect depending on the animation.
	// With this very expensive trick we can flush the entire BBoxCache and compute new bounds every time, for each particular
	// timeSample, which can get us accurate bounds for every frame. Obviously disabled by default, but could be useful if all you want
	// is to import once, or something like this.
	if (GConsiderAllPrimsHaveAnimatedBounds)
	{
		BBoxCache->Clear();
	}

	// We should do this here or else the BBoxCache may be set to a different time and we'd be reading wrong bounds
	// (This can happening when opening stages as we may switch the BBoxCache time around when setting up the Sequencer tracks).
	// Note that this does nothing in case BBoxCache is already at this time
	BBoxCache->SetTime(EvalTime);

	// Note: This can be extremely expensive, as it may fallback to computing new bounds, traversing points and everything for the entire subtree.
	// We don't have a choice if we want decent bounds though, and in practice the user's assets will have (or can be set with) authored bounds,
	// that should make this pretty fast
	pxr::GfBBox3d BoxAndMatrix = BBoxCache->ComputeUntransformedBound(Prim);
	pxr::GfRange3d Box = BoxAndMatrix.ComputeAlignedRange();

	// USD will return a FLT_MAX box in case the prim doesn't contain anything, so we need to check for that as putting FLT_MAX directly
	// on the component bounds is bad news
	if (!Box.IsEmpty())
	{
		// Note that after we convert the USD min/max to UE coordinate space, due to stage up axis the points may flip sign
		// (e.g. the USD max ends up at UE's negative Y axis), so we need to compute min/max in UE space again
		FUsdStageInfo StageInfo{Prim.GetStage()};
		FVector UESpaceUSDMin = UsdToUnreal::ConvertVector(StageInfo, Box.GetMin());
		FVector UESpaceUSDMax = UsdToUnreal::ConvertVector(StageInfo, Box.GetMax());
		FBox UEBox{
			TArray<FVector>{UESpaceUSDMin, UESpaceUSDMax}
		};

		DrawModeComponent->SetBoundsMin(UEBox.Min);
		DrawModeComponent->SetBoundsMax(UEBox.Max);
	}

	if (pxr::UsdGeomModelAPI GeomModelAPI{Prim})
	{
		pxr::GfVec3f Color;
		pxr::UsdAttribute ColorAttr = GeomModelAPI.GetModelDrawModeColorAttr();
		if (ColorAttr && ColorAttr.Get(&Color))
		{
			DrawModeComponent->SetBoundsColor(UsdToUnreal::ConvertColor(Color));
		}

		// It's not super efficient to call this whole function every time but not only this lets us
		// resolve inherited values for the draw mode, but also lets us have our logic in one place.
		// That is useful because GetAppliedDrawMode is used to decide which component to spawn, and
		// it will return Default even when we have a particular draw mode specified (in case e.g.
		// the prim is not a model, or doesn't have applyDrawMode enabled, etc.) so we can't just check
		// the drawMode attribute directly here
		EUsdDrawMode DrawMode = UsdUtils::GetAppliedDrawMode(Prim);
		DrawModeComponent->SetDrawMode(DrawMode);

		pxr::TfToken CardGeometryToken;
		pxr::UsdAttribute GeometryAttr = GeomModelAPI.GetModelCardGeometryAttr();
		if (GeometryAttr && GeometryAttr.Get(&CardGeometryToken))
		{
			EUsdModelCardGeometry CardGeometry = EUsdModelCardGeometry::Cross;
			if (CardGeometryToken == pxr::UsdGeomTokens->box)
			{
				CardGeometry = EUsdModelCardGeometry::Box;
			}
			else if (CardGeometryToken == pxr::UsdGeomTokens->fromTexture)
			{
				CardGeometry = EUsdModelCardGeometry::FromTexture;
			}
			DrawModeComponent->SetCardGeometry(CardGeometry);
		}
	}

	return true;
}

namespace UE::USDPrimConversion::Private
{
	const static std::string UsdNamespaceDelimiter = UnrealToUsd::ConvertString(*UnrealIdentifiers::UsdNamespaceDelimiter).Get();

	bool ShouldSkipField(const FString& FullFieldPath, const TArray<FString>& BlockedPrefixFilters, bool bInvertFilters)
	{
		if (bInvertFilters)
		{
			// Yes this can be simplified further as this code is just a copy paste of the case below,
			// but splitting the cases should be quicker to understand
			for (const FString& Prefix : BlockedPrefixFilters)
			{
				if (FullFieldPath.StartsWith(Prefix))
				{
					return false;
				}
			}

			return true;
		}
		else
		{
			for (const FString& Prefix : BlockedPrefixFilters)
			{
				if (FullFieldPath.StartsWith(Prefix))
				{
					return true;
				}
			}

			return false;
		}
	}

	// Converts the entries within Dictionary into metadata entries within InOutPrimMetadata, using the provided filters
	// and the additional FieldPathPrefix for the entry keys
	void ConvertMetadataDictionary(
		const pxr::VtDictionary& Dictionary,
		const std::string& FieldPathPrefix,
		FUsdPrimMetadata& InOutPrimMetadata,
		const TArray<FString>& BlockedPrefixFilters = {},
		bool bInvertFilters = false
	)
	{
		FScopedUsdAllocs Allocs;

		// We should only call this for nested dicts, at which point we should already have a field path prefix
		ensure(!FieldPathPrefix.empty());

		for (pxr::VtDictionary::const_iterator ValueIter = Dictionary.begin(); ValueIter != Dictionary.end(); ++ValueIter)
		{
			const std::string& DictFieldName = ValueIter->first;
			const pxr::VtValue& DictFieldValue = ValueIter->second;

			std::string FieldFullPath = FieldPathPrefix + UsdNamespaceDelimiter + DictFieldName;
			FString FieldFullString = UsdToUnreal::ConvertString(FieldFullPath);

			if (DictFieldValue.IsHolding<pxr::VtDictionary>())
			{
				ConvertMetadataDictionary(
					DictFieldValue.UncheckedGet<pxr::VtDictionary>(),
					FieldFullPath,
					InOutPrimMetadata,
					BlockedPrefixFilters,
					bInvertFilters
				);
			}
			else
			{
				// Note how we only check the filter when we have our *full* key path. It may seem wasteful to not do an early check in case
				// the field is a dictionary, but consider this:
				// 	Field: "customData:int"	AllowFilter: "customData"  		--> Should allow
				// 	Field: "customData"		AllowFilter: "customData:int"  	--> Should... also allow? If we want to eventually allow the int
				//                                                              we need to allow its parent dict too
				//  Field: "abcde"			AllowFilter: "ab"  				--> Should allow
				//  Field: "ab"				AllowFilter: "abcde"  			--> Should... not allow? It really doesn't start with that prefix...
				// Our desired behavior changes when the prefix consists of an "incomplete path" to the key we're really interested in...
				// The simplest way to handle that is probably to simply never early compare the path like in the second example at all, by
				// only ever checking *full* paths against the filter, which is what we're doing here.
				if (ShouldSkipField(FieldFullString, BlockedPrefixFilters, bInvertFilters))
				{
					continue;
				}

				FUsdMetadataValue& Metadata = InOutPrimMetadata.Metadata.FindOrAdd(FieldFullString);
				Metadata.StringifiedValue = UsdUtils::Stringify(DictFieldValue);

				// Prefer the SdfTypeNameToken over Value.GetTypeName() (the former is like "double3[]" and is the
				// same you type out on the .usda files, while the latter matches the C++ type, like "VtArray<GfVec3d>")
				if (pxr::SdfValueTypeName TypeName = pxr::SdfGetValueTypeNameForValue(DictFieldValue))
				{
					Metadata.TypeName = UsdToUnreal::ConvertToken(TypeName.GetAsToken());
				}
				else
				{
					Metadata.TypeName = UsdToUnreal::ConvertString(DictFieldValue.GetTypeName());
				}
			}
		}
	}

	void CollectMetadataForPrim(
		const pxr::UsdPrim& Prim,
		FUsdCombinedPrimMetadata& InOutCombinedMetadata,
		const TArray<FString>& BlockedPrefixFilters,
		bool bInvertFilters
	)
	{
		if (!Prim)
		{
			return;
		}

		FScopedUsdAllocs Allocs;

		FUsdPrimMetadata* PrimMetadata = nullptr;

		std::map<pxr::TfToken, pxr::VtValue, pxr::TfDictionaryLessThan> MetadataMap = Prim.GetAllAuthoredMetadata();
		for (std::map<pxr::TfToken, pxr::VtValue, pxr::TfDictionaryLessThan>::const_iterator MetadataIter = MetadataMap.begin();
			 MetadataIter != MetadataMap.end();
			 ++MetadataIter)
		{
			const pxr::TfToken& FieldName = MetadataIter->first;
			const pxr::VtValue& FieldValue = MetadataIter->second;

			// There is no real point in keeping track of these as they are defined on every prim and should just match
			// what the prim's definition is. It's probably a bad idea to author a value that differs from the prim
			// definition too
			const static std::unordered_set<pxr::TfToken, pxr::TfHash> FieldsToSkip = {pxr::SdfFieldKeys->Specifier, pxr::SdfFieldKeys->TypeName};
			if (FieldsToSkip.count(FieldName) > 0)
			{
				continue;
			}

			// We have a valid field we want to collect, so let's on-demand create a PrimMetadata entry.
			// Creating on-demand prevents us from creating useless structs for prims without metadata.
			if (!PrimMetadata)
			{
				FString PrimPath = UsdToUnreal::ConvertPath(Prim.GetPrimPath());
				PrimMetadata = &InOutCombinedMetadata.PrimPathToMetadata.FindOrAdd(PrimPath);
			}

			if (FieldValue.IsHolding<pxr::VtDictionary>())
			{
				ConvertMetadataDictionary(
					FieldValue.UncheckedGet<pxr::VtDictionary>(),
					FieldName.GetString(),
					*PrimMetadata,
					BlockedPrefixFilters,
					bInvertFilters
				);
			}
			else
			{
				// c.f. the comment within ConvertMetadataDictionary
				FString FieldNameString = UsdToUnreal::ConvertToken(FieldName);
				if (ShouldSkipField(FieldNameString, BlockedPrefixFilters, bInvertFilters))
				{
					continue;
				}

				FUsdMetadataValue& Metadata = PrimMetadata->Metadata.FindOrAdd(FieldNameString);
				Metadata.StringifiedValue = UsdUtils::Stringify(FieldValue);

				if (pxr::SdfValueTypeName TypeName = pxr::SdfGetValueTypeNameForValue(FieldValue))
				{
					Metadata.TypeName = UsdToUnreal::ConvertToken(TypeName.GetAsToken());
				}
				else
				{
					Metadata.TypeName = UsdToUnreal::ConvertString(FieldValue.GetTypeName());
				}
			}
		}
	}
};	  // namespace UE::USDPrimConversion::Private

bool UsdToUnreal::ConvertMetadata(
	const pxr::UsdPrim& Prim,
	FUsdCombinedPrimMetadata& CombinedMetadata,
	const TArray<FString>& BlockedPrefixFilters,
	bool bInvertFilters,
	bool bCollectFromEntireSubtrees
)
{
	if (!Prim)
	{
		return false;
	}

	UE::USDPrimConversion::Private::CollectMetadataForPrim(Prim, CombinedMetadata, BlockedPrefixFilters, bInvertFilters);

	if (bCollectFromEntireSubtrees)
	{
		pxr::UsdPrimRange PrimRange{Prim, pxr::UsdTraverseInstanceProxies()};
		for (pxr::UsdPrimRange::iterator It = ++PrimRange.begin(); It != PrimRange.end(); ++It)
		{
			UE::USDPrimConversion::Private::CollectMetadataForPrim(*It, CombinedMetadata, BlockedPrefixFilters, bInvertFilters);
		}
	}

	return true;
}

bool UsdToUnreal::ConvertMetadata(
	const pxr::UsdPrim& Prim,
	UUsdAssetUserData* AssetUserData,
	const TArray<FString>& BlockedPrefixFilters,
	bool bInvertFilters,
	bool bCollectFromEntireSubtrees
)
{
	if (!Prim || !AssetUserData)
	{
		return false;
	}

	FScopedUsdAllocs Allocs;

	FString PrimPath = UsdToUnreal::ConvertPath(Prim.GetPrimPath());
	pxr::UsdStageRefPtr StagePtr = Prim.GetStage();
	FString StageIdentifier = UsdToUnreal::ConvertString(StagePtr->GetRootLayer()->GetIdentifier());

	FUsdCombinedPrimMetadata& CombinedMetadata = AssetUserData->StageIdentifierToMetadata.FindOrAdd(StageIdentifier);

	return UsdToUnreal::ConvertMetadata(Prim, CombinedMetadata, BlockedPrefixFilters, bInvertFilters, bCollectFromEntireSubtrees);
}

bool UnrealToUsd::ConvertCameraComponent(const UCineCameraComponent& CameraComponent, pxr::UsdPrim& Prim, double UsdTimeCode)
{
	FScopedUsdAllocs UsdAllocs;

	pxr::UsdGeomCamera GeomCamera{Prim};
	if (!GeomCamera)
	{
		return false;
	}

	FUsdStageInfo StageInfo(Prim.GetStage());

	if (pxr::UsdAttribute Attr = GeomCamera.CreateFocalLengthAttr())
	{
		Attr.Set<float>(UnrealToUsd::ConvertDistance(StageInfo, CameraComponent.CurrentFocalLength), UsdTimeCode);
		UsdUtils::NotifyIfOverriddenOpinion(Attr);
	}

	if (CameraComponent.FocusSettings.FocusMethod == ECameraFocusMethod::Manual)
	{
		if (CameraComponent.PostProcessSettings.bOverride_DepthOfFieldFstop)
		{
			if (pxr::UsdAttribute Attr = GeomCamera.CreateFStopAttr())
			{
				Attr.Set<float>(CameraComponent.PostProcessSettings.DepthOfFieldFstop, UsdTimeCode);
				UsdUtils::NotifyIfOverriddenOpinion(Attr);
			}
		}

		if (pxr::UsdAttribute Attr = GeomCamera.CreateFocusDistanceAttr())
		{
			Attr.Set<float>(UnrealToUsd::ConvertDistance(StageInfo, CameraComponent.FocusSettings.ManualFocusDistance), UsdTimeCode);
			UsdUtils::NotifyIfOverriddenOpinion(Attr);
		}
	}

	if (pxr::UsdAttribute Attr = GeomCamera.CreateHorizontalApertureAttr())
	{
		Attr.Set<float>(UnrealToUsd::ConvertDistance(StageInfo, CameraComponent.Filmback.SensorWidth), UsdTimeCode);
		UsdUtils::NotifyIfOverriddenOpinion(Attr);
	}

	if (pxr::UsdAttribute Attr = GeomCamera.CreateVerticalApertureAttr())
	{
		Attr.Set<float>(UnrealToUsd::ConvertDistance(StageInfo, CameraComponent.Filmback.SensorHeight), UsdTimeCode);
		UsdUtils::NotifyIfOverriddenOpinion(Attr);
	}

	if (pxr::UsdAttribute Attr = GeomCamera.CreateHorizontalApertureOffsetAttr())
	{
		Attr.Set<float>(UnrealToUsd::ConvertDistance(StageInfo, CameraComponent.Filmback.SensorHorizontalOffset), UsdTimeCode);
		UsdUtils::NotifyIfOverriddenOpinion(Attr);
	}

	if (pxr::UsdAttribute Attr = GeomCamera.CreateVerticalApertureOffsetAttr())
	{
		Attr.Set<float>(UnrealToUsd::ConvertDistance(StageInfo, CameraComponent.Filmback.SensorVerticalOffset), UsdTimeCode);
		UsdUtils::NotifyIfOverriddenOpinion(Attr);
	}

	if (CameraComponent.PostProcessSettings.bOverride_AutoExposureBias)
	{
		if (pxr::UsdAttribute Attr = GeomCamera.CreateExposureAttr())
		{
			Attr.Set<float>(CameraComponent.PostProcessSettings.AutoExposureBias, UsdTimeCode);
			UsdUtils::NotifyIfOverriddenOpinion(Attr);
		}
	}

	if (pxr::UsdAttribute Attr = GeomCamera.CreateProjectionAttr())
	{
		Attr.Set<pxr::TfToken>(
			CameraComponent.ProjectionMode == ECameraProjectionMode::Orthographic ? pxr::UsdGeomTokens->orthographic
																				  : pxr::UsdGeomTokens->perspective,
			UsdTimeCode
		);
		UsdUtils::NotifyIfOverriddenOpinion(Attr);
	}

	// Write clipping planes from the orthographic mode properties, if we're in orthographic mode
	if (CameraComponent.ProjectionMode == ECameraProjectionMode::Orthographic)
	{
		if (pxr::UsdAttribute Attr = GeomCamera.CreateClippingRangeAttr())
		{
			Attr.Set<pxr::GfVec2f>(
				pxr::GfVec2f{
					UnrealToUsd::ConvertDistance(StageInfo, CameraComponent.OrthoNearClipPlane),
					UnrealToUsd::ConvertDistance(StageInfo, CameraComponent.OrthoFarClipPlane)
				},
				UsdTimeCode
			);
			UsdUtils::NotifyIfOverriddenOpinion(Attr);
		}
	}
	else
	{
		if (pxr::UsdAttribute Attr = GeomCamera.CreateClippingRangeAttr(); CameraComponent.bOverride_CustomNearClippingPlane)
		{
			// We only have a property for near, so let's leave the far plane at its current value
			pxr::GfVec2f ClippingRangeUsdUnits{1, 1000000};	   // Values copied from the schema, as a fallback
			Attr.Get(&ClippingRangeUsdUnits, UsdTimeCode);

			ClippingRangeUsdUnits[0] = UnrealToUsd::ConvertDistance(StageInfo, CameraComponent.CustomNearClippingPlane);

			Attr.Set<pxr::GfVec2f>(ClippingRangeUsdUnits, UsdTimeCode);

			UsdUtils::NotifyIfOverriddenOpinion(Attr);
		}
	}

	if (CameraComponent.PostProcessSettings.bOverride_AutoExposureMethod && CameraComponent.PostProcessSettings.AutoExposureMethod == EAutoExposureMethod::AEM_Manual)
	{
		if (pxr::UsdAttribute Attr = GeomCamera.CreateExposureFStopAttr())
		{
			Attr.Set<float>(CameraComponent.CurrentAperture, UsdTimeCode);
			UsdUtils::NotifyIfOverriddenOpinion(Attr);
		}

		if (CameraComponent.PostProcessSettings.bOverride_CameraISO)
		{
			if (pxr::UsdAttribute Attr = GeomCamera.CreateExposureIsoAttr())
			{
				Attr.Set<float>(CameraComponent.PostProcessSettings.CameraISO, UsdTimeCode);
				UsdUtils::NotifyIfOverriddenOpinion(Attr);
			}
		}

		if (CameraComponent.PostProcessSettings.bOverride_CameraShutterSpeed)
		{
			if (pxr::UsdAttribute Attr = GeomCamera.CreateExposureTimeAttr())
			{
				float UsdValue = CameraComponent.PostProcessSettings.CameraShutterSpeed != 0.0f ? (1.0f / CameraComponent.PostProcessSettings.CameraShutterSpeed) : 0.0f;
				Attr.Set<float>(UsdValue, UsdTimeCode);
				UsdUtils::NotifyIfOverriddenOpinion(Attr);
			}
		}
	}

	return true;
}

bool UnrealToUsd::ConvertAudioComponent(const UAudioComponent& AudioComponent, pxr::UsdPrim& Prim, bool bFilePathOnly, double UsdTimeCode)
{
	FScopedUsdAllocs UsdAllocs;

	pxr::UsdMediaSpatialAudio SpatialAudio{Prim};
	if (!SpatialAudio)
	{
		return false;
	}

	if (!bFilePathOnly)
	{
		// auralmode
		if (pxr::UsdAttribute Attr = SpatialAudio.CreateAuralModeAttr())
		{
			const bool bIsSpatial = AudioComponent.bAllowSpatialization
									&& (AudioComponent.bOverrideAttenuation || AudioComponent.AttenuationSettings);

			Attr.Set<pxr::TfToken>(bIsSpatial ? pxr::UsdMediaTokens->spatial : pxr::UsdMediaTokens->nonSpatial);
			UsdUtils::NotifyIfOverriddenOpinion(Attr);
		}

		// gain
		if (pxr::UsdAttribute Attr = SpatialAudio.CreateGainAttr())
		{
			Attr.Set<double>(static_cast<double>(AudioComponent.VolumeMultiplier), UsdTimeCode);
			UsdUtils::NotifyIfOverriddenOpinion(Attr);
		}

		// playbackMode
		if (pxr::UsdAttribute Attr = SpatialAudio.CreatePlaybackModeAttr())
		{
			if (USoundWave* Sound = Cast<USoundWave>(AudioComponent.Sound))	   // GetSound() is not const so we can't use it
			{
				const bool bIsLooping = Sound->IsLooping();

				Attr.Set<pxr::TfToken>(bIsLooping ? pxr::UsdMediaTokens->loopFromStart : pxr::UsdMediaTokens->onceFromStart);
				UsdUtils::NotifyIfOverriddenOpinion(Attr);
			}
		}

		// startTime
		// We don't really want to author anything here, but since there is no concept of "startTime" on the UE side
		// we really want this audio to play at startTime zero, so if for some reason this prim already has an opinion
		// otherwise we need to override it (note how we're just using GetAttr instead of CreateAttr).
		if (pxr::UsdAttribute Attr = SpatialAudio.GetStartTimeAttr())
		{
			Attr.Set<pxr::SdfTimeCode>(0.0);
			UsdUtils::NotifyIfOverriddenOpinion(Attr);
		}
	}

#if WITH_EDITOR
	// filePath
	if (pxr::UsdAttribute Attr = SpatialAudio.CreateFilePathAttr())
	{
		UsdUtils::NotifyIfOverriddenOpinion(Attr);

		FString FilePath;
		if (USoundWave* Sound = Cast<USoundWave>(AudioComponent.Sound))	   // GetSound() is not const so we can't use it
		{
			if (UAssetImportData* ImportData = Sound->AssetImportData)
			{
				FilePath = ImportData->GetFirstFilename();
			}
		}

		if (!FilePath.IsEmpty())
		{
			if (!FPaths::FileExists(FilePath))
			{
				USD_LOG_WARNING(
					TEXT(
						"Used '%s' as the audio file when converting AudioComponent '%s' onto prim '%s', but the file does not exist on the filesystem!"
					),
					*FilePath,
					*AudioComponent.GetPathName(),
					*UsdToUnreal::ConvertPath(Prim.GetPrimPath())
				);
			}

			pxr::SdfAssetPath AssetPath{UnrealToUsd::ConvertString(*FilePath).Get()};
			Attr.Set(AssetPath);
		}
		else
		{
			Attr.ClearAtTime(pxr::UsdTimeCode::Default());
		}
	}
#endif	  // WITH_EDITOR

	return true;
}

bool UnrealToUsd::ConvertAudioSection(
	const UMovieSceneAudioSection& AudioSection,
	const FMovieSceneSequenceTransform& SequenceTransform,
	pxr::UsdPrim& Prim
)
{
	FScopedUsdAllocs UsdAllocs;

	pxr::UsdMediaSpatialAudio SpatialAudio{Prim};
	if (!SpatialAudio)
	{
		return false;
	}

	UMovieScene* MovieScene = AudioSection.GetTypedOuter<UMovieScene>();
	if (!MovieScene)
	{
		return false;
	}

	const double StageTimeCodesPerSecond = Prim.GetStage()->GetTimeCodesPerSecond();
	const FFrameRate StageFrameRate(StageTimeCodesPerSecond, 1);

#if WITH_EDITOR
	// filePath
	if (pxr::UsdAttribute Attr = SpatialAudio.CreateFilePathAttr())
	{
		FString FilePath;
		if (const USoundWave* Sound = Cast<USoundWave>(AudioSection.GetSound()))
		{
			if (UAssetImportData* ImportData = Sound->AssetImportData)
			{
				FilePath = ImportData->GetFirstFilename();
			}
		}

		if (!FilePath.IsEmpty())
		{
			// Don't author anything if we're just trying to set a relative path version
			// of the same file that is currently set as an absolute path (or vice-versa)
			bool bSetNewPath = true;
			if (Attr.HasAuthoredValue())
			{
				FString CurrentPath = UsdUtils::GetResolvedAssetPath(Attr, pxr::UsdTimeCode::Default());

				// Both paths should be absolute at this point
				bSetNewPath = !FPaths::IsSamePath(FilePath, CurrentPath);
			}

			if (bSetNewPath)
			{
				UsdUtils::NotifyIfOverriddenOpinion(Attr);
				pxr::SdfAssetPath AssetPath{UnrealToUsd::ConvertString(*FilePath).Get()};
				Attr.Set(AssetPath);
			}
		}
		else
		{
			UsdUtils::NotifyIfOverriddenOpinion(Attr);
			Attr.ClearAtTime(pxr::UsdTimeCode::Default());
		}
	}
#endif	  // WITH_EDITOR

	// mediaOffset
	if (pxr::UsdAttribute Attr = SpatialAudio.CreateMediaOffsetAttr())
	{
		const FFrameNumber& StartOffset = AudioSection.GetStartOffset();

		UsdUtils::NotifyIfOverriddenOpinion(Attr);

		const FFrameRate Resolution = MovieScene->GetTickResolution();
		double OffsetSeconds = Resolution.AsSeconds(StartOffset);

		Attr.Set(OffsetSeconds);
	}

	// startTime and endTime
	double bChangedTimes = false;
	pxr::UsdAttribute StartAttr = SpatialAudio.CreateStartTimeAttr();
	pxr::UsdAttribute EndAttr = SpatialAudio.CreateEndTimeAttr();
	if (StartAttr && EndAttr)
	{
		FMovieSceneInverseSequenceTransform InverseSequenceTransform = SequenceTransform.Inverse();

		UsdUtils::NotifyIfOverriddenOpinion(StartAttr);
		UsdUtils::NotifyIfOverriddenOpinion(EndAttr);

		const FFrameRate Resolution = MovieScene->GetTickResolution();
		const FFrameRate DisplayRate = MovieScene->GetDisplayRate();

		TRange<FFrameNumber> Range = AudioSection.GetRange();
		const FFrameNumber& StartTick = Range.GetLowerBoundValue();
		const FFrameNumber& EndTick = Range.GetUpperBoundValue();

		FFrameTime UsdStartTickTime = FFrameRate::Snap(StartTick, Resolution, DisplayRate).FloorToFrame();
		FFrameTime UsdEndTickTime = FFrameRate::Snap(EndTick, Resolution, DisplayRate).FloorToFrame();
		UsdStartTickTime = InverseSequenceTransform.TryTransformTime(UsdStartTickTime).Get(UsdStartTickTime);
		UsdEndTickTime = InverseSequenceTransform.TryTransformTime(UsdEndTickTime).Get(UsdEndTickTime);
		double UsdStartTimeCode = FFrameRate::TransformTime(UsdStartTickTime, Resolution, StageFrameRate).AsDecimal();
		double UsdEndTimeCode = FFrameRate::TransformTime(UsdEndTickTime, Resolution, StageFrameRate).AsDecimal();

		// Get the previous start/end times from the attribute
		bool bGotPreviousStart = false;
		bool bGotPreviousEnd = false;
		pxr::SdfTimeCode PreviousStartTimeCode = 0.0;
		pxr::SdfTimeCode PreviousEndTimeCode = 0.0;
		if (StartAttr.Get(&PreviousStartTimeCode))
		{
			bGotPreviousStart = true;
		}
		if (EndAttr.Get(&PreviousEndTimeCode))
		{
			bGotPreviousEnd = true;
		}

		StartAttr.Set(pxr::SdfTimeCode{UsdStartTimeCode});
		EndAttr.Set(pxr::SdfTimeCode{UsdEndTimeCode});

		// Retrieve the new values from the attribute and record whether they changed or not.
		//
		// We're not just using the values we just set above (e.g. UsdStartTimeCode) because
		// USD does automatic layer offset/scale conversions for these, that we would need to manually
		// undo here anyway in order to properly compare with the previous values.
		// We may as well just let USD do them by calling Get() instead, and then we can know
		// for sure whether our actual set values changed
		pxr::SdfTimeCode SetStartTimeCode = 0.0;
		pxr::SdfTimeCode SetEndTimeCode = 0.0;
		if (bGotPreviousStart && StartAttr.Get(&SetStartTimeCode))
		{
			if (!FMath::IsNearlyEqual(PreviousStartTimeCode.GetValue(), SetStartTimeCode.GetValue()))
			{
				bChangedTimes = true;
			}
		}
		if (!bChangedTimes && bGotPreviousEnd && EndAttr.Get(&SetEndTimeCode))
		{
			if (!FMath::IsNearlyEqual(PreviousEndTimeCode.GetValue(), SetEndTimeCode.GetValue()))
			{
				bChangedTimes = true;
			}
		}
	}

	// playbackMode
	// We lose some "degrees of freedom" here by just authoring 2 out of the 5 possible values, but in Unreal
	// the audio section must always have closed start and end frames, which means only the "FromStartToEnd"
	// options really make sense. It would probably be more confusing to try and come up with some heuristics
	// as to when we should guess that the playbackMode really should be something else.
	// As a small concession though, we'll only change the playback mode in case we have some new value for
	// start/endTime/looping, so that the user can keep their custom playbackMode until we have to actually
	// change it
	if (pxr::UsdAttribute Attr = SpatialAudio.CreatePlaybackModeAttr())
	{
		bool bPreviousLooping = false;
		pxr::TfToken CurrentPlaybackMode;
		if (Attr.Get(&CurrentPlaybackMode))
		{
			bPreviousLooping = (CurrentPlaybackMode != pxr::UsdMediaTokens->onceFromStart)
							   && (CurrentPlaybackMode != pxr::UsdMediaTokens->onceFromStartToEnd);
		}

		const bool bNowLooping = AudioSection.GetLooping();
		const bool bChangedLooping = bNowLooping ^ bPreviousLooping;

		if (bChangedTimes || bChangedLooping)
		{
			UsdUtils::NotifyIfOverriddenOpinion(Attr);

			if (bNowLooping)
			{
				Attr.Set(pxr::UsdMediaTokens->loopFromStartToEnd);
			}
			else
			{
				Attr.Set(pxr::UsdMediaTokens->onceFromStartToEnd);
			}
		}
	}

	// auralmode
	if (pxr::UsdAttribute Attr = SpatialAudio.CreateAuralModeAttr())
	{
		const bool bIsSpatial = AudioSection.GetOverrideAttenuation() && AudioSection.GetAttenuationSettings();
		Attr.Set<pxr::TfToken>(bIsSpatial ? pxr::UsdMediaTokens->spatial : pxr::UsdMediaTokens->nonSpatial);
		UsdUtils::NotifyIfOverriddenOpinion(Attr);
	}

	// gain
	if (pxr::UsdAttribute Attr = SpatialAudio.CreateGainAttr())
	{
		const FMovieSceneFloatChannel& VolumeChannel = AudioSection.GetSoundVolumeChannel();

		// Write out default value
		// Note that even the default is only set on the section and not the component. This because setting the volume
		// on the component itself will do precisely nothing if the Sequencer is usually the source of the audio anyway
		if (TOptional<float> Default = VolumeChannel.GetDefault())
		{
			Attr.Set(static_cast<double>(Default.GetValue()));
		}

		UE::USDPrimConversion::Private::PrepareAttributeForAnimationBake(Attr);

		// Write out timeSamples
		TFunction<void(float, double)> BakerFunc = [Attr](float UEValue, double UsdTimeCode) mutable
		{
			Attr.Set(static_cast<double>(UEValue), UsdTimeCode);
		};
		UE::FUsdPrim WrappedPrim{Prim};
		UnrealToUsd::ConvertFloatChannel(VolumeChannel, *MovieScene, SequenceTransform, BakerFunc, WrappedPrim);
	}

	return true;
}

bool UnrealToUsd::ConvertBoolTrack(
	const UMovieScenePropertyTrack& MovieSceneTrack,
	const FMovieSceneSequenceTransform& SequenceTransform,
	const TFunction<void(bool, double)>& WriterFunc,
	UE::FUsdPrim& Prim
)
{
	if (!WriterFunc || !Prim)
	{
		return false;
	}

	UMovieScene* MovieScene = MovieSceneTrack.GetTypedOuter<UMovieScene>();
	if (!MovieScene)
	{
		return false;
	}

	UE::FUsdStage Stage = Prim.GetStage();

	const TRange<FFrameNumber> PlaybackRange = MovieScene->GetPlaybackRange();
	const FFrameRate Resolution = MovieScene->GetTickResolution();
	const FFrameRate DisplayRate = MovieScene->GetDisplayRate();

	const double StageTimeCodesPerSecond = Stage.GetTimeCodesPerSecond();
	const FFrameRate StageFrameRate(StageTimeCodesPerSecond, 1);

	FMovieSceneInverseSequenceTransform SequenceToRootTransform = SequenceTransform.Inverse();

	auto EvaluateChannel = [&Resolution,
							&DisplayRate](const FMovieSceneBoolChannel& Channel, bool InDefaultValue) -> TArray<TPair<FFrameNumber, bool>>
	{
		TArray<TPair<FFrameNumber, bool>> Values;

		bool DefaultValue = Channel.GetDefault().Get(InDefaultValue);

		TArrayView<const FFrameNumber> KeyTimes = Channel.GetTimes();
		TArrayView<const bool> KeyValues = Channel.GetValues();

		for (int32 KeyIndex = 0; KeyIndex < KeyTimes.Num(); ++KeyIndex)
		{
			const FFrameNumber KeyTime = KeyTimes[KeyIndex];
			bool KeyValue = KeyValues[KeyIndex];

			FFrameTime SnappedKeyTime{FFrameRate::Snap(KeyTime, Resolution, DisplayRate).FloorToFrame()};

			// We never need to bake bool tracks
			Values.Emplace(SnappedKeyTime.GetFrame(), KeyValue);
		}

		return Values;
	};

	for (UMovieSceneSection* Section : MovieSceneTrack.GetAllSections())
	{
		if (UMovieSceneBoolSection* BoolSection = Cast<UMovieSceneBoolSection>(Section))
		{
			for (const TPair<FFrameNumber, bool>& Pair : EvaluateChannel(BoolSection->GetChannel(), false))
			{
				TOptional<FFrameTime> RootTime = SequenceToRootTransform.TryTransformTime(Pair.Key);
				if (RootTime)
				{
					// @todo: Resolution here is actually the local tick res - this is incorrect and should be converted to use the root resolution
					FFrameTime UsdFrameTime = FFrameRate::TransformTime(RootTime.GetValue(), Resolution, StageFrameRate);
					bool UEValue = Pair.Value;

					WriterFunc(UEValue, UsdFrameTime.AsDecimal());
				}
			}
		}
	}

	return true;
}

bool UnrealToUsd::ConvertFloatChannel(
	const FMovieSceneFloatChannel& MovieSceneChannel,
	const UMovieScene& MovieSceneOuter,
	const FMovieSceneSequenceTransform& SequenceTransform,
	const TFunction<void(float, double)>& WriterFunc,
	UE::FUsdPrim& Prim
)
{
	if (!WriterFunc || !Prim)
	{
		return false;
	}

	UE::FUsdStage Stage = Prim.GetStage();

	ERichCurveInterpMode StageInterpMode;
	{
		FScopedUsdAllocs Allocs;
		pxr::UsdStageRefPtr UsdStage{Stage};
		StageInterpMode = (UsdStage->GetInterpolationType() == pxr::UsdInterpolationTypeLinear) ? ERichCurveInterpMode::RCIM_Linear
																								: ERichCurveInterpMode::RCIM_Constant;
	}

	const TRange<FFrameNumber> PlaybackRange = MovieSceneOuter.GetPlaybackRange();
	const FFrameRate Resolution = MovieSceneOuter.GetTickResolution();
	const FFrameRate DisplayRate = MovieSceneOuter.GetDisplayRate();

	const double StageTimeCodesPerSecond = Stage.GetTimeCodesPerSecond();
	const FFrameRate StageFrameRate(StageTimeCodesPerSecond, 1);

	auto EvaluateChannel = [&Resolution,
							&DisplayRate,
							StageInterpMode](const FMovieSceneFloatChannel& Channel, float InDefaultValue) -> TArray<TPair<FFrameNumber, float>>
	{
		TArray<TPair<FFrameNumber, float>> Values;

		const FFrameTime BakeInterval = FFrameRate::TransformTime(1, DisplayRate, Resolution);

		float DefaultValue = Channel.GetDefault().Get(InDefaultValue);

		TMovieSceneChannelData<const FMovieSceneFloatValue> ChannelData = Channel.GetData();
		TArrayView<const FFrameNumber> KeyTimes = ChannelData.GetTimes();
		TArrayView<const FMovieSceneFloatValue> KeyValues = ChannelData.GetValues();

		for (int32 KeyIndex = 0; KeyIndex < KeyTimes.Num(); ++KeyIndex)
		{
			const FFrameNumber KeyTime = KeyTimes[KeyIndex];
			const FMovieSceneFloatValue& KeyValue = KeyValues[KeyIndex];

			// If the channel has the same interpolation type as the stage (or we're the last key),
			// we don't need to bake anything: Just write out the keyframe as is
			if (KeyValue.InterpMode == StageInterpMode || KeyIndex == (KeyTimes.Num() - 1))
			{
				FFrameTime SnappedKeyTime(FFrameRate::Snap(KeyTime, Resolution, DisplayRate).FloorToFrame());
				Values.Emplace(SnappedKeyTime.GetFrame(), KeyValue.Value);
			}
			// We need to bake: Start from this key up until the next key (non-inclusive). We always want to put a keyframe at
			// KeyTime, but then snap the other ones to the stage framerate
			else
			{
				// Don't use the snapped key time for the end of the bake range, because if the snapping moves it
				// later we may end up stepping back again when it's time to bake from that key onwards
				const FFrameNumber NextKey = KeyTimes[KeyIndex + 1];
				const FFrameTime NextKeyTime{NextKey};

				for (FFrameTime EvalTime = KeyTime; EvalTime < NextKeyTime; EvalTime += BakeInterval)
				{
					FFrameNumber BakedKeyTime = FFrameRate::Snap(EvalTime, Resolution, DisplayRate).FloorToFrame();

					float Value = DefaultValue;
					Channel.Evaluate(BakedKeyTime, Value);

					Values.Emplace(BakedKeyTime, Value);
				}
			}
		}

		return Values;
	};

	FMovieSceneInverseSequenceTransform SequenceToRootTransform = SequenceTransform.Inverse();

	for (const TPair<FFrameNumber, float>& Pair : EvaluateChannel(MovieSceneChannel, 0.0f))
	{
		TOptional<FFrameTime> RootTime = SequenceToRootTransform.TryTransformTime(Pair.Key);
		if (RootTime)
		{
			// @todo: Resolution here is actually the local tick res - this is incorrect and should be converted to use the root resolution
			FFrameTime UsdFrameTime = FFrameRate::TransformTime(RootTime.GetValue(), Resolution, StageFrameRate);
			float UEValue = Pair.Value;

			WriterFunc(UEValue, UsdFrameTime.AsDecimal());
		}
	}
	return true;
}

bool UnrealToUsd::ConvertFloatTrack(
	const UMovieSceneFloatTrack& MovieSceneTrack,
	const FMovieSceneSequenceTransform& SequenceTransform,
	const TFunction<void(float, double)>& WriterFunc,
	UE::FUsdPrim& Prim
)
{
	if (UMovieScene* MovieScene = MovieSceneTrack.GetTypedOuter<UMovieScene>())
	{
		for (UMovieSceneSection* Section : MovieSceneTrack.GetAllSections())
		{
			if (UMovieSceneFloatSection* FloatSection = Cast<UMovieSceneFloatSection>(Section))
			{
				return ConvertFloatChannel(FloatSection->GetChannel(), *MovieScene, SequenceTransform, WriterFunc, Prim);
			}
		}
	}

	return false;
}

bool UnrealToUsd::ConvertColorTrack(
	const UMovieSceneColorTrack& MovieSceneTrack,
	const FMovieSceneSequenceTransform& SequenceTransform,
	const TFunction<void(const FLinearColor&, double)>& WriterFunc,
	UE::FUsdPrim& Prim
)
{
	if (!WriterFunc || !Prim)
	{
		return false;
	}

	UMovieScene* MovieScene = MovieSceneTrack.GetTypedOuter<UMovieScene>();
	if (!MovieScene)
	{
		return false;
	}

	UE::FUsdStage Stage = Prim.GetStage();

	ERichCurveInterpMode StageInterpMode;
	{
		FScopedUsdAllocs Allocs;
		pxr::UsdStageRefPtr UsdStage{Stage};
		StageInterpMode = (UsdStage->GetInterpolationType() == pxr::UsdInterpolationTypeLinear) ? ERichCurveInterpMode::RCIM_Linear
																								: ERichCurveInterpMode::RCIM_Constant;
	}

	const TRange<FFrameNumber> PlaybackRange = MovieScene->GetPlaybackRange();
	const FFrameRate Resolution = MovieScene->GetTickResolution();
	const FFrameRate DisplayRate = MovieScene->GetDisplayRate();

	const double StageTimeCodesPerSecond = Stage.GetTimeCodesPerSecond();
	const FFrameRate StageFrameRate(StageTimeCodesPerSecond, 1);

	auto AppendChannelBakeTimes =
		[&Resolution, &DisplayRate, StageInterpMode](const FMovieSceneFloatChannel& Channel, TSet<FFrameNumber>& OutBakeTimes)
	{
		const FFrameTime BakeInterval = FFrameRate::TransformTime(1, DisplayRate, Resolution);

		TMovieSceneChannelData<const FMovieSceneFloatValue> ChannelData = Channel.GetData();
		TArrayView<const FFrameNumber> KeyTimes = ChannelData.GetTimes();
		TArrayView<const FMovieSceneFloatValue> KeyValues = ChannelData.GetValues();

		for (int32 KeyIndex = 0; KeyIndex < KeyTimes.Num(); ++KeyIndex)
		{
			const FFrameNumber KeyTime = KeyTimes[KeyIndex];
			const FMovieSceneFloatValue& KeyValue = KeyValues[KeyIndex];

			// If the channel has the same interpolation type as the stage (or we're the last key),
			// we don't need to bake anything: Just write out the keyframe as is
			if (KeyValue.InterpMode == StageInterpMode || KeyIndex == (KeyTimes.Num() - 1))
			{
				FFrameNumber SnappedKeyTime = FFrameRate::Snap(KeyTime, Resolution, DisplayRate).FloorToFrame();
				OutBakeTimes.Emplace(SnappedKeyTime);
			}
			// We need to bake: Start from this key up until the next key (non-inclusive). We always want to put a keyframe at
			// KeyTime, but then snap the other ones to the stage framerate
			else
			{
				// Don't use the snapped key time for the end of the bake range, because if the snapping moves it
				// later we may end up stepping back again when it's time to bake from that key onwards
				const FFrameNumber NextKey = KeyTimes[KeyIndex + 1];
				const FFrameTime NextKeyTime{NextKey};

				for (FFrameTime EvalTime = KeyTime; EvalTime < NextKeyTime; EvalTime += BakeInterval)
				{
					FFrameNumber BakedKeyTime = FFrameRate::Snap(EvalTime, Resolution, DisplayRate).FloorToFrame();
					OutBakeTimes.Emplace(BakedKeyTime);
				}
			}
		}
	};

	FMovieSceneInverseSequenceTransform SequenceToRootTransform = SequenceTransform.Inverse();

	for (UMovieSceneSection* Section : MovieSceneTrack.GetAllSections())
	{
		if (UMovieSceneColorSection* ColorSection = Cast<UMovieSceneColorSection>(Section))
		{
			const FMovieSceneFloatChannel& RedChannel = ColorSection->GetRedChannel();
			const FMovieSceneFloatChannel& GreenChannel = ColorSection->GetGreenChannel();
			const FMovieSceneFloatChannel& BlueChannel = ColorSection->GetBlueChannel();
			const FMovieSceneFloatChannel& AlphaChannel = ColorSection->GetAlphaChannel();

			// Get the baked FFrameNumbers for each channel (without evaluating the channels yet),
			// because they may have independent keys
			TSet<FFrameNumber> ChannelBakeTimes;
			AppendChannelBakeTimes(RedChannel, ChannelBakeTimes);
			AppendChannelBakeTimes(GreenChannel, ChannelBakeTimes);
			AppendChannelBakeTimes(BlueChannel, ChannelBakeTimes);
			AppendChannelBakeTimes(AlphaChannel, ChannelBakeTimes);

			TArray<FFrameNumber> BakeTimeUnion = ChannelBakeTimes.Array();
			BakeTimeUnion.Sort();

			// Sample all channels at the union of bake times, construct the value and write it out
			for (const FFrameNumber UntransformedBakeTime : BakeTimeUnion)
			{
				TOptional<FFrameTime> RootTime = SequenceToRootTransform.TryTransformTime(UntransformedBakeTime);
				if (!RootTime)
				{
					continue;
				}

				float RedValue = 0.0f;
				float GreenValue = 0.0f;
				float BlueValue = 0.0f;
				float AlphaValue = 1.0f;

				RedChannel.Evaluate(UntransformedBakeTime, RedValue);
				GreenChannel.Evaluate(UntransformedBakeTime, GreenValue);
				BlueChannel.Evaluate(UntransformedBakeTime, BlueValue);
				AlphaChannel.Evaluate(UntransformedBakeTime, AlphaValue);

				FLinearColor Color{RedValue, GreenValue, BlueValue, AlphaValue};

				FFrameTime UsdFrameTime = FFrameRate::TransformTime(RootTime.GetValue(), Resolution, StageFrameRate);

				WriterFunc(Color, UsdFrameTime.AsDecimal());
			}
		}
	}

	return true;
}

bool UnrealToUsd::ConvertBoundsVectorTracks(
	const UMovieSceneDoubleVectorTrack* MinTrack,
	const UMovieSceneDoubleVectorTrack* MaxTrack,
	const FMovieSceneSequenceTransform& SequenceTransform,
	const TFunction<void(const FVector&, const FVector&, double)>& WriterFunc,
	UE::FUsdPrim& Prim
)
{
	if (!WriterFunc || !Prim)
	{
		return false;
	}

	if (!MinTrack && !MaxTrack)
	{
		return false;
	}

	const UMovieSceneDoubleVectorTrack* MainTrack = MinTrack ? MinTrack : MaxTrack;

	UMovieScene* MovieScene = MainTrack->GetTypedOuter<UMovieScene>();
	if (!MovieScene)
	{
		return false;
	}

	UE::FUsdStage Stage = Prim.GetStage();

	ERichCurveInterpMode StageInterpMode;
	{
		FScopedUsdAllocs Allocs;
		pxr::UsdStageRefPtr UsdStage{Stage};
		StageInterpMode = (UsdStage->GetInterpolationType() == pxr::UsdInterpolationTypeLinear) ? ERichCurveInterpMode::RCIM_Linear
																								: ERichCurveInterpMode::RCIM_Constant;
	}

	const FFrameRate Resolution = MovieScene->GetTickResolution();
	const FFrameRate DisplayRate = MovieScene->GetDisplayRate();

	const double StageTimeCodesPerSecond = Stage.GetTimeCodesPerSecond();
	const FFrameRate StageFrameRate(StageTimeCodesPerSecond, 1);

	TFunction<void(const FMovieSceneDoubleChannel&, TSet<FFrameNumber>&)> AppendChannelBakeTimes =
		[&Resolution, &DisplayRate, StageInterpMode](const FMovieSceneDoubleChannel& Channel, TSet<FFrameNumber>& OutBakeTimes)
	{
		const FFrameTime BakeInterval = FFrameRate::TransformTime(1, DisplayRate, Resolution);

		TMovieSceneChannelData<const FMovieSceneDoubleValue> ChannelData = Channel.GetData();
		TArrayView<const FFrameNumber> KeyTimes = ChannelData.GetTimes();
		TArrayView<const FMovieSceneDoubleValue> KeyValues = ChannelData.GetValues();

		for (int32 KeyIndex = 0; KeyIndex < KeyTimes.Num(); ++KeyIndex)
		{
			const FFrameNumber KeyTime = KeyTimes[KeyIndex];
			const FMovieSceneDoubleValue& KeyValue = KeyValues[KeyIndex];

			// If the channel has the same interpolation type as the stage (or we're the last key),
			// we don't need to bake anything: Just write out the keyframe as is
			if (KeyValue.InterpMode == StageInterpMode || KeyIndex == (KeyTimes.Num() - 1))
			{
				FFrameNumber SnappedKeyTime = FFrameRate::Snap(KeyTime, Resolution, DisplayRate).FloorToFrame();
				OutBakeTimes.Emplace(SnappedKeyTime);
			}
			// We need to bake: Start from this key up until the next key (non-inclusive). We always want to put a keyframe at
			// KeyTime, but then snap the other ones to the stage framerate
			else
			{
				// Don't use the snapped key time for the end of the bake range, because if the snapping moves it
				// later we may end up stepping back again when it's time to bake from that key onwards
				const FFrameNumber NextKey = KeyTimes[KeyIndex + 1];
				const FFrameTime NextKeyTime{NextKey};

				for (FFrameTime EvalTime = KeyTime; EvalTime < NextKeyTime; EvalTime += BakeInterval)
				{
					FFrameNumber BakedKeyTime = FFrameRate::Snap(EvalTime, Resolution, DisplayRate).FloorToFrame();
					OutBakeTimes.Emplace(BakedKeyTime);
				}
			}
		}
	};

	TArray<UMovieSceneSection*> MinSections = MinTrack ? MinTrack->GetAllSections() : TArray<UMovieSceneSection*>{};
	TArray<UMovieSceneSection*> MaxSections = MaxTrack ? MaxTrack->GetAllSections() : TArray<UMovieSceneSection*>{};

	// Collect all time samples to bake with
	TSet<FFrameNumber> AllBakeTimes;
	for (UMovieSceneSection* Section : MinSections)
	{
		if (UMovieSceneDoubleVectorSection* VectorSection = Cast<UMovieSceneDoubleVectorSection>(Section))
		{
			for (int32 ChannelIndex = 0; ChannelIndex < VectorSection->GetChannelsUsed(); ++ChannelIndex)
			{
				const FMovieSceneDoubleChannel& Channel = VectorSection->GetChannel(ChannelIndex);
				AppendChannelBakeTimes(Channel, AllBakeTimes);
			}
		}
	}
	for (UMovieSceneSection* Section : MaxSections)
	{
		if (UMovieSceneDoubleVectorSection* VectorSection = Cast<UMovieSceneDoubleVectorSection>(Section))
		{
			for (int32 ChannelIndex = 0; ChannelIndex < VectorSection->GetChannelsUsed(); ++ChannelIndex)
			{
				const FMovieSceneDoubleChannel& Channel = VectorSection->GetChannel(ChannelIndex);
				AppendChannelBakeTimes(Channel, AllBakeTimes);
			}
		}
	}

	TArray<FFrameNumber> BakeTimeUnion = AllBakeTimes.Array();
	BakeTimeUnion.Sort();

	FMovieSceneInverseSequenceTransform SequenceToRootTransform = SequenceTransform.Inverse();

	// Sample all channels at the union of bake times, construct the value and write it out
	// This could be done more efficently, but in the general case we're only going to have one
	// section per track anyway so it shouldn't matter much
	for (const FFrameNumber UntransformedBakeTime : BakeTimeUnion)
	{
		TOptional<FFrameTime> RootTime = SequenceToRootTransform.TryTransformTime(UntransformedBakeTime);
		if (!RootTime)
		{
			continue;
		}

		FVector MinValue{0};
		for (const UMovieSceneSection* Section : MinSections)
		{
			const UMovieSceneDoubleVectorSection* VectorSection = Cast<UMovieSceneDoubleVectorSection>(Section);
			if (!VectorSection->IsTimeWithinSection(UntransformedBakeTime))
			{
				continue;
			}

			for (int32 ChannelIndex = 0; ChannelIndex < VectorSection->GetChannelsUsed(); ++ChannelIndex)
			{
				const FMovieSceneDoubleChannel& Channel = VectorSection->GetChannel(ChannelIndex);
				Channel.Evaluate(UntransformedBakeTime, MinValue[ChannelIndex]);
			}
		}

		FVector MaxValue{0};
		for (const UMovieSceneSection* Section : MaxSections)
		{
			const UMovieSceneDoubleVectorSection* VectorSection = Cast<UMovieSceneDoubleVectorSection>(Section);
			if (!VectorSection->IsTimeWithinSection(UntransformedBakeTime))
			{
				continue;
			}

			for (int32 ChannelIndex = 0; ChannelIndex < VectorSection->GetChannelsUsed(); ++ChannelIndex)
			{
				const FMovieSceneDoubleChannel& Channel = VectorSection->GetChannel(ChannelIndex);
				Channel.Evaluate(UntransformedBakeTime, MaxValue[ChannelIndex]);
			}
		}

		FFrameTime UsdFrameTime = FFrameRate::TransformTime(RootTime.GetValue(), Resolution, StageFrameRate);

		WriterFunc(MinValue, MaxValue, UsdFrameTime.AsDecimal());
	}

	return true;
}

bool UnrealToUsd::Convert3DTransformTrack(
	const UMovieScene3DTransformTrack& MovieSceneTrack,
	const FMovieSceneSequenceTransform& SequenceTransform,
	const TFunction<void(const FTransform&, double)>& WriterFunc,
	UE::FUsdPrim& Prim
)
{
	if (!WriterFunc || !Prim)
	{
		return false;
	}

	UMovieScene* MovieScene = MovieSceneTrack.GetTypedOuter<UMovieScene>();
	if (!MovieScene)
	{
		return false;
	}

	UE::FUsdStage Stage = Prim.GetStage();

	ERichCurveInterpMode StageInterpMode;
	{
		FScopedUsdAllocs Allocs;
		pxr::UsdStageRefPtr UsdStage{Stage};
		StageInterpMode = (UsdStage->GetInterpolationType() == pxr::UsdInterpolationTypeLinear) ? ERichCurveInterpMode::RCIM_Linear
																								: ERichCurveInterpMode::RCIM_Constant;
	}

	const TRange<FFrameNumber> PlaybackRange = MovieScene->GetPlaybackRange();
	const FFrameRate Resolution = MovieScene->GetTickResolution();
	const FFrameRate DisplayRate = MovieScene->GetDisplayRate();

	const double StageTimeCodesPerSecond = Stage.GetTimeCodesPerSecond();
	const FFrameRate StageFrameRate(StageTimeCodesPerSecond, 1);

	auto EvaluateChannelTimes = [&Resolution, &DisplayRate, StageInterpMode](const FMovieSceneDoubleChannel* Channel) -> TSet<FFrameNumber>
	{
		TSet<FFrameNumber> BakeTimes;

		if (!Channel)
		{
			return BakeTimes;
		}

		const FFrameTime BakeInterval = FFrameRate::TransformTime(1, DisplayRate, Resolution);

		TMovieSceneChannelData<const FMovieSceneDoubleValue> ChannelData = Channel->GetData();
		TArrayView<const FFrameNumber> KeyTimes = ChannelData.GetTimes();
		TArrayView<const FMovieSceneDoubleValue> KeyValues = ChannelData.GetValues();

		for (int32 KeyIndex = 0; KeyIndex < KeyTimes.Num(); ++KeyIndex)
		{
			const FFrameNumber KeyTime = KeyTimes[KeyIndex];
			const FMovieSceneDoubleValue& KeyValue = KeyValues[KeyIndex];

			// If the channel has the same interpolation type as the stage (or we're the last key),
			// we don't need to bake anything: Just write out the keyframe as is
			if (KeyValue.InterpMode == StageInterpMode || KeyIndex == (KeyTimes.Num() - 1))
			{
				FFrameNumber SnappedKeyTime = FFrameRate::Snap(KeyTime, Resolution, DisplayRate).FloorToFrame();
				BakeTimes.Emplace(SnappedKeyTime);
			}
			// We need to bake: Start from this key up until the next key (non-inclusive). We always want to put a keyframe at
			// KeyTime, but then snap the other ones to the stage framerate
			else
			{
				// Don't use the snapped key time for the end of the bake range, because if the snapping moves it
				// later we may end up stepping back again when it's time to bake from that key onwards
				const FFrameNumber NextKey = KeyTimes[KeyIndex + 1];
				const FFrameTime NextKeyTime{NextKey};

				for (FFrameTime EvalTime = KeyTime; EvalTime < NextKeyTime; EvalTime += BakeInterval)
				{
					FFrameNumber BakedKeyTime = FFrameRate::Snap(EvalTime, Resolution, DisplayRate).FloorToFrame();
					BakeTimes.Emplace(BakedKeyTime);
				}
			}
		}

		return BakeTimes;
	};

	FMovieSceneInverseSequenceTransform SequenceToRootTransform = SequenceTransform.Inverse();

	for (UMovieSceneSection* Section : MovieSceneTrack.GetAllSections())
	{
		if (UMovieScene3DTransformSection* TransformSection = Cast<UMovieScene3DTransformSection>(Section))
		{
			TArrayView<FMovieSceneDoubleChannel*> Channels = TransformSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();
			if (Channels.Num() < 9)
			{
				USD_LOG_ERROR(
					TEXT("Unexpected number of double tracks (%d) in transform section '%s'"),
					Channels.Num(),
					*TransformSection->GetPathName()
				);
				continue;
			}

			FMovieSceneDoubleChannel* LocationXChannel = Channels[0];
			FMovieSceneDoubleChannel* LocationYChannel = Channels[1];
			FMovieSceneDoubleChannel* LocationZChannel = Channels[2];

			FMovieSceneDoubleChannel* RotationXChannel = Channels[3];
			FMovieSceneDoubleChannel* RotationYChannel = Channels[4];
			FMovieSceneDoubleChannel* RotationZChannel = Channels[5];

			FMovieSceneDoubleChannel* ScaleXChannel = Channels[6];
			FMovieSceneDoubleChannel* ScaleYChannel = Channels[7];
			FMovieSceneDoubleChannel* ScaleZChannel = Channels[8];

			TSet<FFrameNumber> LocationValuesX = EvaluateChannelTimes(LocationXChannel);
			TSet<FFrameNumber> LocationValuesY = EvaluateChannelTimes(LocationYChannel);
			TSet<FFrameNumber> LocationValuesZ = EvaluateChannelTimes(LocationZChannel);

			TSet<FFrameNumber> RotationValuesX = EvaluateChannelTimes(RotationXChannel);
			TSet<FFrameNumber> RotationValuesY = EvaluateChannelTimes(RotationYChannel);
			TSet<FFrameNumber> RotationValuesZ = EvaluateChannelTimes(RotationZChannel);

			TSet<FFrameNumber> ScaleValuesX = EvaluateChannelTimes(ScaleXChannel);
			TSet<FFrameNumber> ScaleValuesY = EvaluateChannelTimes(ScaleYChannel);
			TSet<FFrameNumber> ScaleValuesZ = EvaluateChannelTimes(ScaleZChannel);

			LocationValuesX.Append(LocationValuesY);
			LocationValuesX.Append(LocationValuesZ);

			LocationValuesX.Append(RotationValuesX);
			LocationValuesX.Append(RotationValuesY);
			LocationValuesX.Append(RotationValuesZ);

			LocationValuesX.Append(ScaleValuesX);
			LocationValuesX.Append(ScaleValuesY);
			LocationValuesX.Append(ScaleValuesZ);

			TArray<FFrameNumber> BakeTimeUnion = LocationValuesX.Array();
			BakeTimeUnion.Sort();

			// Sample all channels at the union of bake times, construct the value and write it out
			for (const FFrameNumber UntransformedBakeTime : BakeTimeUnion)
			{
				TOptional<FFrameTime> RootTime = SequenceToRootTransform.TryTransformTime(UntransformedBakeTime);
				if (!RootTime)
				{
					continue;
				}

				double LocX = 0.0f;
				double LocY = 0.0f;
				double LocZ = 0.0f;

				double RotX = 0.0f;
				double RotY = 0.0f;
				double RotZ = 0.0f;

				double ScaleX = 1.0f;
				double ScaleY = 1.0f;
				double ScaleZ = 1.0f;

				if (LocationXChannel)
				{
					LocationXChannel->Evaluate(UntransformedBakeTime, LocX);
				}
				if (LocationYChannel)
				{
					LocationYChannel->Evaluate(UntransformedBakeTime, LocY);
				}
				if (LocationZChannel)
				{
					LocationZChannel->Evaluate(UntransformedBakeTime, LocZ);
				}

				if (RotationXChannel)
				{
					RotationXChannel->Evaluate(UntransformedBakeTime, RotX);
				}
				if (RotationYChannel)
				{
					RotationYChannel->Evaluate(UntransformedBakeTime, RotY);
				}
				if (RotationZChannel)
				{
					RotationZChannel->Evaluate(UntransformedBakeTime, RotZ);
				}

				if (ScaleXChannel)
				{
					ScaleXChannel->Evaluate(UntransformedBakeTime, ScaleX);
				}
				if (ScaleYChannel)
				{
					ScaleYChannel->Evaluate(UntransformedBakeTime, ScaleY);
				}
				if (ScaleZChannel)
				{
					ScaleZChannel->Evaluate(UntransformedBakeTime, ScaleZ);
				}

				// Casting this to float right now because depending on the build and the LWC status FVectors contain FLargeWorldCoordinatesReal,
				// which can be floats and turn these into narrowing conversions, which require explicit casts.
				// TODO: Replace these casts with the underlying FVector type later
				FVector Location{(float)LocX, (float)LocY, (float)LocZ};
				FRotator Rotation{(float)RotY, (float)RotZ, (float)RotX};
				FVector Scale{(float)ScaleX, (float)ScaleY, (float)ScaleZ};
				FTransform Transform{Rotation, Location, Scale};

				FFrameTime UsdFrameTime = FFrameRate::TransformTime(RootTime.GetValue(), Resolution, StageFrameRate);

				WriterFunc(Transform, UsdFrameTime.AsDecimal());
			}
		}
	}

	return true;
}

bool UnrealToUsd::ConvertSceneComponent(const pxr::UsdStageRefPtr& Stage, const USceneComponent* SceneComponent, pxr::UsdPrim& UsdPrim)
{
	if (!UsdPrim || !SceneComponent)
	{
		return false;
	}

	FScopedUsdAllocs UsdAllocs;

	// Transform
	pxr::UsdGeomXformable XForm(UsdPrim);
	if (!XForm)
	{
		return false;
	}

	// If we're attached to a socket our RelativeTransform will be relative to the socket, instead of the parent
	// component space. If we were to use GetRelativeTransform directly, we're in charge of managing the socket
	// transform too (and any other N obscure features we don't know about/don't exist yet). If we fetch directly
	// the component-to-world transform however, the component will do that on its own (as that is the transform
	// that is actually used to show it on the level), so we don't have to worry about it!
	FTransform RelativeTransform;
	if (USceneComponent* Parent = SceneComponent->GetAttachParent())
	{
		Parent->ConditionalUpdateComponentToWorld();
		Parent->UpdateChildTransforms();
		RelativeTransform = SceneComponent->GetComponentTransform().GetRelativeTransform(Parent->GetComponentTransform());
	}
	else
	{
		RelativeTransform = SceneComponent->GetRelativeTransform();
	}

	// Compensate different orientation for light or camera components:
	// In USD cameras shoot towards local - Z, with + Y up. Lights also emit towards local - Z, with + Y up
	// In UE cameras shoot towards local + X, with + Z up. Lights also emit towards local + X, with + Z up
	// Note that this wouldn't have worked in case we collapsed light and camera components, but these always get their own
	// actors, so we know that we don't have a single component that represents a large collapsed prim hierarchy
	if (UsdPrim.IsA<pxr::UsdGeomCamera>() || UsdPrim.HasAPI<pxr::UsdLuxLightAPI>())
	{
		FTransform AdditionalRotation = FTransform(FRotator(0.0f, 90.f, 0.0f));

		if (UsdUtils::GetUsdStageUpAxis(Stage) == pxr::UsdGeomTokens->z)
		{
			AdditionalRotation *= FTransform(FRotator(90.0f, 0.f, 0.0f));
		}

		RelativeTransform = AdditionalRotation * RelativeTransform;
	}

	// Invert compensation applied to parent if it's a light or camera component
	if (pxr::UsdPrim ParentPrim = UsdPrim.GetParent())
	{
		if (ParentPrim.IsA<pxr::UsdGeomCamera>() || ParentPrim.HasAPI<pxr::UsdLuxLightAPI>())
		{
			FTransform AdditionalRotation = FTransform(FRotator(0.0f, 90.f, 0.0f));

			if (UsdUtils::GetUsdStageUpAxis(Stage) == pxr::UsdGeomTokens->z)
			{
				AdditionalRotation *= FTransform(FRotator(90.0f, 0.f, 0.0f));
			}

			RelativeTransform = RelativeTransform * AdditionalRotation.Inverse();
		}
	}

	// Transform
	ConvertXformable(RelativeTransform, UsdPrim, UsdUtils::GetDefaultTimeCode());

	// Per-prim visibility
	if (pxr::UsdAttribute VisibilityAttr = XForm.CreateVisibilityAttr())
	{
		pxr::TfToken Value = pxr::UsdGeomTokens->inherited;

		if (SceneComponent->ComponentTags.Contains(UnrealIdentifiers::Invisible))
		{
			Value = pxr::UsdGeomTokens->invisible;
		}
		else if (!SceneComponent->ComponentTags.Contains(UnrealIdentifiers::Inherited))
		{
			// We don't have visible nor inherited tags: We're probably exporting a pure UE component, so write out component visibility instead.
			// Ignore invisibility from brush components though because they are always forced to bHiddenInGame=true, with the property even
			// being hidden on the details panel
			Value = SceneComponent->bHiddenInGame && !SceneComponent->IsA<UBrushComponent>() ? pxr::UsdGeomTokens->invisible
																							 : pxr::UsdGeomTokens->inherited;
		}

		VisibilityAttr.Set<pxr::TfToken>(Value);
		UsdUtils::NotifyIfOverriddenOpinion(VisibilityAttr);
	}

	return true;
}

bool UnrealToUsd::ConvertMeshComponent(const pxr::UsdStageRefPtr& Stage, const UMeshComponent* MeshComponent, pxr::UsdPrim& UsdPrim)
{
	if (!UsdPrim || !MeshComponent)
	{
		return false;
	}

	UObject* MeshAsset = nullptr;

	// Handle material overrides
	if (const UGeometryCacheComponent* GeometryCacheComponent = Cast<const UGeometryCacheComponent>(MeshComponent))
	{
		MeshAsset = GeometryCacheComponent->GetGeometryCache();
	}
	else if (const UStaticMeshComponent* StaticMeshComponent = Cast<const UStaticMeshComponent>(MeshComponent))
	{
		MeshAsset = StaticMeshComponent->GetStaticMesh();
	}
	else if (const USkinnedMeshComponent* SkinnedMeshComponent = Cast<const USkinnedMeshComponent>(MeshComponent))
	{
		MeshAsset = SkinnedMeshComponent->GetSkinnedAsset();
	}

	// Component doesn't have any mesh so this function doesn't need to do anything
	if (!MeshAsset)
	{
		return true;
	}

	return UnrealToUsd::ConvertMaterialOverrides(MeshAsset, MeshComponent->OverrideMaterials, UsdPrim);
}

bool UnrealToUsd::ConvertHierarchicalInstancedStaticMeshComponent(
	const UHierarchicalInstancedStaticMeshComponent* HISMComponent,
	pxr::UsdPrim& UsdPrim,
	double TimeCode
)
{
	return ConvertInstancedStaticMeshComponent(HISMComponent, UsdPrim, TimeCode);
}

bool UnrealToUsd::ConvertInstancedStaticMeshComponent(const UInstancedStaticMeshComponent* ISMComponent, pxr::UsdPrim& UsdPrim, double TimeCode)
{
	using namespace pxr;

	FScopedUsdAllocs Allocs;

	UsdGeomPointInstancer PointInstancer{UsdPrim};
	if (!PointInstancer || !ISMComponent)
	{
		return false;
	}

	UsdStageRefPtr Stage = UsdPrim.GetStage();
	FUsdStageInfo StageInfo{Stage};

	VtArray<int> ProtoIndices;
	VtArray<GfVec3f> Positions;
	VtArray<GfQuath> Orientations;
	VtArray<GfVec3f> Scales;

	const int32 NumInstances = ISMComponent->GetInstanceCount();
	ProtoIndices.reserve(ProtoIndices.size() + NumInstances);
	Positions.reserve(Positions.size() + NumInstances);
	Orientations.reserve(Orientations.size() + NumInstances);
	Scales.reserve(Scales.size() + NumInstances);

	for (const FInstancedStaticMeshInstanceData& InstanceData : ISMComponent->PerInstanceSMData)
	{
		// Convert axes
		FTransform UETransform{InstanceData.Transform};
		FTransform USDTransform = UsdUtils::ConvertAxes(StageInfo.UpAxis == EUsdUpAxis::ZAxis, UETransform);

		FVector Translation = USDTransform.GetTranslation();
		FQuat Rotation = USDTransform.GetRotation();
		FVector Scale = USDTransform.GetScale3D();

		// Compensate metersPerUnit
		const double UEMetersPerUnit = 0.01;
		if (!FMath::IsNearlyEqual(UEMetersPerUnit, StageInfo.MetersPerUnit))
		{
			Translation *= (UEMetersPerUnit / StageInfo.MetersPerUnit);
		}

		ProtoIndices.push_back(0);	  // We will always export a single prototype per PointInstancer, since ISM components handle only 1 mesh at a
									  // time
		Positions.push_back(GfVec3f(Translation.X, Translation.Y, Translation.Z));
		Orientations.push_back(GfQuath(Rotation.W, Rotation.X, Rotation.Y, Rotation.Z));
		Scales.push_back(GfVec3f(Scale.X, Scale.Y, Scale.Z));
	}

	const pxr::UsdTimeCode UsdTimeCode(TimeCode);

	if (UsdAttribute Attr = PointInstancer.CreateProtoIndicesAttr())
	{
		Attr.Set(ProtoIndices, UsdTimeCode);
		UsdUtils::NotifyIfOverriddenOpinion(Attr);
	}

	if (UsdAttribute Attr = PointInstancer.CreatePositionsAttr())
	{
		Attr.Set(Positions, UsdTimeCode);
		UsdUtils::NotifyIfOverriddenOpinion(Attr);
	}

	if (UsdAttribute Attr = PointInstancer.CreateOrientationsAttr())
	{
		Attr.Set(Orientations, UsdTimeCode);
		UsdUtils::NotifyIfOverriddenOpinion(Attr);
	}

	if (UsdAttribute Attr = PointInstancer.CreateScalesAttr())
	{
		Attr.Set(Scales, UsdTimeCode);
		UsdUtils::NotifyIfOverriddenOpinion(Attr);
	}

	return true;
}

bool UnrealToUsd::ConvertMaterialOverrides(
	const UObject* MeshAsset,
	const TArray<UMaterialInterface*> MaterialOverrides,
	pxr::UsdPrim& UsdPrim,
	int32 LowestLOD,
	int32 HighestLOD,
	bool bCheckAssetUserData
)
{
#if WITH_EDITOR
	if (!UsdPrim)
	{
		return false;
	}

	FScopedUsdAllocs Allocs;
	pxr::UsdStageRefPtr Stage = UsdPrim.GetStage();
	FString MeshPrimPath = UsdToUnreal::ConvertPath(UsdPrim.GetPrimPath());

	// If we're inside of an instance then we have to account for the fact that with instance-aware translation only one
	// of the prototype's instances will create/fetch the static mesh and record their material slot paths. This means that
	// in order to tell whether a slot prim path from asset user data corresponds to a child of our target mesh prim we
	// need to compare paths within the prototype instead
	const bool bIsInsideInstance = UsdPrim.IsInstanceProxy();
	if (bIsInsideInstance)
	{
		MeshPrimPath = UsdToUnreal::ConvertPath(UsdPrim.GetPrimInPrototype().GetPrimPath());
	}

	const bool bAllLODs = LowestLOD == INDEX_NONE && HighestLOD == INDEX_NONE;

	TFunction<void(const UUsdMeshAssetUserData*, int32, const TSet<FString>&, const FString&)> AuthorOverrideViaUserData =
		[bIsInsideInstance,
		 &Stage,
		 &UsdPrim](
			const UUsdMeshAssetUserData* UserData,	  //
			int32 MatIndex,
			const TSet<FString>& MeshPrimPaths,
			const FString& OverrideMaterialPath
		)
	{
		const FUsdPrimPathList* SourcePrimPaths = UserData->MaterialSlotToPrimPaths.Find(MatIndex);
		if (!SourcePrimPaths)
		{
			return;
		}

		for (FString SourcePrimPath : SourcePrimPaths->PrimPaths)
		{
			if (bIsInsideInstance)
			{
				pxr::UsdPrim PrimAtPath = Stage->GetPrimAtPath(UnrealToUsd::ConvertPath(*SourcePrimPath).Get());
				if (PrimAtPath.IsInstanceProxy())
				{
					SourcePrimPath = UsdToUnreal::ConvertPath(PrimAtPath.GetPrimInPrototype().GetPrimPath());
				}
			}

			// Our mesh assets are shared between multiple prims via the asset cache, and all user prims of that asset will record
			// their source prim paths for each material slot. In here we just want to apply overrides to the prims that
			// correspond to the modified component (i.e. are within its subtree), and not the others.
			for (const FString& MeshPrimPath : MeshPrimPaths)
			{
				if (SourcePrimPath.StartsWith(MeshPrimPath))
				{
					if (bIsInsideInstance)
					{
						// Have to author collection-based bindings on the instance root.
						//
						// SourcePrimPath and MeshPrimPath are both relative to the prototype in this case. If we know they match, we
						// know this source prim path pertains to something inside our instanceable, and so we should author the override

						// Find the instance root
						pxr::UsdPrim InstanceRoot = UsdPrim.GetParent();
						while (InstanceRoot && !InstanceRoot.IsPseudoRoot() && !InstanceRoot.IsInstance())
						{
							InstanceRoot = InstanceRoot.GetParent();
						}
						if (InstanceRoot.IsInstance())
						{
							// Now we just have a bunch of paths relative to the prototype. In order to find the correct instance proxy
							// path to override, we must do some path surgery to move the SourcePrimPath suffix below the instance root
							// prefix, onto the true instance proxy path on the stage.
							//
							// Note that we have to do this (and not just use the original SourcePrimPaths directly) mainly because with
							// instance-aware translation it is now possible that SourcePrimPath doesn't contain the paths for *all*
							// instances of the prototype, and only for the one instance that actually led to it being translated.
							// UsdPrim/InstanceRoot may refer to another instance entirely, so we need our paths to match them instead.

							// e.g. "/root/instanceable7"
							pxr::SdfPath InstanceRootPath = InstanceRoot.GetPrimPath();

							// e.g. "/__Prototype_3/child_prim/slot
							pxr::SdfPath InstanceProxySuffix = UnrealToUsd::ConvertPath(*SourcePrimPath).Get();

							// After the loop PrototypePrefix becomes e.g. "/__Prototype_3"
							pxr::SdfPath PrototypePrefix = InstanceProxySuffix;
							while (true)
							{
								pxr::SdfPath ParentPath = PrototypePrefix.GetParentPath();
								if (ParentPath.IsEmpty() || ParentPath.IsAbsoluteRootPath())
								{
									break;
								}
								PrototypePrefix = ParentPath;
							}

							// e.g. "/root/instanceable7/child_prim/slot"
							pxr::SdfPath InstanceProxyFullPath = InstanceProxySuffix.ReplacePrefix(PrototypePrefix, InstanceRootPath);

							pxr::UsdPrim InstanceRootOver = Stage->OverridePrim(InstanceRoot.GetPrimPath());
							pxr::UsdPrim InstanceProxyPrim = Stage->GetPrimAtPath(InstanceProxyFullPath);
							UsdUtils::AuthorUnrealCollectionBasedMaterialBinding(	 //
								InstanceRootOver,
								InstanceProxyPrim,
								OverrideMaterialPath
							);
						}
					}
					else
					{
						// Here we can just author a regular material binding opinion
						pxr::SdfPath OverridePrimPath = UnrealToUsd::ConvertPath(*SourcePrimPath).Get();
						pxr::UsdPrim MeshPrim = Stage->OverridePrim(OverridePrimPath);
						UsdUtils::AuthorUnrealMaterialBinding(MeshPrim, OverrideMaterialPath);
					}
				}
			}
		}
	};

	if (const UGeometryCache* GeometryCache = Cast<const UGeometryCache>(MeshAsset))
	{
		for (int32 MatIndex = 0; MatIndex < MaterialOverrides.Num(); ++MatIndex)
		{
			UMaterialInterface* Override = MaterialOverrides[MatIndex];
			if (!Override)
			{
				continue;
			}

			const UUsdMeshAssetUserData* UserData = bCheckAssetUserData
				? const_cast<UGeometryCache*>(GeometryCache)->GetAssetUserData<UUsdMeshAssetUserData>()
				: nullptr;

			// If we have user data this is one of our meshes, so we know exactly the prim that corresponds to each
			// material slot. Let's use that.
			//
			// const_cast as there's no const access to asset user data on IInterface_AssetUserData, but we won't
			// modify anything
			if (UserData)
			{
				AuthorOverrideViaUserData(UserData, MatIndex, {MeshPrimPath}, Override->GetPathName());
			}
			// If we don't, we have to fallback to writing the same prim patterns that the mesh exporters
			// generate when exporting meshes, so that we can override its opinions. This happens when exporting
			// geometry cache / static mesh / skeletal mesh components, for example
			else
			{
				pxr::SdfPath OverridePrimPath = UsdPrim.GetPath();
				pxr::UsdPrim MeshPrim = Stage->OverridePrim(OverridePrimPath);
				UsdUtils::AuthorUnrealMaterialBinding(MeshPrim, Override->GetPathName());
			}
		}
	}
	else if (const UStaticMesh* StaticMesh = Cast<const UStaticMesh>(MeshAsset))
	{
		int32 NumLODs = StaticMesh->GetNumLODs();
		if (bAllLODs)
		{
			HighestLOD = NumLODs - 1;
			LowestLOD = 0;
		}
		else
		{
			// Make sure they're both >= 0 (the options dialog slider is clamped, but this may be called directly)
			LowestLOD = FMath::Clamp(LowestLOD, 0, NumLODs - 1);
			HighestLOD = FMath::Clamp(HighestLOD, 0, NumLODs - 1);

			// Make sure Lowest <= Highest
			int32 Temp = FMath::Min(LowestLOD, HighestLOD);
			HighestLOD = FMath::Max(LowestLOD, HighestLOD);
			LowestLOD = Temp;

			// Make sure there's at least one LOD
			NumLODs = FMath::Max(HighestLOD - LowestLOD + 1, 1);
		}
		const bool bHasLODs = NumLODs > 1;

		const UUsdMeshAssetUserData* UserData = bCheckAssetUserData
			? const_cast<UStaticMesh*>(StaticMesh)->GetAssetUserData<UUsdMeshAssetUserData>() 
			: nullptr;

		for (int32 MatIndex = 0; MatIndex < MaterialOverrides.Num(); ++MatIndex)
		{
			UMaterialInterface* Override = MaterialOverrides[MatIndex];
			if (!Override)
			{
				continue;
			}

			for (int32 LODIndex = LowestLOD; LODIndex <= HighestLOD; ++LODIndex)
			{
				int32 NumSections = StaticMesh->GetNumSections(LODIndex);
				const bool bHasSubsets = NumSections > 1;

				for (int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
				{
					int32 SectionMatIndex = StaticMesh->GetSectionInfoMap().Get(LODIndex, SectionIndex).MaterialIndex;
					if (SectionMatIndex != MatIndex)
					{
						continue;
					}

					if (UserData)
					{
						AuthorOverrideViaUserData(UserData, MatIndex, {MeshPrimPath}, Override->GetPathName());
					}
					else
					{
						pxr::SdfPath OverridePrimPath = UsdPrim.GetPath();

						// If we have only 1 LOD, the asset's DefaultPrim will be the Mesh prim directly.
						// If we have multiple, the default prim won't have any schema, but will contain separate
						// Mesh prims for each LOD named "LOD0", "LOD1", etc., switched via a "LOD" variant set
						if (bHasLODs)
						{
							OverridePrimPath = OverridePrimPath.AppendPath(UnrealToUsd::ConvertPath(*FString::Printf(TEXT("LOD%d"), LODIndex)).Get());
						}

						// If our LOD has only one section, its material assignment will be authored directly on the Mesh prim.
						// If it has more than one material slot, we'll author UsdGeomSubset for each LOD Section, and author the material
						// assignment there
						if (bHasSubsets)
						{
							// Assume the UE sections are in the same order as the USD ones
							std::vector<pxr::UsdGeomSubset> GeomSubsets = pxr::UsdShadeMaterialBindingAPI(UsdPrim).GetMaterialBindSubsets();
							if (SectionIndex < static_cast<int32>(GeomSubsets.size()))
							{
								OverridePrimPath = OverridePrimPath.AppendChild(GeomSubsets[SectionIndex].GetPrim().GetName());
							}
							else
							{
								OverridePrimPath = OverridePrimPath.AppendPath(
									UnrealToUsd::ConvertPath(*FString::Printf(TEXT("Section%d"), SectionIndex)).Get()
								);
							}
						}

						pxr::UsdPrim MeshPrim = Stage->OverridePrim(OverridePrimPath);
						UsdUtils::AuthorUnrealMaterialBinding(MeshPrim, Override->GetPathName());
					}
				}
			}
		}
	}
	else if (const USkeletalMesh* SkeletalMesh = Cast<const USkeletalMesh>(MeshAsset))
	{
		FSkeletalMeshRenderData* RenderData = SkeletalMesh->GetResourceForRendering();
		if (!RenderData)
		{
			return false;
		}

		const UUsdMeshAssetUserData* UserData = bCheckAssetUserData
			? const_cast<USkeletalMesh*>(SkeletalMesh)->GetAssetUserData<UUsdMeshAssetUserData>()
			: nullptr;

		TIndirectArray<FSkeletalMeshLODRenderData>& LodRenderData = RenderData->LODRenderData;
		if (LodRenderData.Num() == 0)
		{
			return false;
		}

		int32 NumLODs = SkeletalMesh->GetLODNum();
		if (bAllLODs)
		{
			HighestLOD = NumLODs - 1;
			LowestLOD = 0;
		}
		else
		{
			// Make sure they're both >= 0 (the options dialog slider is clamped, but this may be called directly)
			LowestLOD = FMath::Clamp(LowestLOD, 0, NumLODs - 1);
			HighestLOD = FMath::Clamp(HighestLOD, 0, NumLODs - 1);

			// Make sure Lowest <= Highest
			int32 Temp = FMath::Min(LowestLOD, HighestLOD);
			HighestLOD = FMath::Max(LowestLOD, HighestLOD);
			LowestLOD = Temp;

			// Make sure there's at least one LOD
			NumLODs = FMath::Max(HighestLOD - LowestLOD + 1, 1);
		}
		const bool bHasLODs = NumLODs > 1;

		if (!UsdPrim.IsA<pxr::UsdSkelSkeleton>())
		{
			USD_LOG_WARNING(
				TEXT("For the skeletal case, ConvertMaterialOverrides must now receive a Skeleton prim! ('%s' was provided)"),
				*UsdToUnreal::ConvertPath(UsdPrim.GetPrimPath())
			);
			return false;
		}

		// If performance becomes an issue we can start storing our skel caches in the info cache and optionally provide it
		// to this function
		pxr::UsdSkelRoot SkelRoot = pxr::UsdSkelRoot{UsdUtils::GetClosestParentSkelRoot(UsdPrim)};
		if (!SkelRoot)
		{
			return false;
		}
		pxr::UsdSkelBinding SkelBinding;
		pxr::UsdSkelSkeletonQuery SkeletonQuery;
		if (!UsdUtils::GetSkelQueries(SkelRoot, pxr::UsdSkelSkeleton{UsdPrim}, SkelBinding, SkeletonQuery))
		{
			return false;
		}

		// Collect all skinned prim paths
		const pxr::VtArray<pxr::UsdSkelSkinningQuery>& SkinningTargets = SkelBinding.GetSkinningTargets();
		TSet<FString> SkinnedMeshPaths;
		SkinnedMeshPaths.Reserve(SkinningTargets.size());
		for (const pxr::UsdSkelSkinningQuery& SkinningTarget : SkinningTargets)
		{
			pxr::UsdPrim SkinnedPrim = SkinningTarget.GetPrim();
			if (bIsInsideInstance && SkinnedPrim.IsInstanceProxy())
			{
				SkinnedPrim = SkinnedPrim.GetPrimInPrototype();
			}

			if (pxr::UsdGeomMesh SkinnedMesh = pxr::UsdGeomMesh{SkinnedPrim})
			{
				SkinnedMeshPaths.Add(UsdToUnreal::ConvertPath(SkinnedPrim.GetPrimPath()));
			}
		}

		for (int32 MatIndex = 0; MatIndex < MaterialOverrides.Num(); ++MatIndex)
		{
			UMaterialInterface* Override = MaterialOverrides[MatIndex];
			if (!Override)
			{
				continue;
			}

			for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
			{
				if (!LodRenderData.IsValidIndex(LODIndex))
				{
					continue;
				}

				const FSkeletalMeshLODInfo* LODInfo = SkeletalMesh->GetLODInfo(LODIndex);

				const TArray<FSkelMeshRenderSection>& Sections = LodRenderData[LODIndex].RenderSections;
				int32 NumSections = Sections.Num();
				const bool bHasSubsets = NumSections > 1;

				for (int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
				{
					int32 SectionMatIndex = Sections[SectionIndex].MaterialIndex;

					// If we have a LODInfo map, we need to reroute the material index through it
					if (LODInfo && LODInfo->LODMaterialMap.IsValidIndex(SectionIndex))
					{
						SectionMatIndex = LODInfo->LODMaterialMap[SectionIndex];
					}

					if (SectionMatIndex != MatIndex)
					{
						continue;
					}

					if (UserData)
					{
						AuthorOverrideViaUserData(UserData, MatIndex, SkinnedMeshPaths, Override->GetPathName());
					}
					// TODO: We really need a separate function for ConvertingMaterialOverrides (to an opened stage) and ExportingMaterialOverrides
					// that we can use when the SkeletalMesh is not something we generated ourselves (with annotated MaterialSlotToPrimPaths).
					// We could collect an analogue for MaterialSlotToPrimPaths during the mesh export process to accurately author these overrides
					// too
					else
					{
						pxr::SdfPath OverridePrimPath;

						// If we have only 1 LOD, the asset's DefaultPrim will be a SkelRoot, and the Mesh will be a subprim
						// with the same name. If we have multiple LODS, the default prim is also the SkelRoot but will contain separate
						// Mesh prims for each LOD named "LOD0", "LOD1", etc., switched via a "LOD" variant set
						if (bHasLODs)
						{
							OverridePrimPath = SkelRoot.GetPath().AppendPath(UnrealToUsd::ConvertPath(*FString::Printf(TEXT("LOD%d"), LODIndex)).Get()
							);
						}
						else
						{
							// Here we're guessing that we're converting material overrides for our exported level, which will use our
							// own prims from exported SkeletalMeshes that all just have a single skinned mesh anyway
							FString MeshName;
							if (SkinningTargets.size() > 0)
							{
								if (pxr::UsdPrim FirstSkinnedPrim = SkinningTargets[0].GetPrim())
								{
									MeshName = UsdToUnreal::ConvertString(FirstSkinnedPrim.GetName());
								}
							}

							OverridePrimPath = SkelRoot.GetPath().AppendElementString(UnrealToUsd::ConvertString(*MeshName).Get());
						}

						// If our LOD has only one section, its material assignment will be authored directly on the Mesh prim.
						// If it has more than one material slot, we'll author UsdGeomSubset for each LOD Section, and author the material
						// assignment there
						if (bHasSubsets)
						{
							// Assume the UE sections are in the same order as the USD ones
							std::vector<pxr::UsdGeomSubset> GeomSubsets = pxr::UsdShadeMaterialBindingAPI(UsdPrim).GetMaterialBindSubsets();
							if (SectionIndex < static_cast<int32>(GeomSubsets.size()))
							{
								OverridePrimPath = OverridePrimPath.AppendChild(GeomSubsets[SectionIndex].GetPrim().GetName());
							}
							else
							{
								OverridePrimPath = OverridePrimPath.AppendPath(
									UnrealToUsd::ConvertPath(*FString::Printf(TEXT("Section%d"), SectionIndex)).Get()
								);
							}
						}

						pxr::UsdPrim MeshPrim = Stage->OverridePrim(OverridePrimPath);
						UsdUtils::AuthorUnrealMaterialBinding(MeshPrim, Override->GetPathName());
					}
				}
			}
		}
	}
	else
	{
		ensure(false);
		return false;
	}
#endif	  // WITH_EDITOR

	return true;
}

bool UnrealToUsd::ConvertXformable(const FTransform& RelativeTransform, pxr::UsdPrim& UsdPrim, double TimeCode)
{
	if (!UsdPrim)
	{
		return false;
	}

	FScopedUsdAllocs UsdAllocs;

	// Transform
	pxr::UsdGeomXformable XForm(UsdPrim);
	if (!XForm)
	{
		return false;
	}

	FUsdStageInfo StageInfo(UsdPrim.GetStage());
	pxr::GfMatrix4d UsdTransform = UnrealToUsd::ConvertTransform(StageInfo, RelativeTransform);

	const pxr::UsdTimeCode UsdTimeCode(TimeCode);

	if (pxr::UsdGeomXformOp MatrixXform = UE::USDPrimConversion::Private::ForceMatrixXform(XForm))
	{
		MatrixXform.Set(UsdTransform, UsdTimeCode);

		UsdUtils::NotifyIfOverriddenOpinion(MatrixXform.GetAttr());
		UsdUtils::NotifyIfOverriddenOpinion(XForm.GetXformOpOrderAttr());
	}

	return true;
}

#if WITH_EDITOR
namespace UE::USDPrimConversion::Private
{
	void ConvertFoliageInstances(
		const FFoliageInfo& Info,
		const TSet<int32>& UEInstances,
		const FTransform& UEWorldToFoliageActor,
		const FUsdStageInfo& StageInfo,
		int PrototypeIndex,
		pxr::VtArray<int>& ProtoIndices,
		pxr::VtArray<pxr::GfVec3f>& Positions,
		pxr::VtArray<pxr::GfQuath>& Orientations,
		pxr::VtArray<pxr::GfVec3f>& Scales
	)
	{
		FScopedUsdAllocs Allocs;

		const int32 NumInstances = UEInstances.Num();

		ProtoIndices.reserve(ProtoIndices.size() + NumInstances);
		Positions.reserve(Positions.size() + NumInstances);
		Orientations.reserve(Orientations.size() + NumInstances);
		Scales.reserve(Scales.size() + NumInstances);

		for (int32 InstanceIndex : UEInstances)
		{
			const FFoliageInstancePlacementInfo* Instance = &Info.Instances[InstanceIndex];

			// Convert axes
			FTransform UEWorldTransform{Instance->Rotation, (FVector)Instance->Location, (FVector)Instance->DrawScale3D};
			FTransform USDTransform = UsdUtils::ConvertAxes(StageInfo.UpAxis == EUsdUpAxis::ZAxis, UEWorldTransform * UEWorldToFoliageActor);

			FVector Translation = USDTransform.GetTranslation();
			FQuat Rotation = USDTransform.GetRotation();
			FVector Scale = USDTransform.GetScale3D();

			// Compensate metersPerUnit
			const double UEMetersPerUnit = 0.01;
			if (!FMath::IsNearlyEqual(UEMetersPerUnit, StageInfo.MetersPerUnit))
			{
				Translation *= (UEMetersPerUnit / StageInfo.MetersPerUnit);
			}

			ProtoIndices.push_back(PrototypeIndex);
			Positions.push_back(pxr::GfVec3f(Translation.X, Translation.Y, Translation.Z));
			Orientations.push_back(pxr::GfQuath(Rotation.W, Rotation.X, Rotation.Y, Rotation.Z));
			Scales.push_back(pxr::GfVec3f(Scale.X, Scale.Y, Scale.Z));
		}
	}
}	 // namespace UE::USDPrimConversion::Private
#endif	  // WITH_EDITOR

bool UnrealToUsd::ConvertInstancedFoliageActor(const AInstancedFoliageActor& Actor, pxr::UsdPrim& UsdPrim, double TimeCode, ULevel* InstancesLevel)
{
#if WITH_EDITOR
	using namespace pxr;

	FScopedUsdAllocs Allocs;

	UsdGeomPointInstancer PointInstancer{UsdPrim};
	if (!PointInstancer)
	{
		return false;
	}

	UsdStageRefPtr Stage = UsdPrim.GetStage();
	FUsdStageInfo StageInfo{Stage};

	VtArray<int> ProtoIndices;
	VtArray<GfVec3f> Positions;
	VtArray<GfQuath> Orientations;
	VtArray<GfVec3f> Scales;

	TSet<FFoliageInstanceBaseId> HandledComponents;

	// It seems like the foliage instance transforms are actually world transforms, so to get them into the coordinate space of the generated
	// point instancer, we'll have to concatenate with the inverse the foliage actor's ActorToWorld transform
	FTransform UEWorldToFoliageActor = Actor.GetTransform().Inverse();

	int PrototypeIndex = 0;
	for (const TPair<UFoliageType*, TUniqueObj<FFoliageInfo>>& FoliagePair : Actor.GetFoliageInfos())
	{
		const UFoliageType* FoliageType = FoliagePair.Key;
		const FFoliageInfo& Info = FoliagePair.Value.Get();

		// Traverse valid foliage instances: Those that are being tracked to belonging to a particular component
		for (const TPair<FFoliageInstanceBaseId, FFoliageInstanceBaseInfo>& FoliageInstancePair : Actor.InstanceBaseCache.InstanceBaseMap)
		{
			const FFoliageInstanceBaseId& ComponentId = FoliageInstancePair.Key;
			HandledComponents.Add(ComponentId);

			UActorComponent* Comp = FoliageInstancePair.Value.BasePtr.Get();
			if (!Comp || (InstancesLevel && (Comp->GetComponentLevel() != InstancesLevel)))
			{
				continue;
			}

			if (const TSet<int32>* InstanceSet = Info.ComponentHash.Find(ComponentId))
			{
				UE::USDPrimConversion::Private::ConvertFoliageInstances(
					Info,
					*InstanceSet,
					UEWorldToFoliageActor,
					StageInfo,
					PrototypeIndex,
					ProtoIndices,
					Positions,
					Orientations,
					Scales
				);
			}
		}

		// Do another pass to grab invalid foliage instances (not assigned to any particular component)
		// Only export these when we're not given a particular level to export, or if that level is the actor's level (essentially
		// pretending the invalid instances belong to the actor's level). This mostly helps prevent it from exporting the invalid instances
		// multiple times in case we're calling this function repeatedly for each individual sublevel
		if (!InstancesLevel || InstancesLevel == Actor.GetLevel())
		{
			for (const TPair<FFoliageInstanceBaseId, TSet<int32>>& Pair : Info.ComponentHash)
			{
				const FFoliageInstanceBaseId& ComponentId = Pair.Key;
				if (HandledComponents.Contains(ComponentId))
				{
					continue;
				}

				const TSet<int32>& InstanceSet = Pair.Value;
				UE::USDPrimConversion::Private::ConvertFoliageInstances(
					Info,
					InstanceSet,
					UEWorldToFoliageActor,
					StageInfo,
					PrototypeIndex,
					ProtoIndices,
					Positions,
					Orientations,
					Scales
				);
			}
		}

		++PrototypeIndex;
	}

	const pxr::UsdTimeCode UsdTimeCode(TimeCode);

	if (UsdAttribute Attr = PointInstancer.CreateProtoIndicesAttr())
	{
		Attr.Set(ProtoIndices, UsdTimeCode);
		UsdUtils::NotifyIfOverriddenOpinion(Attr);
	}

	if (UsdAttribute Attr = PointInstancer.CreatePositionsAttr())
	{
		Attr.Set(Positions, UsdTimeCode);
		UsdUtils::NotifyIfOverriddenOpinion(Attr);
	}

	if (UsdAttribute Attr = PointInstancer.CreateOrientationsAttr())
	{
		Attr.Set(Orientations, UsdTimeCode);
		UsdUtils::NotifyIfOverriddenOpinion(Attr);
	}

	if (UsdAttribute Attr = PointInstancer.CreateScalesAttr())
	{
		Attr.Set(Scales, UsdTimeCode);
		UsdUtils::NotifyIfOverriddenOpinion(Attr);
	}

	return true;
#else
	return false;
#endif	  // WITH_EDITOR
}

namespace UE::USDPrimConversion::Private
{
	// If we're going to bake a LevelSequence with bindings to blueprints that have been set with bRunConstructionScriptInSequencer,
	// on every single baked frame the construction script will be rerun, likely recreating all of the blueprint's components.
	// This means we can't just capture component references into our baker lambdas, and must instead capture something that can
	// find its ideal component again in that case.
	//
	// We're going to assume component names are consistent across every run of the construction scripts, and just use these getters
	// to reconnect with the ideal components. The actor itself survives the construction script, as it is the blueprint actor itself
	template<typename T>
	TFunction<T*(void)> CreateComponentGetter(T* OldComponent)
	{
		if (!OldComponent)
		{
			return []()
			{
				return nullptr;
			};
		}

		const AActor* Actor = OldComponent->GetOwner();
		if (!Actor)
		{
			// Can't do much without an actor (shouldn't really ever happen though)
			return [OldComponent]()
			{
				return IsValid(OldComponent) ? OldComponent : nullptr;
			};
		}

		const FName ComponentName = OldComponent->GetFName();
		return [OldComponent, Actor, ComponentName]() -> T*
		{
			if (IsValid(OldComponent))
			{
				return OldComponent;
			}

			// TODO: Maybe keep track of the last index where we found our component,
			// and start searching there the next call?
			for (UActorComponent* ChildComponent : Actor->GetComponents())
			{
				if (T* CastComponent = Cast<T>(ChildComponent))
				{
					if (ChildComponent->GetFName() == ComponentName)
					{
						return CastComponent;
					}
				}
			}

			return nullptr;
		};
	}

	template<typename UEType, typename UsdType>
	TFunction<void(UEType, double)> CreateCachedAttrSetter(pxr::UsdAttribute Attr, TFunction<UsdType(UEType)> PostConversion)
	{
		if (!Attr)
		{
			return [](UEType, double)
			{
			};
		}

		return [LastValue = TOptional<std::decay_t<UEType>>(),
				LastTimeCode = -DBL_MAX,
				Attr,
				PostConversion](UEType NewValue, double NewTimeCode) mutable
		{
			bool bNewValueIsEqual = false;
			if (LastValue.IsSet())
			{
				if constexpr (std::is_same_v<std::decay_t<UEType>, FTransform>)
				{
					// Transforms don't have an operator== it seems
					bNewValueIsEqual = LastValue.GetValue().Equals(NewValue);
				}
				else if constexpr (std::is_floating_point_v<std::decay_t<UEType>>)
				{
					bNewValueIsEqual = FMath::IsNearlyEqual(LastValue.GetValue(), NewValue);
				}
				else
				{
					bNewValueIsEqual = LastValue.GetValue() == NewValue;
				}
			}

			if (GSkipConstantValues && bNewValueIsEqual)
			{
				LastTimeCode = NewTimeCode;
				return;
			}

			Attr.Set<UsdType>(PostConversion(NewValue), NewTimeCode);

			if (!bNewValueIsEqual && LastValue.IsSet())
			{
				// Have to make sure that we write the last timeSample of any stretch where the value stayed constant,
				// or else we will affect how the resulting curve interpolates from the old value to the new value
				Attr.Set<UsdType>(PostConversion(LastValue.GetValue()), LastTimeCode);
			}

			LastValue = NewValue;
			LastTimeCode = NewTimeCode;
		};
	}

	TFunction<void(bool, double)> CreateCachedAttrSetter(pxr::UsdAttribute Attr, pxr::UsdGeomImageable Imageable)
	{
		if (!Attr)
		{
			return [](bool, double)
			{
			};
		}

		return [LastValue = TOptional<bool>(), LastTimeCode = -DBL_MAX, Attr, Imageable](bool NewValue, double NewTimeCode) mutable
		{
			const bool bNewValueIsEqual = LastValue.IsSet() ? LastValue.GetValue() == NewValue : false;
			if (GSkipConstantValues && bNewValueIsEqual)
			{
				LastTimeCode = NewTimeCode;
				return;
			}

			if (NewValue)
			{
				Imageable.MakeVisible(NewTimeCode);
			}
			else
			{
				Imageable.MakeInvisible(NewTimeCode);
			}

			// Imagine our visibility track has a single key that switches to hidden at frame 60.
			// If our prim is visible by default, MakeVisible will author absolutely nothing, and we'll end up
			// with a timeSamples that just has '60: "invisible"'. Weirdly enough, in USD that means the prim
			// will be invisible throughout *the entire duration of the animation* though, which is not what we want.
			// This check will ensure that if we're visible we should have a value here and not rely on the
			// fallback value of 'visible', as that doesn't behave how we want.
			if (!Attr.HasAuthoredValue())
			{
				Attr.Set<pxr::TfToken>(NewValue ? pxr::UsdGeomTokens->inherited : pxr::UsdGeomTokens->invisible, NewTimeCode);

				if (!bNewValueIsEqual && LastValue.IsSet())
				{
					Attr.Set<pxr::TfToken>(LastValue ? pxr::UsdGeomTokens->inherited : pxr::UsdGeomTokens->invisible, LastTimeCode);
				}
			}

			LastValue = NewValue;
			LastTimeCode = NewTimeCode;
		};
	}

	// Overloads intended to handle the translation/rotation/scales of skeletal transforms
	// (respectively pxr::VtVec3fArray, pxr::VtQuatfArray, pxr::VtVec3hArray)
	template<typename UEArrayType, typename UsdArrayType>
	TFunction<void(const UEArrayType&, double)> CreateCachedAttrSetter(
		pxr::UsdAttribute Attr,
		TFunction<void(const UEArrayType&, UsdArrayType&)> PostConversion
	)
	{
		if (!Attr)
		{
			return [](const UEArrayType&, double)
			{
			};
		}

		return [LastValue = TOptional<UEArrayType>(),
				LastTimeCode = -DBL_MAX,
				Attr,
				PostConversion,
				ConvertedValues = TUsdStore<UsdArrayType>()	   //
		](const UEArrayType& NewValue, double NewTimeCode) mutable
		{
			bool bNewValueIsEqual = false;
			if (LastValue.IsSet())
			{
				if (LastValue.GetValue().Num() != NewValue.Num())
				{
					bNewValueIsEqual = false;
				}
				else
				{
					bNewValueIsEqual = true;
					for (int32 Index = 0; Index < NewValue.Num(); ++Index)
					{
						// Translations
						if constexpr (std::is_same_v<UsdArrayType, pxr::VtVec3fArray>)
						{
							if (!LastValue.GetValue()[Index].GetTranslation().Equals(NewValue[Index].GetTranslation()))
							{
								bNewValueIsEqual = false;
								break;
							}
						}
						// Rotations
						else if constexpr (std::is_same_v<UsdArrayType, pxr::VtQuatfArray>)
						{
							if (!LastValue.GetValue()[Index].GetRotation().Equals(NewValue[Index].GetRotation()))
							{
								bNewValueIsEqual = false;
								break;
							}
						}
						// Scales
						else if constexpr (std::is_same_v<UsdArrayType, pxr::VtVec3hArray>)
						{
							if (!LastValue.GetValue()[Index].GetScale3D().Equals(NewValue[Index].GetScale3D()))
							{
								bNewValueIsEqual = false;
								break;
							}
						}
						// Blend shape weights
						else if constexpr (std::is_same_v<UsdArrayType, pxr::VtArray<float>>)
						{
							if (!FMath::IsNearlyEqual(LastValue.GetValue()[Index], NewValue[Index]))
							{
								bNewValueIsEqual = false;
								break;
							}
						}
					}
				}
			}
			if (GSkipConstantValues && bNewValueIsEqual)
			{
				LastTimeCode = NewTimeCode;
				return;
			}

			PostConversion(NewValue, ConvertedValues.Get());
			Attr.Set(ConvertedValues.Get(), NewTimeCode);

			if (!bNewValueIsEqual && LastValue.IsSet())
			{
				PostConversion(LastValue.GetValue(), ConvertedValues.Get());
				Attr.Set(ConvertedValues.Get(), LastTimeCode);
			}

			LastValue = NewValue;
			LastTimeCode = NewTimeCode;
		};
	}
}	 // namespace UE::USDPrimConversion::Private

bool UnrealToUsd::CreateComponentPropertyBaker(
	UE::FUsdPrim& Prim,
	const USceneComponent& Component,
	const FString& PropertyPath,
	FComponentBaker& OutBaker
)
{
	using namespace UE::USDPrimConversion::Private;

	EBakingType BakerType = EBakingType::None;
	TFunction<void(double)> BakerFunction;

	FScopedUsdAllocs Allocs;

	pxr::UsdPrim UsdPrim{Prim};
	pxr::UsdStageRefPtr UsdStage = UsdPrim.GetStage();
	FUsdStageInfo StageInfo{UsdStage};

	TFunction<float(float)> DistanceConversion = [StageInfo](float UEValue)
	{
		return UnrealToUsd::ConvertDistance(StageInfo, UEValue);
	};

	TFunction<float(float)> NoFloatConversion = [StageInfo](float UEValue)
	{
		return UEValue;
	};

	TFunction<float(float)> InvertConversion = [StageInfo](float UEValue)
	{
		return UEValue != 0.0f ? (1.0f / UEValue) : 0.0f;
	};

	TFunction<bool(bool)> NoBoolConversion = [](bool Value)
	{
		return Value;
	};

	TFunction<pxr::GfVec3f(const FColor&)> ColorConversion = [](const FColor& Value)
	{
		pxr::GfVec4f LinearColor = UnrealToUsd::ConvertColor(Value);
		return pxr::GfVec3f(LinearColor[0], LinearColor[1], LinearColor[2]);
	};

	TFunction<pxr::GfMatrix4d(const FTransform&)> TransformConversion = [StageInfo](const FTransform& UEValue)
	{
		return UnrealToUsd::ConvertTransform(StageInfo, UEValue);
	};

	TFunction<pxr::VtArray<pxr::GfVec3f>(const FBox&)> BoxConversion = [StageInfo](const FBox& UEBox)
	{
		pxr::GfVec3f UEBoundsMinUsdSpace = UnrealToUsd::ConvertVectorFloat(StageInfo, UEBox.Min);
		pxr::GfVec3f UEBoundsMaxUsdSpace = UnrealToUsd::ConvertVectorFloat(StageInfo, UEBox.Max);

		pxr::GfVec3f UsdMin{
			FMath::Min(UEBoundsMinUsdSpace[0], UEBoundsMaxUsdSpace[0]),
			FMath::Min(UEBoundsMinUsdSpace[1], UEBoundsMaxUsdSpace[1]),
			FMath::Min(UEBoundsMinUsdSpace[2], UEBoundsMaxUsdSpace[2])
		};

		pxr::GfVec3f UsdMax{
			FMath::Max(UEBoundsMinUsdSpace[0], UEBoundsMaxUsdSpace[0]),
			FMath::Max(UEBoundsMinUsdSpace[1], UEBoundsMaxUsdSpace[1]),
			FMath::Max(UEBoundsMinUsdSpace[2], UEBoundsMaxUsdSpace[2])
		};

		return pxr::VtArray<pxr::GfVec3f>{UsdMin, UsdMax};
	};

	// SceneComponent
	{
		const static FString TransformPropertyString = UnrealIdentifiers::TransformPropertyName.ToString();
		const static FString HiddenPropertyString = UnrealIdentifiers::HiddenPropertyName.ToString();
		const static FString HiddenInGamePropertyString = UnrealIdentifiers::HiddenInGamePropertyName.ToString();

		if (PropertyPath == TransformPropertyString)
		{
			pxr::UsdGeomXformable Xformable(UsdPrim);
			if (!Xformable)
			{
				return false;
			}

			// Clear existing transform data and leave just one Transform op there
			pxr::UsdGeomXformOp TransformOp = UE::USDPrimConversion::Private::ForceMatrixXform(Xformable);
			if (!TransformOp)
			{
				return false;
			}

			pxr::UsdAttribute Attr = TransformOp.GetAttr();
			if (!Attr)
			{
				return false;
			}

			Attr.Clear();

			// Keep track of our original attach parent so that we can export baked transforms in case we have attach tracks.
			// We're generating these before the Sequencer is evaluated with our Sequence, so this OriginalAttachParent should be
			// the same parent that will be exported to the USD layers
			USceneComponent* OriginalAttachParent = Component.GetAttachParent();

			const bool bStageIsZUp = UsdUtils::GetUsdStageUpAxis(UsdStage) == pxr::UsdGeomTokens->z;

			// Compensate different orientation for light or camera components
			FTransform CameraCompensation = FTransform::Identity;
			if (UsdPrim.IsA<pxr::UsdGeomCamera>() || UsdPrim.HasAPI<pxr::UsdLuxLightAPI>())
			{
				CameraCompensation = FTransform(FRotator(0.0f, 90.0f, 0.0f));

				if (bStageIsZUp)
				{
					CameraCompensation *= FTransform(FRotator(90.0f, 0.0f, 0.0f));
				}
			}

			// Note how we only need the ParentCameraCompensation for our actual OriginalAttachParent. When we're "attached"
			// to a camera or light via our attach track the world transform for that camera or light won't contain a CameraCompensation
			// itself, as that is something that we add ourselves only when exporting the relative transforms to USD
			FTransform InverseParentCameraCompensation = FTransform::Identity;
			if (OriginalAttachParent)
			{
				if (OriginalAttachParent->IsA(UCineCameraComponent::StaticClass()) || OriginalAttachParent->IsA(ULightComponent::StaticClass()))
				{
					InverseParentCameraCompensation = FTransform(FRotator(0.0f, 90.f, 0.0f));

					if (bStageIsZUp)
					{
						InverseParentCameraCompensation *= FTransform(FRotator(90.0f, 0.f, 0.0f));
					}

					InverseParentCameraCompensation = InverseParentCameraCompensation.Inverse();
				}
			}

			TFunction<const USceneComponent*(void)> ComponentGetter = CreateComponentGetter(&Component);
			TFunction<USceneComponent*(void)> OriginalAttachParentGetter = CreateComponentGetter(OriginalAttachParent);
			TFunction<void(const FTransform&, double)> TransformSetter = CreateCachedAttrSetter(Attr, TransformConversion);

			BakerType = EBakingType::Transform;
			BakerFunction = [CameraCompensation,	//
							 InverseParentCameraCompensation,
							 TransformSetter,
							 ComponentGetter,
							 OriginalAttachParentGetter](double UsdTimeCode) mutable
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(TransformBaker);

				FScopedUsdAllocs Allocs;

				const USceneComponent* Component = ComponentGetter();
				if (!Component)
				{
					return;
				}

				// Some setup (like CameraRig_Rail) require that the parent actor on which the component is attached be Tick'ed to update their
				// positions
				if (AActor* AttachParentActor = Component->GetAttachParentActor())
				{
					AttachParentActor->Tick(0.0f);
				}

				USceneComponent* OriginalAttachParent = OriginalAttachParentGetter();

				// If we're attached to a socket our RelativeTransform will be relative to the socket, instead of the parent
				// component space. If we were to use GetRelativeTransform directly, we're in charge of managing the socket
				// transform too (and any other N obscure features we don't know about/don't exist yet). If we fetch directly
				// the component-to-world transform however, the component will do that on its own (as that is the transform
				// that is actually used to show it on the level), so we don't have to worry about it!
				// It may seem wasteful to do this inside the baker function, but you can place "Attach tracks" on the
				// Sequencer that may make the attach socket change every frame, so we do need this
				FTransform RelativeTransform;
				if (OriginalAttachParent)
				{
					OriginalAttachParent->ConditionalUpdateComponentToWorld();
					OriginalAttachParent->UpdateChildTransforms();
					RelativeTransform = Component->GetComponentTransform().GetRelativeTransform(OriginalAttachParent->GetComponentTransform());
				}
				else
				{
					// Use the world transform here, because while we may not have an *original* attach parent, this sequence
					// could have attach tracks, meaning that we may gain a different attach parent at some point
					RelativeTransform = Component->GetComponentTransform();
				}
				RelativeTransform = CameraCompensation * RelativeTransform * InverseParentCameraCompensation;

				TransformSetter(RelativeTransform, UsdTimeCode);
			};
		}
		// bHidden is for the actor, and bHiddenInGame is for a component
		// A component is only visible when it's not hidden and its actor is not hidden
		// A bHidden is just handled like a bHiddenInGame for the actor's root component
		// Whenever we handle a bHiddenInGame, we always combine it with the actor's bHidden
		else if (PropertyPath == HiddenPropertyString || PropertyPath == HiddenInGamePropertyString)
		{
			pxr::UsdGeomImageable Imageable(UsdPrim);
			if (!Imageable)
			{
				return false;
			}

			pxr::UsdAttribute Attr = Imageable.CreateVisibilityAttr();
			Attr.Clear();

			TFunction<const USceneComponent*(void)> ComponentGetter = CreateComponentGetter(&Component);
			TFunction<void(bool, double)> VisibilitySetter = CreateCachedAttrSetter(Attr, Imageable);

			BakerType = EBakingType::Visibility;
			BakerFunction = [VisibilitySetter, ComponentGetter](double UsdTimeCode)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(VisibilityBaker);

				const USceneComponent* Component = ComponentGetter();
				if (!Component)
				{
					return;
				}

				const bool bVisibleInUE = !(Component->bHiddenInGame || Component->GetOwner()->IsHidden());
				VisibilitySetter(bVisibleInUE, UsdTimeCode);
			};
		}
	}

	if (const UCineCameraComponent* CameraComponent = Cast<UCineCameraComponent>(&Component))
	{
		static TSet<FString> RelevantProperties = {
			UnrealIdentifiers::CurrentFocalLengthPropertyName.ToString(),
			UnrealIdentifiers::ManualFocusDistancePropertyName.ToString(),
			UnrealIdentifiers::CurrentAperturePropertyName.ToString(),
			UnrealIdentifiers::SensorWidthPropertyName.ToString(),
			UnrealIdentifiers::SensorHeightPropertyName.ToString(),
			UnrealIdentifiers::SensorHorizontalOffsetPropertyName.ToString(),
			UnrealIdentifiers::SensorVerticalOffsetPropertyName.ToString(),
			UnrealIdentifiers::ExposureCompensationPropertyName.ToString(),
			UnrealIdentifiers::ProjectionModePropertyName.ToString(),
			UnrealIdentifiers::OrthoFarClipPlanePropertyName.ToString(),
			UnrealIdentifiers::OrthoNearClipPlanePropertyName.ToString(),
			UnrealIdentifiers::CustomNearClipppingPlanePropertyName.ToString()
		};

		if (RelevantProperties.Contains(PropertyPath))
		{
			pxr::UsdGeomCamera GeomCamera{Prim};
			if (!GeomCamera)
			{
				return false;
			}

			TFunction<pxr::TfToken(ECameraProjectionMode::Type)> ProjectionConversion = [](ECameraProjectionMode::Type UEMode)
			{
				return UEMode == ECameraProjectionMode::Orthographic ? pxr::UsdGeomTokens->orthographic : pxr::UsdGeomTokens->perspective;
			};

			TFunction<pxr::GfVec2f(const FVector2D& UEVec)> ClippingRangeConversion = [StageInfo](const FVector2D& UEVec)
			{
				return pxr::GfVec2f{
					UnrealToUsd::ConvertDistance(StageInfo, UEVec[0]),	  //
					UnrealToUsd::ConvertDistance(StageInfo, UEVec[1])
				};
			};

			TFunction<const UCineCameraComponent*(void)> ComponentGetter = CreateComponentGetter(CameraComponent);

			pxr::UsdAttribute FocalLengthAttr = GeomCamera.CreateFocalLengthAttr();
			pxr::UsdAttribute FocusDistanceAttr = GeomCamera.CreateFocusDistanceAttr();
			pxr::UsdAttribute FStopAttr = GeomCamera.CreateFStopAttr();
			pxr::UsdAttribute HorizontalApertureAttr = GeomCamera.CreateHorizontalApertureAttr();
			pxr::UsdAttribute VerticalApertureAttr = GeomCamera.CreateVerticalApertureAttr();
			pxr::UsdAttribute HorizontalApertureOffsetAttr = GeomCamera.CreateHorizontalApertureOffsetAttr();
			pxr::UsdAttribute VerticalApertureOffsetAttr = GeomCamera.CreateVerticalApertureOffsetAttr();
			pxr::UsdAttribute ExposureAttr = GeomCamera.CreateExposureAttr();
			pxr::UsdAttribute ProjectionAttr = GeomCamera.CreateProjectionAttr();
			pxr::UsdAttribute ClippingRangeAttr = GeomCamera.CreateClippingRangeAttr();
			pxr::UsdAttribute ExposureFStopAttr = GeomCamera.CreateExposureFStopAttr();
			pxr::UsdAttribute ExposureISOAttr = GeomCamera.CreateExposureIsoAttr();
			pxr::UsdAttribute ExposureTimeAttr = GeomCamera.CreateExposureTimeAttr();

			TFunction<void(float, double)> FocalLengthSetter = CreateCachedAttrSetter(FocalLengthAttr, DistanceConversion);
			TFunction<void(float, double)> FocusDistanceSetter = CreateCachedAttrSetter(FocusDistanceAttr, DistanceConversion);
			TFunction<void(float, double)> FStopSetter = CreateCachedAttrSetter(FStopAttr, NoFloatConversion);
			TFunction<void(float, double)> HorizontalApertureSetter = CreateCachedAttrSetter(HorizontalApertureAttr, DistanceConversion);
			TFunction<void(float, double)> VerticalApertureSetter = CreateCachedAttrSetter(VerticalApertureAttr, DistanceConversion);
			TFunction<void(float, double)> HorizontalApertureOffsetSetter = CreateCachedAttrSetter(HorizontalApertureOffsetAttr, DistanceConversion);
			TFunction<void(float, double)> VerticalApertureOffsetSetter = CreateCachedAttrSetter(VerticalApertureOffsetAttr, DistanceConversion);
			TFunction<void(float, double)> ExposureSetter = CreateCachedAttrSetter(ExposureAttr, NoFloatConversion);
			TFunction<void(ECameraProjectionMode::Type, double)> ProjectionSetter = CreateCachedAttrSetter(ProjectionAttr, ProjectionConversion);
			TFunction<void(const FVector2D&, double)> ClippingRangeSetter = CreateCachedAttrSetter(ClippingRangeAttr, ClippingRangeConversion);
			TFunction<void(float, double)> ExposureFStopSetter = CreateCachedAttrSetter(ExposureFStopAttr, NoFloatConversion);
			TFunction<void(float, double)> ExposureISOSetter = CreateCachedAttrSetter(ExposureISOAttr, NoFloatConversion);
			TFunction<void(float, double)> ExposureTimeSetter = CreateCachedAttrSetter(ExposureTimeAttr, InvertConversion);

			float DefaultUEFarPlane = FLT_MAX;
			if (ClippingRangeAttr)
			{
				pxr::GfVec2f ClippingRangeUsdUnits{1, 1000000};	   // Values copied from the schema, as a fallback
				ClippingRangeAttr.Get(&ClippingRangeUsdUnits, pxr::UsdTimeCode::EarliestTime());

				DefaultUEFarPlane = UsdToUnreal::ConvertDistance(StageInfo, ClippingRangeUsdUnits[1]);
			}

			BakerType = EBakingType::Camera;
			BakerFunction = [DefaultUEFarPlane,
							 ComponentGetter,	 //
							 FocalLengthSetter,
							 FocusDistanceSetter,
							 FStopSetter,
							 HorizontalApertureSetter,
							 VerticalApertureSetter,
							 HorizontalApertureOffsetSetter,
							 VerticalApertureOffsetSetter,
							 ExposureSetter,
							 ProjectionSetter,
							 ClippingRangeSetter,
							 ExposureFStopSetter,
							 ExposureISOSetter,
							 ExposureTimeSetter](double UsdTimeCode) mutable
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(CameraBaker);

				const UCineCameraComponent* CameraComponent = ComponentGetter();
				if (!CameraComponent)
				{
					return;
				}

				FocalLengthSetter(CameraComponent->CurrentFocalLength, UsdTimeCode);
				FocusDistanceSetter(CameraComponent->FocusSettings.ManualFocusDistance, UsdTimeCode);

				if (CameraComponent->PostProcessSettings.bOverride_DepthOfFieldFstop)
				{
					FStopSetter(CameraComponent->PostProcessSettings.DepthOfFieldFstop, UsdTimeCode);
				}

				HorizontalApertureSetter(CameraComponent->Filmback.SensorWidth, UsdTimeCode);
				VerticalApertureSetter(CameraComponent->Filmback.SensorHeight, UsdTimeCode);
				HorizontalApertureOffsetSetter(CameraComponent->Filmback.SensorHorizontalOffset, UsdTimeCode);
				VerticalApertureOffsetSetter(CameraComponent->Filmback.SensorVerticalOffset, UsdTimeCode);

				if (CameraComponent->PostProcessSettings.bOverride_AutoExposureBias)
				{
					ExposureSetter(CameraComponent->PostProcessSettings.AutoExposureBias, UsdTimeCode);
				}

				ProjectionSetter(CameraComponent->ProjectionMode, UsdTimeCode);

				bool bWriteClippingRange = false;
				FVector2D UEClippingRange{0, DefaultUEFarPlane};
				if (CameraComponent->ProjectionMode == ECameraProjectionMode::Perspective)
				{
					bWriteClippingRange = CameraComponent->bOverride_CustomNearClippingPlane;
					if (bWriteClippingRange)
					{
						UEClippingRange[0] = CameraComponent->CustomNearClippingPlane;
					}
				}
				else
				{
					bWriteClippingRange = true;
					UEClippingRange[0] = CameraComponent->OrthoNearClipPlane;
					UEClippingRange[1] = CameraComponent->OrthoFarClipPlane;
				}
				if (bWriteClippingRange)
				{
					ClippingRangeSetter(UEClippingRange, UsdTimeCode);
				}

				ExposureFStopSetter(CameraComponent->CurrentAperture, UsdTimeCode);

				if (CameraComponent->PostProcessSettings.bOverride_CameraISO)
				{
					ExposureISOSetter(CameraComponent->PostProcessSettings.CameraISO, UsdTimeCode);
				}

				if (CameraComponent->PostProcessSettings.bOverride_CameraShutterSpeed)
				{
					ExposureTimeSetter(CameraComponent->PostProcessSettings.CameraShutterSpeed, UsdTimeCode);
				}
			};
		}
	}
	else if (const ULightComponentBase* LightComponentBase = Cast<ULightComponentBase>(&Component))
	{
		const ULightComponent* LightComponent = Cast<ULightComponent>(LightComponentBase);

		pxr::UsdLuxLightAPI LightAPI(Prim);
		if (!LightAPI)
		{
			return false;
		}

		pxr::UsdAttribute IntensityAttr = LightAPI.CreateIntensityAttr();
		pxr::UsdAttribute ColorAttr = LightAPI.CreateColorAttr();
		pxr::UsdAttribute EnableTemperatureAttr;
		pxr::UsdAttribute TemperatureAttr;
		pxr::UsdAttribute ShadowEnableAttr;

		if (LightComponent)
		{
			EnableTemperatureAttr = LightAPI.CreateEnableColorTemperatureAttr();
			TemperatureAttr = LightAPI.CreateColorTemperatureAttr();

			if (!LightComponent->CastShadows)
			{
				if (pxr::UsdLuxShadowAPI ShadowAPI = pxr::UsdLuxShadowAPI::Apply(Prim))
				{
					ShadowEnableAttr = ShadowAPI.CreateShadowEnableAttr();
				}
			}
		}

		TFunction<void(float, double)> IntensitySetter = CreateCachedAttrSetter(IntensityAttr, NoFloatConversion);
		TFunction<void(const FColor&, double)> ColorSetter = CreateCachedAttrSetter(ColorAttr, ColorConversion);
		TFunction<void(bool, double)> EnableTemperatureSetter = CreateCachedAttrSetter(EnableTemperatureAttr, NoBoolConversion);
		TFunction<void(float, double)> TemperatureSetter = CreateCachedAttrSetter(TemperatureAttr, NoFloatConversion);
		TFunction<void(bool, double)> ShadowSetter = CreateCachedAttrSetter(ShadowEnableAttr, NoBoolConversion);

		static IConsoleVariable* MinLightSourceSizeCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("USD.LightExport.MinLightSourceSize"));
		const float MinLightSourceSize = MinLightSourceSizeCvar ? MinLightSourceSizeCvar->GetFloat() : 1.0f;

		if (const URectLightComponent* RectLightComponent = Cast<URectLightComponent>(LightComponentBase))
		{
			static TSet<FString> RelevantProperties =
				{TEXT("SourceHeight"), TEXT("SourceWidth"), TEXT("Temperature"), TEXT("bUseTemperature"), TEXT("LightColor"), TEXT("Intensity")};

			if (RelevantProperties.Contains(PropertyPath))
			{
				pxr::UsdLuxRectLight RectLight{Prim};
				if (!RectLight)
				{
					return false;
				}

				pxr::UsdAttribute WidthAttr = RectLight.CreateWidthAttr();
				pxr::UsdAttribute HeightAttr = RectLight.CreateHeightAttr();

				TFunction<const URectLightComponent*(void)> ComponentGetter = CreateComponentGetter(RectLightComponent);

				TFunction<void(float, double)> WidthSetter = CreateCachedAttrSetter(WidthAttr, DistanceConversion);
				TFunction<void(float, double)> HeightSetter = CreateCachedAttrSetter(HeightAttr, DistanceConversion);

				BakerType = EBakingType::Light;
				BakerFunction = [ComponentGetter,
								 IntensitySetter,
								 ColorSetter,
								 EnableTemperatureSetter,
								 TemperatureSetter,
								 ShadowSetter,
								 WidthSetter,
								 HeightSetter,
								 MinLightSourceSize](double UsdTimeCode) mutable
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(RectLightBaker);

					const URectLightComponent* RectLightComponent = ComponentGetter();
					if (!RectLightComponent)
					{
						return;
					}

					const float SourceWidthUE = FMath::Max(RectLightComponent->SourceWidth, MinLightSourceSize);
					const float SourceHeightUE = FMath::Max(RectLightComponent->SourceHeight, MinLightSourceSize);
					const float SolidAngle = UsdUtils::GetRectLightSolidAngle();
					const float AreaInSqMeters = UsdUtils::GetRectLightArea(SourceWidthUE / 100.0f, SourceHeightUE / 100.0f);
					const float FinalIntensityNits = UsdUtils::ConvertIntensityToNits(
						RectLightComponent->Intensity,
						SolidAngle,
						AreaInSqMeters,
						RectLightComponent->IntensityUnits
					);

					IntensitySetter(FinalIntensityNits, UsdTimeCode);
					WidthSetter(SourceWidthUE, UsdTimeCode);
					HeightSetter(SourceHeightUE, UsdTimeCode);

					ColorSetter(RectLightComponent->LightColor, UsdTimeCode);
					EnableTemperatureSetter(RectLightComponent->bUseTemperature, UsdTimeCode);
					TemperatureSetter(RectLightComponent->Temperature, UsdTimeCode);

					ShadowSetter(RectLightComponent->CastShadows, UsdTimeCode);
				};
			}
		}
		else if (const USpotLightComponent* SpotLightComponent = Cast<USpotLightComponent>(LightComponentBase))
		{
			static TSet<FString> RelevantProperties =
				{TEXT("OuterConeAngle"), TEXT("InnerConeAngle"), TEXT("Temperature"), TEXT("bUseTemperature"), TEXT("LightColor"), TEXT("Intensity")};

			if (RelevantProperties.Contains(PropertyPath))
			{
				pxr::UsdLuxSphereLight SphereLight{Prim};
				pxr::UsdLuxShapingAPI ShapingAPI{Prim};
				if (!SphereLight || !ShapingAPI)
				{
					return false;
				}

				pxr::UsdAttribute RadiusAttr = SphereLight.CreateRadiusAttr();
				pxr::UsdAttribute TreatAsPointAttr = SphereLight.CreateTreatAsPointAttr();
				pxr::UsdAttribute AngleAttr = ShapingAPI.CreateShapingConeAngleAttr();
				pxr::UsdAttribute SoftnessAttr = ShapingAPI.CreateShapingConeSoftnessAttr();

				TFunction<const USpotLightComponent*(void)> ComponentGetter = CreateComponentGetter(SpotLightComponent);

				TFunction<void(bool, double)> TreatAsPointSetter = CreateCachedAttrSetter(TreatAsPointAttr, NoBoolConversion);
				TFunction<void(float, double)> RadiusSetter = CreateCachedAttrSetter(RadiusAttr, DistanceConversion);
				TFunction<void(float, double)> AngleSetter = CreateCachedAttrSetter(AngleAttr, NoFloatConversion);
				TFunction<void(float, double)> SoftnessSetter = CreateCachedAttrSetter(SoftnessAttr, NoFloatConversion);

				BakerType = EBakingType::Light;
				BakerFunction = [ComponentGetter,
								 IntensitySetter,
								 TreatAsPointSetter,
								 RadiusSetter,
								 ColorSetter,
								 EnableTemperatureSetter,
								 TemperatureSetter,
								 ShadowSetter,
								 AngleSetter,
								 SoftnessSetter,
								 MinLightSourceSize](double UsdTimeCode) mutable
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(SpotLightBaker);

					const USpotLightComponent* SpotLightComponent = ComponentGetter();
					if (!SpotLightComponent)
					{
						return;
					}

					const float SourceRadiusUE = FMath::Max(SpotLightComponent->SourceRadius, MinLightSourceSize);
					const float SolidAngle = UsdUtils::GetSpotLightSolidAngle(SpotLightComponent->OuterConeAngle, SpotLightComponent->InnerConeAngle);
					const float AreaInSqMeters = UsdUtils::GetSpotLightArea(SourceRadiusUE / 100.0f);
					const float FinalIntensityNits = UsdUtils::ConvertIntensityToNits(
						SpotLightComponent->Intensity,
						SolidAngle,
						AreaInSqMeters,
						SpotLightComponent->IntensityUnits
					);

					IntensitySetter(FinalIntensityNits, UsdTimeCode);
					TreatAsPointSetter(FMath::IsNearlyZero(SpotLightComponent->SourceRadius), UsdTimeCode);
					RadiusSetter(SourceRadiusUE, UsdTimeCode);
					
					float UsdConeAngle = 90.0;
					float UsdConeSoftness = 0.0f;
					UnrealToUsd::ConvertSpotLightConeAngles(SpotLightComponent->OuterConeAngle, SpotLightComponent->InnerConeAngle, UsdConeAngle, UsdConeSoftness);
					AngleSetter(UsdConeAngle, UsdTimeCode);
					SoftnessSetter(UsdConeSoftness, UsdTimeCode);

					ColorSetter(SpotLightComponent->LightColor, UsdTimeCode);
					EnableTemperatureSetter(SpotLightComponent->bUseTemperature, UsdTimeCode);
					TemperatureSetter(SpotLightComponent->Temperature, UsdTimeCode);

					ShadowSetter(SpotLightComponent->CastShadows, UsdTimeCode);
				};
			}
		}
		else if (const UPointLightComponent* PointLightComponent = Cast<UPointLightComponent>(LightComponentBase))
		{
			static TSet<FString> RelevantProperties =
				{TEXT("SourceRadius"), TEXT("Temperature"), TEXT("bUseTemperature"), TEXT("LightColor"), TEXT("Intensity")};

			if (RelevantProperties.Contains(PropertyPath))
			{
				pxr::UsdLuxSphereLight SphereLight{Prim};
				if (!SphereLight)
				{
					return false;
				}

				pxr::UsdAttribute RadiusAttr = SphereLight.CreateRadiusAttr();
				pxr::UsdAttribute TreatAsPointAttr = SphereLight.CreateTreatAsPointAttr();

				TFunction<const UPointLightComponent*(void)> ComponentGetter = CreateComponentGetter(PointLightComponent);

				TFunction<void(bool, double)> TreatAsPointSetter = CreateCachedAttrSetter(TreatAsPointAttr, NoBoolConversion);
				TFunction<void(float, double)> RadiusSetter = CreateCachedAttrSetter(RadiusAttr, DistanceConversion);

				BakerType = EBakingType::Light;
				BakerFunction = [ComponentGetter,
								 IntensitySetter,
								 TreatAsPointSetter,
								 RadiusSetter,
								 ColorSetter,
								 EnableTemperatureSetter,
								 TemperatureSetter,
								 ShadowSetter,
								 MinLightSourceSize](double UsdTimeCode) mutable
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(PointLightBaker);

					const UPointLightComponent* PointLightComponent = ComponentGetter();
					if (!PointLightComponent)
					{
						return;
					}

					const float SourceRadiusUE = FMath::Max(PointLightComponent->SourceRadius, MinLightSourceSize);
					const float SolidAngle = UsdUtils::GetPointLightSolidAngle();
					const float AreaInSqMeters = UsdUtils::GetPointLightArea(SourceRadiusUE / 100.0f);
					const float FinalIntensityNits = UsdUtils::ConvertIntensityToNits(
						PointLightComponent->Intensity,
						SolidAngle,
						AreaInSqMeters,
						PointLightComponent->IntensityUnits
					);

					IntensitySetter(FinalIntensityNits, UsdTimeCode);
					TreatAsPointSetter(FMath::IsNearlyZero(PointLightComponent->SourceRadius), UsdTimeCode);
					RadiusSetter(SourceRadiusUE, UsdTimeCode);

					ColorSetter(PointLightComponent->LightColor, UsdTimeCode);
					EnableTemperatureSetter(PointLightComponent->bUseTemperature, UsdTimeCode);
					TemperatureSetter(PointLightComponent->Temperature, UsdTimeCode);

					ShadowSetter(PointLightComponent->CastShadows, UsdTimeCode);
				};
			}
		}
		else if (const UDirectionalLightComponent* DirectionalLightComponent = Cast<UDirectionalLightComponent>(LightComponentBase))
		{
			static TSet<FString> RelevantProperties =
				{TEXT("LightSourceAngle"), TEXT("Temperature"), TEXT("bUseTemperature"), TEXT("LightColor"), TEXT("Intensity")};

			if (RelevantProperties.Contains(PropertyPath))
			{
				pxr::UsdLuxDistantLight Light{Prim};
				if (!Light)
				{
					return false;
				}

				pxr::UsdAttribute AngleAttr = Light.CreateAngleAttr();

				TFunction<const UDirectionalLightComponent*(void)> ComponentGetter = CreateComponentGetter(DirectionalLightComponent);

				TFunction<void(float, double)> AngleSetter = CreateCachedAttrSetter(AngleAttr, NoFloatConversion);

				BakerType = EBakingType::Light;
				BakerFunction = [ComponentGetter,	 //
								 IntensitySetter,
								 AngleSetter,
								 ColorSetter,
								 EnableTemperatureSetter,
								 TemperatureSetter,
								 ShadowSetter](double UsdTimeCode) mutable
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(DirectionalLightBaker);

					const UDirectionalLightComponent* DirectionalLightComponent = ComponentGetter();
					if (!DirectionalLightComponent)
					{
						return;
					}

					IntensitySetter(DirectionalLightComponent->Intensity, UsdTimeCode);
					AngleSetter(DirectionalLightComponent->LightSourceAngle, UsdTimeCode);

					ColorSetter(DirectionalLightComponent->LightColor, UsdTimeCode);
					EnableTemperatureSetter(DirectionalLightComponent->bUseTemperature, UsdTimeCode);
					TemperatureSetter(DirectionalLightComponent->Temperature, UsdTimeCode);

					ShadowSetter(DirectionalLightComponent->CastShadows, UsdTimeCode);
				};
			}
		}
	}
	else if (const UUsdDrawModeComponent* DrawModeComponent = Cast<UUsdDrawModeComponent>(&Component))
	{
		// We don't support importing/exporting animated texture cards for now, and the other UsdGeomModelAPI
		// attributes are uniform. The only animation we can export from this are the extents
		static TSet<FString> RelevantProperties = {
			GET_MEMBER_NAME_CHECKED(UUsdDrawModeComponent, BoundsMin).ToString(),
			GET_MEMBER_NAME_CHECKED(UUsdDrawModeComponent, BoundsMax).ToString()
		};

		if (RelevantProperties.Contains(PropertyPath))
		{
			pxr::UsdAttribute ExtentsAttr;
			if (pxr::UsdGeomBoundable Boundable{UsdPrim})
			{
				// Try using the extents attribute if we're a boundable
				ExtentsAttr = Boundable.CreateExtentAttr();
			}
			if (!ExtentsAttr)
			{
				// Otherwise fallback to trying to use extentsHint
				if (pxr::UsdGeomModelAPI GeomModelAPI = pxr::UsdGeomModelAPI::Apply(UsdPrim))
				{
					// Copied from the implementation of pxr::UsdGeomModelAPI::SetExtentsHint() because for whatever reason
					// there is no CreateExtentsHintAttr()
					const bool bCustom = false;
					ExtentsAttr = UsdPrim.CreateAttribute(pxr::UsdGeomTokens->extentsHint, pxr::SdfValueTypeNames->Float3Array, bCustom);
				}
			}
			if (!ExtentsAttr)
			{
				return false;
			}

			TFunction<const UUsdDrawModeComponent*(void)> ComponentGetter = CreateComponentGetter(DrawModeComponent);

			TFunction<void(const FBox&, double)> ExtentsSetter = CreateCachedAttrSetter(ExtentsAttr, BoxConversion);

			BakerType = EBakingType::Bounds;
			BakerFunction = [ComponentGetter, ExtentsSetter](double UsdTimeCode) mutable
			{
				const UUsdDrawModeComponent* DrawModeComponent = ComponentGetter();
				if (!DrawModeComponent)
				{
					return;
				}

				const FBox NewBox{DrawModeComponent->BoundsMin, DrawModeComponent->BoundsMax};
				ExtentsSetter(NewBox, UsdTimeCode);
			};
		}
	}

	if (BakerType != EBakingType::None && BakerFunction)
	{
		OutBaker.BakerType = BakerType;
		OutBaker.BakerFunction = BakerFunction;
		OutBaker.ComponentPath = Component.GetPathName();
		return true;
	}

	return false;
}

bool UnrealToUsd::CreateSkeletalAnimationBaker(
	UE::FUsdPrim& SkeletonPrim,
	UE::FUsdPrim& SkelAnimation,
	USkeletalMeshComponent& Component,
	FComponentBaker& OutBaker
)
{
#if WITH_EDITOR
	using namespace UE::USDPrimConversion::Private;

	USkeletalMesh* SkeletalMesh = Component.GetSkeletalMeshAsset();
	if (!SkeletalMesh)
	{
		return false;
	}

	FScopedUsdAllocs Allocs;

	pxr::UsdSkelSkeleton UsdSkeleton{SkeletonPrim};
	if (!UsdSkeleton)
	{
		USD_LOG_WARNING(TEXT("Failed to create skeletal animation baker: Prim '%s' must be a UsdSkeleton!"), *SkeletonPrim.GetPrimPath().GetString());
		return false;
	}

	pxr::UsdSkelAnimation UsdSkelAnimation{SkelAnimation};
	if (!UsdSkeleton || !SkelAnimation)
	{
		return false;
	}

	// Make sure that the skeleton is using our animation
	pxr::UsdPrim SkelAnimPrim = UsdSkelAnimation.GetPrim();
	UsdUtils::BindAnimationSource(SkeletonPrim, SkelAnimPrim);

	FUsdStageInfo StageInfo{SkeletonPrim.GetStage()};

	pxr::UsdAttribute JointsAttr = UsdSkelAnimation.CreateJointsAttr();
	pxr::UsdAttribute TranslationsAttr = UsdSkelAnimation.CreateTranslationsAttr();
	pxr::UsdAttribute RotationsAttr = UsdSkelAnimation.CreateRotationsAttr();
	pxr::UsdAttribute ScalesAttr = UsdSkelAnimation.CreateScalesAttr();

	// Joints
	const FReferenceSkeleton& RefSkeleton = SkeletalMesh->GetRefSkeleton();
	const int32 NumBones = RefSkeleton.GetRefBoneInfo().Num();
	UnrealToUsd::ConvertJointsAttribute(RefSkeleton, JointsAttr);

	// Build active morph targets array if it isn't setup already
	TArray<TObjectPtr<UMorphTarget>>& MorphTargets = SkeletalMesh->GetMorphTargets();
	if (Component.ActiveMorphTargets.Num() != Component.MorphTargetWeights.Num() && MorphTargets.Num() != 0)
	{
		for (int32 MorphTargetIndex = 0; MorphTargetIndex < MorphTargets.Num(); ++MorphTargetIndex)
		{
			Component.ActiveMorphTargets.Add(MorphTargets[MorphTargetIndex], MorphTargetIndex);
		}
	}

	// Blend shape names
	// Here we have to export UMorphTarget FNames in some order, then the weights in that same order. That is all.
	// Those work out as "channels", and USD will resolve those to match the right thing on each mesh.
	// We sort them in weight index order so that within the Baker we just write out weights in the order they are in.
	pxr::UsdAttribute BlendShapeWeightsAttr;
	pxr::UsdAttribute BlendShapesAttr;
	const int32 NumMorphTargets = Component.MorphTargetWeights.Num();
	if (NumMorphTargets > 0)
	{
		BlendShapeWeightsAttr = UsdSkelAnimation.CreateBlendShapeWeightsAttr();
		BlendShapesAttr = UsdSkelAnimation.CreateBlendShapesAttr();

		TArray<FMorphTargetWeightMap::ElementType> SortedMorphTargets = Component.ActiveMorphTargets.Array();
		SortedMorphTargets.Sort(
			[](const FMorphTargetWeightMap::ElementType& Left, const FMorphTargetWeightMap::ElementType& Right)
			{
				return Left.Value < Right.Value;
			}
		);

		pxr::VtArray<pxr::TfToken> BlendShapeNames;
		BlendShapeNames.reserve(SortedMorphTargets.Num());

		for (const FMorphTargetWeightMap::ElementType& ActiveMorphTarget : SortedMorphTargets)
		{
			FString BlendShapeName;
			if (const UMorphTarget* MorphTarget = ActiveMorphTarget.Key)
			{
				BlendShapeName = MorphTarget->GetFName().ToString();
			}

			BlendShapeNames.push_back(UnrealToUsd::ConvertToken(*BlendShapeName).Get());
		}

		BlendShapesAttr.Set(BlendShapeNames);
	}

	TFunction<USkeletalMeshComponent*(void)> ComponentGetter = CreateComponentGetter(&Component);

	TFunction<void(const TArray<FTransform>&, pxr::VtVec3fArray&)> BoneTranslationConversion =
		[StageInfo](const TArray<FTransform>& UEBones, pxr::VtVec3fArray& Translations)
	{
		const int32 NumBones = UEBones.Num();
		Translations.resize(NumBones);

		for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
		{
			// Pulled from UsdUtils::ConvertAxes since we don't care about the other components of the transform here
			FVector Translation = UEBones[BoneIndex].GetTranslation();
			if (StageInfo.UpAxis == EUsdUpAxis::ZAxis)
			{
				Translation.Y = -Translation.Y;
			}
			else
			{
				Swap(Translation.Y, Translation.Z);
			}

			Translations[BoneIndex] = UnrealToUsd::ConvertVectorFloat(Translation) * ((float)(0.01 / StageInfo.MetersPerUnit));
		}
	};

	TFunction<void(const TArray<FTransform>&, pxr::VtQuatfArray&)> BoneRotationConversion =
		[StageInfo](const TArray<FTransform>& UEBones, pxr::VtQuatfArray& Rotations)
	{
		const int32 NumBones = UEBones.Num();

		Rotations.resize(NumBones);

		for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
		{
			// Pulled from UsdUtils::ConvertAxes since we don't care about the other components of the transform here
			FQuat Rotation = UEBones[BoneIndex].GetRotation();
			if (StageInfo.UpAxis == EUsdUpAxis::ZAxis)
			{
				Rotation.X = -Rotation.X;
				Rotation.Z = -Rotation.Z;
			}
			else
			{
				Rotation = Rotation.Inverse();
				Swap(Rotation.Y, Rotation.Z);
			}

			Rotations[BoneIndex] = UnrealToUsd::ConvertQuatFloat(Rotation).GetNormalized();
		}
	};

	TFunction<void(const TArray<FTransform>&, pxr::VtVec3hArray&)> BoneScaleConversion =
		[StageInfo](const TArray<FTransform>& UEBones, pxr::VtVec3hArray& Scales)
	{
		const int32 NumBones = UEBones.Num();

		Scales.resize(NumBones);

		for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
		{
			// Pulled from UsdUtils::ConvertAxes since we don't care about the other components of the transform here
			FVector Scale = UEBones[BoneIndex].GetScale3D();
			if (StageInfo.UpAxis != EUsdUpAxis::ZAxis)
			{
				Swap(Scale.Y, Scale.Z);
			}

			Scales[BoneIndex] = UnrealToUsd::ConvertVectorHalf(Scale);
		}
	};

	TFunction<void(const TArray<float>&, pxr::VtArray<float>&)> WeightConversion = [](const TArray<float>& UEWeights, pxr::VtArray<float>& UsdWeights)
	{
		UsdWeights.resize(UEWeights.Num());
		FMemory::Memcpy(UsdWeights.data(), UEWeights.GetData(), UEWeights.GetTypeSize() * UEWeights.Num());
	};

	TFunction<void(const TArray<FTransform>&, double)> BoneTranslationSetter = CreateCachedAttrSetter(	  //
		TranslationsAttr,
		BoneTranslationConversion
	);

	TFunction<void(const TArray<FTransform>&, double)> BoneRotationSetter = CreateCachedAttrSetter(	   //
		RotationsAttr,
		BoneRotationConversion
	);

	TFunction<void(const TArray<FTransform>&, double)> BoneScaleSetter = CreateCachedAttrSetter(	//
		ScalesAttr,
		BoneScaleConversion
	);

	TFunction<void(const TArray<float>&, double)> WeightSetter = CreateCachedAttrSetter(	//
		BlendShapeWeightsAttr,
		WeightConversion
	);

	OutBaker.ComponentPath = Component.GetPathName();
	OutBaker.BakerType = EBakingType::Skeletal;
	OutBaker.BakerFunction =
		[ComponentGetter, BoneTranslationSetter, BoneRotationSetter, BoneScaleSetter, WeightSetter, NumBones](double UsdTimeCode) mutable
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SkeletalBaker);

		USkeletalMeshComponent* Component = ComponentGetter();
		if (!Component)
		{
			return;
		}

		if (USkeletalMeshComponent* Leader = Cast<USkeletalMeshComponent>(Component->LeaderPoseComponent.Get()))
		{
			UsdUtils::RefreshSkeletalMeshComponent(*Leader);
		}
		UsdUtils::RefreshSkeletalMeshComponent(*Component);

		// I'm not entirely sure why this is needed but FFbxExporter::ExportAnimTrack and FFbxExporter::ExportLevelSequenceBaked3DTransformTrack
		// do this so for safety maybe we should as well?
		if (AActor* Owner = Component->GetOwner())
		{
			Owner->Tick(0.0f);
		}

		// Handle morph target weights
		{
			WeightSetter(Component->MorphTargetWeights, UsdTimeCode);
		}

		// Handle bone transforms
		{
			TArray<FTransform> LocalBoneTransforms;
			UsdUtils::GetBoneTransforms(Component, LocalBoneTransforms);

			// For whatever reason it seems that sometimes this is not ready for us, so let's force it to be recalculated
			if (LocalBoneTransforms.Num() == 0)
			{
				const int32 LODIndex = 0;
				Component->RecalcRequiredBones(LODIndex);
			}
			if (LocalBoneTransforms.Num() != NumBones)
			{
				USD_LOG_WARNING(
					TEXT(
						"Failed to retrieve bone transforms when baking skeletal animation for component '%s' at timeCode '%f'. Expected %d transforms, "
						"received %d"
					),
					*Component->GetPathName(),
					UsdTimeCode,
					NumBones,
					LocalBoneTransforms.Num()
				);
				return;
			}

			BoneTranslationSetter(LocalBoneTransforms, UsdTimeCode);
			BoneRotationSetter(LocalBoneTransforms, UsdTimeCode);
			BoneScaleSetter(LocalBoneTransforms, UsdTimeCode);
		}
	};

	return true;
#else
	return false;
#endif	  // WITH_EDITOR
}

UnrealToUsd::FPropertyTrackWriter UnrealToUsd::CreatePropertyTrackWriter(
	const USceneComponent& Component,
	const UMovieScenePropertyTrack& Track,
	UE::FUsdPrim& Prim,
	TSet<FName>& OutPropertyPathsToRefresh
)
{
	UnrealToUsd::FPropertyTrackWriter Result;

	if (!Prim)
	{
		return Result;
	}

	FScopedUsdAllocs Allocs;

	pxr::UsdPrim UsdPrim{Prim};
	pxr::UsdStageRefPtr UsdStage = UsdPrim.GetStage();
	FUsdStageInfo StageInfo{UsdStage};

	TArray<pxr::UsdAttribute> Attrs = {pxr::UsdAttribute{}};
	pxr::UsdAttribute& Attr = Attrs[0];
	{
		pxr::SdfChangeBlock ChangeBlock;

		const FName& PropertyPath = Track.GetPropertyPath();

		// Note that it's important that each individual case authors a spec for the relevant attribute right now,
		// even though it returns a FPropertyTrackWriter to actually do the baking later. This will be done with
		// CreateXformOpOrderAttr() or CreateVisibilityAttr() or CreateFocalLengthAttr(), etc.
		// This just ensures that our check for overridden attributes at the bottom of this function call works
		// properly.

		// SceneComponent
		{
			if (PropertyPath == UnrealIdentifiers::TransformPropertyName)
			{
				if (pxr::UsdGeomXformable Xformable{UsdPrim})
				{
					if (pxr::UsdGeomXformOp TransformOp = UE::USDPrimConversion::Private::ForceMatrixXform(Xformable))
					{
						Attr = TransformOp.GetAttr();

						const bool bStageIsZUp = StageInfo.UpAxis == EUsdUpAxis::ZAxis;

						// Compensate different orientation for light or camera components
						FTransform Compensation = FTransform::Identity;
						if (UsdPrim.IsA<pxr::UsdGeomCamera>() || UsdPrim.HasAPI<pxr::UsdLuxLightAPI>())
						{
							Compensation = FTransform(FRotator(0.0f, 90.0f, 0.0f));

							if (bStageIsZUp)
							{
								Compensation *= FTransform(FRotator(90.0f, 0.0f, 0.0f));
							}
						}

						// Invert compensation applied to parent if it's a light or camera component
						FTransform InverseParentCompensation = FTransform::Identity;
						if (const USceneComponent* AttachParent = Component.GetAttachParent())
						{
							if (AttachParent->IsA(UCineCameraComponent::StaticClass()) || AttachParent->IsA(ULightComponent::StaticClass()))
							{
								InverseParentCompensation = FTransform(FRotator(0.0f, 90.f, 0.0f));

								if (bStageIsZUp)
								{
									InverseParentCompensation *= FTransform(FRotator(90.0f, 0.f, 0.0f));
								}

								InverseParentCompensation = InverseParentCompensation.Inverse();
							}
						}

						Result.TransformWriter =
							[Compensation, InverseParentCompensation, StageInfo, Attr](const FTransform& UEValue, double UsdTimeCode)
						{
							FTransform FinalUETransform = Compensation * UEValue * InverseParentCompensation;
							pxr::GfMatrix4d UsdTransform = UnrealToUsd::ConvertTransform(StageInfo, FinalUETransform);
							Attr.Set<pxr::GfMatrix4d>(UsdTransform, UsdTimeCode);
						};
					}
				}
			}
			// bHidden is for the actor, and bHiddenInGame is for a component
			// A component is only visible when it's not hidden and its actor is not hidden
			// A bHidden is just handled like a bHiddenInGame for the actor's root component
			// Whenever we handle a bHiddenInGame, we always combine it with the actor's bHidden
			else if (PropertyPath == UnrealIdentifiers::HiddenPropertyName || PropertyPath == UnrealIdentifiers::HiddenInGamePropertyName)
			{
				if (pxr::UsdGeomImageable Imageable{UsdPrim})
				{
					Attr = Imageable.CreateVisibilityAttr();
					if (Attr)
					{
						Result.BoolWriter = [Imageable, Attr](bool UEValue, double UsdTimeCode)
						{
							if (UEValue)
							{
								// We have to do both here as MakeVisible will ensure we also flip any parent prims,
								// and setting the attribute will ensure we write a timeSample. Otherwise if MakeVisible
								// finds that the prim should already be visible due to a stronger opinion, it won't write anything
								Attr.Set<pxr::TfToken>(pxr::UsdGeomTokens->inherited, UsdTimeCode);
								Imageable.MakeVisible(UsdTimeCode);
							}
							else
							{
								Attr.Set<pxr::TfToken>(pxr::UsdGeomTokens->invisible, UsdTimeCode);
								Imageable.MakeInvisible(UsdTimeCode);
							}
						};
					}
				}
			}
		}

		if (pxr::UsdGeomCamera Camera{UsdPrim})
		{
			bool bInvertValue = false;
			bool bConvertDistance = true;
			TOptional<int32> ClippingPlaneIndex;

			if (PropertyPath == UnrealIdentifiers::CurrentFocalLengthPropertyName)
			{
				Attr = Camera.CreateFocalLengthAttr();
			}
			else if (PropertyPath == UnrealIdentifiers::ManualFocusDistancePropertyName)
			{
				Attr = Camera.CreateFocusDistanceAttr();
			}
			else if (PropertyPath == UnrealIdentifiers::CurrentAperturePropertyName)
			{
				bConvertDistance = false;
				Attr = Camera.CreateExposureFStopAttr();
			}
			else if (PropertyPath == UnrealIdentifiers::SensorWidthPropertyName)
			{
				Attr = Camera.CreateHorizontalApertureAttr();
			}
			else if (PropertyPath == UnrealIdentifiers::SensorHeightPropertyName)
			{
				Attr = Camera.CreateVerticalApertureAttr();
			}
			else if (PropertyPath == UnrealIdentifiers::SensorHorizontalOffsetPropertyName)
			{
				Attr = Camera.CreateHorizontalApertureOffsetAttr();
			}
			else if (PropertyPath == UnrealIdentifiers::SensorVerticalOffsetPropertyName)
			{
				Attr = Camera.CreateVerticalApertureOffsetAttr();
			}
			else if (PropertyPath == UnrealIdentifiers::ExposureCompensationPropertyName)
			{
				bConvertDistance = false;
				Attr = Camera.CreateExposureAttr();
			}
			else if (PropertyPath == UnrealIdentifiers::OrthoFarClipPlanePropertyName)
			{
				Attr = Camera.CreateClippingRangeAttr();
				ClippingPlaneIndex = 1;
			}
			else if (PropertyPath == UnrealIdentifiers::OrthoNearClipPlanePropertyName)
			{
				Attr = Camera.CreateClippingRangeAttr();
				ClippingPlaneIndex = 0;
			}
			else if (PropertyPath == UnrealIdentifiers::CustomNearClipppingPlanePropertyName)
			{
				Attr = Camera.CreateClippingRangeAttr();
				ClippingPlaneIndex = 0;
			}
			else if (PropertyPath == UnrealIdentifiers::DepthOfFieldFstopPropertyName)
			{
				bConvertDistance = false;
				Attr = Camera.CreateFStopAttr();
			}
			else if (PropertyPath == UnrealIdentifiers::CameraISOPropertyName)
			{
				bConvertDistance = false;
				Attr = Camera.CreateExposureIsoAttr();
			}
			else if (PropertyPath == UnrealIdentifiers::CameraShutterSpeedPropertyName)
			{
				bConvertDistance = false;
				bInvertValue = true;
				Attr = Camera.CreateExposureTimeAttr();
			}

			// TODO: Write out animation track for camera projection? Would need a new custom Enum writer type

			if (Attr)
			{
				if (ClippingPlaneIndex.IsSet() && ClippingPlaneIndex.GetValue() < 2)
				{
					Result.FloatWriter = [Attr, StageInfo, Index = ClippingPlaneIndex.GetValue()](float UEValue, double UsdTimeCode) mutable
					{
						// This is going to correspond to one track, from either near/far planes. So here we first read the
						// current USD value to use as the base, and then just set either near/far
						pxr::GfVec2f Value{1, 1000000};	   // Default values, copied directly from the schema as a fallback
						Attr.Get<pxr::GfVec2f>(&Value);

						Value[Index] = UnrealToUsd::ConvertDistance(StageInfo, UEValue);

						Attr.Set(Value, UsdTimeCode);
					};
				}
				else if (bConvertDistance)
				{
					Result.FloatWriter = [Attr, StageInfo](float UEValue, double UsdTimeCode) mutable
					{
						Attr.Set(UnrealToUsd::ConvertDistance(StageInfo, UEValue), UsdTimeCode);
					};
				}
				else if (bInvertValue)
				{
					Result.FloatWriter = [Attr](float UEValue, double UsdTimeCode) mutable
					{
						Attr.Set(UEValue != 0.0f ? (1.0f / UEValue) : 0.0f, UsdTimeCode);
					};
				}
				else
				{
					Result.FloatWriter = [Attr](float UEValue, double UsdTimeCode) mutable
					{
						Attr.Set(UEValue, UsdTimeCode);
					};
				}
			}
		}
		else if (pxr::UsdLuxLightAPI LightAPI{Prim})
		{
			if (PropertyPath == UnrealIdentifiers::LightColorPropertyName)
			{
				Attr = LightAPI.GetColorAttr();
				if (Attr)
				{
					Result.ColorWriter = [Attr](const FLinearColor& UEValue, double UsdTimeCode)
					{
						pxr::GfVec4f Vec4 = UnrealToUsd::ConvertColor(UEValue);
						Attr.Set(pxr::GfVec3f{Vec4[0], Vec4[1], Vec4[2]}, UsdTimeCode);
					};
				}
			}
			else if (PropertyPath == UnrealIdentifiers::UseTemperaturePropertyName)
			{
				Attr = LightAPI.GetEnableColorTemperatureAttr();
				if (Attr)
				{
					Result.BoolWriter = [Attr](bool UEValue, double UsdTimeCode)
					{
						Attr.Set(UEValue, UsdTimeCode);
					};
				}
			}
			else if (PropertyPath == UnrealIdentifiers::TemperaturePropertyName)
			{
				Attr = LightAPI.GetColorTemperatureAttr();
				if (Attr)
				{
					Result.FloatWriter = [Attr](float UEValue, double UsdTimeCode)
					{
						Attr.Set(UEValue, UsdTimeCode);
					};
				}
			}

			else if (pxr::UsdLuxSphereLight SphereLight{UsdPrim})
			{
				if (PropertyPath == UnrealIdentifiers::SourceRadiusPropertyName)
				{
					Attr = SphereLight.GetRadiusAttr();
					if (Attr)
					{
						Result.FloatWriter = [Attr, StageInfo](float UEValue, double UsdTimeCode)
						{
							Attr.Set(UnrealToUsd::ConvertDistance(StageInfo, UEValue), UsdTimeCode);
						};
					}
				}

				// Spot light
				else if (UsdPrim.HasAPI<pxr::UsdLuxShapingAPI>())
				{
					pxr::UsdLuxShapingAPI ShapingAPI{UsdPrim};

					if (PropertyPath == UnrealIdentifiers::IntensityPropertyName)
					{
						Attr = SphereLight.GetIntensityAttr();
						pxr::UsdAttribute RadiusAttr = SphereLight.GetRadiusAttr();
						pxr::UsdAttribute ConeAngleAttr = ShapingAPI.GetShapingConeAngleAttr();
						pxr::UsdAttribute ConeSoftnessAttr = ShapingAPI.GetShapingConeSoftnessAttr();

						// Always clear exposure because we'll put all of our "light intensity" on the intensity attr and assume exposure
						// is zero, as we can't manipulate something like that exposure directly from UE anyway
						if (pxr::UsdAttribute ExposureAttr = SphereLight.GetExposureAttr())
						{
							ExposureAttr.Clear();
						}

						// For now we'll assume the light intensity units are constant and the user doesn't have any light intensity unit tracks...
						ELightUnits Units = ELightUnits::Nits;
						if (const ULocalLightComponent* LightComponent = Cast<const ULocalLightComponent>(&Component))
						{
							Units = LightComponent->IntensityUnits;
						}

						if (Attr && RadiusAttr && ConeAngleAttr && ConeSoftnessAttr)
						{
							Result.FloatWriter =
								[Attr, RadiusAttr, ConeAngleAttr, ConeSoftnessAttr, StageInfo, Units](float UEIntensity, double UsdTimeCode)
							{
								const float UsdConeAngle = UsdUtils::GetUsdValue<float>(ConeAngleAttr, UsdTimeCode);
								const float UsdConeSoftness = UsdUtils::GetUsdValue<float>(ConeSoftnessAttr, UsdTimeCode);
								const float UsdRadius = UsdUtils::GetUsdValue<float>(RadiusAttr, UsdTimeCode);

								float InnerConeAngle = 0.0f;
								float OuterConeAngle = 1.0f;
								UsdToUnreal::ConvertSpotLightConeAngles(UsdConeAngle, UsdConeSoftness, OuterConeAngle, InnerConeAngle);

								const float RadiusMeters = UsdToUnreal::ConvertDistance(StageInfo, UsdRadius) / 100.0f;
								const float AreaSqMeters = UsdUtils::GetSpotLightArea(RadiusMeters);
								const float SolidAngle = UsdUtils::GetSpotLightSolidAngle(OuterConeAngle, InnerConeAngle);
								const float UsdIntensity = UsdUtils::ConvertIntensityToNits(UEIntensity, SolidAngle, AreaSqMeters, Units);

								Attr.Set(UsdIntensity, UsdTimeCode);
							};
						}
					}
					else if (PropertyPath == UnrealIdentifiers::OuterConeAnglePropertyName)
					{
						Attr = ShapingAPI.GetShapingConeAngleAttr();
						if (Attr)
						{
							// InnerConeAngle is calculated based on ConeAngleAttr, so we need to refresh it
							OutPropertyPathsToRefresh.Add(UnrealIdentifiers::InnerConeAnglePropertyName);

							Result.FloatWriter = [Attr](float UEValue, double UsdTimeCode)
							{
								Attr.Set(UEValue, UsdTimeCode);
							};
						}
					}
					else if (PropertyPath == UnrealIdentifiers::InnerConeAnglePropertyName)
					{
						Attr = ShapingAPI.GetShapingConeSoftnessAttr();
						pxr::UsdAttribute ConeAngleAttr = ShapingAPI.GetShapingConeAngleAttr();

						if (ConeAngleAttr && Attr)
						{
							Result.FloatWriter = [Attr, ConeAngleAttr](float UEInnerConeAngle, double UsdTimeCode)
							{
								// Awkwardly, to write the cone softness we need not only the inner cone angle, but
								// the outer cone angle too. We know that the outer cone angle matches the USD cone angle,
								// so here we just read the angle from USD
								float UsdConeAngle = UsdUtils::GetUsdValue<float>(ConeAngleAttr, UsdTimeCode);
								float OuterConeAngle = UsdConeAngle;

								float UsdConeSoftness = 0.0f;
								UnrealToUsd::ConvertSpotLightConeAngles(OuterConeAngle, UEInnerConeAngle, UsdConeAngle, UsdConeSoftness);

								Attr.Set(UsdConeSoftness, UsdTimeCode);
							};
						}
					}
				}
				// Just a point light
				else
				{
					if (PropertyPath == UnrealIdentifiers::IntensityPropertyName)
					{
						Attr = SphereLight.GetIntensityAttr();
						pxr::UsdAttribute RadiusAttr = SphereLight.GetRadiusAttr();

						// Always clear exposure because we'll put all of our "light intensity" on the intensity attr and assume exposure
						// is zero, as we can't manipulate something like that exposure directly from UE anyway
						if (pxr::UsdAttribute ExposureAttr = SphereLight.GetExposureAttr())
						{
							ExposureAttr.Clear();
						}

						// For now we'll assume the light intensity units are constant and the user doesn't have any light intensity unit tracks...
						ELightUnits Units = ELightUnits::Nits;
						if (const ULocalLightComponent* LightComponent = Cast<const ULocalLightComponent>(&Component))
						{
							Units = LightComponent->IntensityUnits;
						}

						if (Attr && RadiusAttr)
						{
							Result.FloatWriter = [Attr, RadiusAttr, StageInfo, Units](float UEIntensity, double UsdTimeCode)
							{
								const float UsdRadius = UsdUtils::GetUsdValue<float>(RadiusAttr, UsdTimeCode);

								const float RadiusMeters = UsdToUnreal::ConvertDistance(StageInfo, UsdRadius) / 100.0f;
								const float SolidAngle = UsdUtils::GetPointLightSolidAngle();
								const float AreaSqMeters = UsdUtils::GetPointLightArea(RadiusMeters);
								const float UsdIntensity = UsdUtils::ConvertIntensityToNits(UEIntensity, SolidAngle, AreaSqMeters, Units);

								Attr.Set(UsdIntensity, UsdTimeCode);
							};
						}
					}
				}
			}
			else if (pxr::UsdLuxRectLight RectLight{UsdPrim})
			{
				if (PropertyPath == UnrealIdentifiers::SourceWidthPropertyName)
				{
					Attr = RectLight.GetWidthAttr();
					if (Attr)
					{
						Result.FloatWriter = [Attr, StageInfo](float UEValue, double UsdTimeCode)
						{
							Attr.Set(UnrealToUsd::ConvertDistance(StageInfo, UEValue), UsdTimeCode);
						};
					}
				}
				else if (PropertyPath == UnrealIdentifiers::SourceHeightPropertyName)
				{
					Attr = RectLight.GetHeightAttr();
					if (Attr)
					{
						Result.FloatWriter = [Attr, StageInfo](float UEValue, double UsdTimeCode)
						{
							Attr.Set(UnrealToUsd::ConvertDistance(StageInfo, UEValue), UsdTimeCode);
						};
					}
				}
				else if (PropertyPath == UnrealIdentifiers::IntensityPropertyName)
				{
					Attr = RectLight.GetIntensityAttr();
					pxr::UsdAttribute WidthAttr = RectLight.GetWidthAttr();
					pxr::UsdAttribute HeightAttr = RectLight.GetHeightAttr();

					// Always clear exposure because we'll put all of our "light intensity" on the intensity attr and assume exposure
					// is zero, as we can't manipulate something like that exposure directly from UE anyway
					if (pxr::UsdAttribute ExposureAttr = RectLight.GetExposureAttr())
					{
						ExposureAttr.Clear();
					}

					// For now we'll assume the light intensity units are constant and the user doesn't have any light intensity unit tracks...
					ELightUnits Units = ELightUnits::Nits;
					if (const ULocalLightComponent* LightComponent = Cast<const ULocalLightComponent>(&Component))
					{
						Units = LightComponent->IntensityUnits;
					}

					if (Attr && WidthAttr && HeightAttr)
					{
						Result.FloatWriter = [Attr, WidthAttr, HeightAttr, StageInfo, Units](float UEIntensity, double UsdTimeCode)
						{
							const float UsdWidth = UsdUtils::GetUsdValue<float>(WidthAttr, UsdTimeCode);
							const float UsdHeight = UsdUtils::GetUsdValue<float>(HeightAttr, UsdTimeCode);

							const float WidthMeters = UsdToUnreal::ConvertDistance(StageInfo, UsdWidth) / 100.0f;
							const float HeightMeters = UsdToUnreal::ConvertDistance(StageInfo, UsdHeight) / 100.0f;
							const float SolidAngle = UsdUtils::GetRectLightSolidAngle();
							const float AreaSqMeters = UsdUtils::GetRectLightArea(WidthMeters, HeightMeters);
							const float UsdIntensity = UsdUtils::ConvertIntensityToNits(UEIntensity, SolidAngle, AreaSqMeters, Units);

							Attr.Set(UsdIntensity, UsdTimeCode);
						};
					}
				}
			}
			else if (pxr::UsdLuxDiskLight DiskLight{UsdPrim})
			{
				if (PropertyPath == UnrealIdentifiers::SourceWidthPropertyName || PropertyPath == UnrealIdentifiers::SourceHeightPropertyName)
				{
					Attr = DiskLight.GetRadiusAttr();
					if (Attr)
					{
						// Resync the other to match this one after we bake it, effectively always enforcing the UE rect light into a square shape
						OutPropertyPathsToRefresh.Add(
							PropertyPath == UnrealIdentifiers::SourceWidthPropertyName ? UnrealIdentifiers::SourceHeightPropertyName
																					   : UnrealIdentifiers::SourceWidthPropertyName
						);

						Result.FloatWriter = [Attr, StageInfo](float UEValue, double UsdTimeCode)
						{
							float UEArea = UEValue * UEValue;
							float UERadius = FMath::Sqrt(UEArea / PI);
							Attr.Set(UnrealToUsd::ConvertDistance(StageInfo, UERadius), UsdTimeCode);
						};
					}
				}
				else if (PropertyPath == UnrealIdentifiers::IntensityPropertyName)
				{
					Attr = DiskLight.GetIntensityAttr();
					pxr::UsdAttribute RadiusAttr = DiskLight.GetRadiusAttr();

					// Always clear exposure because we'll put all of our "light intensity" on the intensity attr and assume exposure
					// is zero, as we can't manipulate something like that exposure directly from UE anyway
					if (pxr::UsdAttribute ExposureAttr = DiskLight.GetExposureAttr())
					{
						ExposureAttr.Clear();
					}

					// For now we'll assume the light intensity units are constant and the user doesn't have any light intensity unit tracks...
					ELightUnits Units = ELightUnits::Nits;
					if (const ULocalLightComponent* LightComponent = Cast<const ULocalLightComponent>(&Component))
					{
						Units = LightComponent->IntensityUnits;
					}

					if (Attr && RadiusAttr)
					{
						Result.FloatWriter = [Attr, RadiusAttr, StageInfo, Units](float UEIntensity, double UsdTimeCode)
						{
							const float UsdRadius = UsdUtils::GetUsdValue<float>(RadiusAttr, UsdTimeCode);

							const float RadiusMeters = UsdToUnreal::ConvertDistance(StageInfo, UsdRadius) / 100.0f;
							const float SolidAngle = UsdUtils::GetDiskLightSolidAngle();
							const float AreaSqMeters = UsdUtils::GetDiskLightArea(RadiusMeters);
							const float UsdIntensity = UsdUtils::ConvertIntensityToNits(UEIntensity, SolidAngle, AreaSqMeters, Units);

							Attr.Set(UsdIntensity, UsdTimeCode);
						};
					}
				}
			}
			else if (pxr::UsdLuxDistantLight DistantLight{UsdPrim})
			{
				if (PropertyPath == UnrealIdentifiers::LightSourceAnglePropertyName)
				{
					Attr = DistantLight.GetAngleAttr();
					if (Attr)
					{
						Result.FloatWriter = [Attr](float UEValue, double UsdTimeCode)
						{
							Attr.Set(UEValue, UsdTimeCode);
						};
					}
				}
				else if (PropertyPath == UnrealIdentifiers::IntensityPropertyName)
				{
					Attr = DistantLight.GetIntensityAttr();

					// Always clear exposure because we'll put all of our "light intensity" on the intensity attr and assume exposure
					// is zero, as we can't manipulate something like that exposure directly from UE anyway
					if (pxr::UsdAttribute ExposureAttr = DistantLight.GetExposureAttr())
					{
						ExposureAttr.Clear();
					}

					if (Attr)
					{
						Result.FloatWriter = [Attr](float UEIntensity, double UsdTimeCode)
						{
							// Both UE and USD exclusively use lux for directional light intensity
							Attr.Set(UEIntensity, UsdTimeCode);
						};
					}
				}
			}
			else if (pxr::UsdLuxDomeLight DomeLight{UsdPrim})
			{
				if (PropertyPath == UnrealIdentifiers::IntensityPropertyName)
				{
					Attr = DomeLight.GetIntensityAttr();

					// Always clear exposure because we'll put all of our "light intensity" on the intensity attr and assume exposure
					// is zero, as we can't manipulate something like that exposure directly from UE anyway
					if (pxr::UsdAttribute ExposureAttr = DomeLight.GetExposureAttr())
					{
						ExposureAttr.Clear();
					}

					if (Attr)
					{
						Result.FloatWriter = [Attr](float UEIntensity, double UsdTimeCode)
						{
							// Both UE and USD use a flat multiplier for dome/skylight intensity
							Attr.Set(UEIntensity, UsdTimeCode);
						};
					}
				}
			}
		}
		else if (const UHeterogeneousVolumeComponent* VolumeComponent = Cast<const UHeterogeneousVolumeComponent>(&Component))
		{
			if (Track.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UHeterogeneousVolumeComponent, Frame))
			{
				const TArray<FString>* TimeSamplePaths = nullptr;
				const TArray<FString>* SourceOpenVDBAssetPrimPaths = nullptr;

				const int32 ElementIndex = 0;
				if (UMaterialInterface* CurrentMaterial = VolumeComponent->GetMaterial(ElementIndex))
				{
					TArray<FMaterialParameterInfo> ParameterInfo;
					TArray<FGuid> ParameterIds;
					CurrentMaterial->GetAllSparseVolumeTextureParameterInfo(ParameterInfo, ParameterIds);

					if (ParameterInfo.Num() > 0)
					{
						// Only care about animation on first SVT parameter
						const FMaterialParameterInfo& Info = ParameterInfo[0];
						USparseVolumeTexture* SparseVolumeTexture = nullptr;
						if (CurrentMaterial->GetSparseVolumeTextureParameterValue(Info, SparseVolumeTexture) && SparseVolumeTexture)
						{
							if (SparseVolumeTexture->GetNumFrames() > 1)
							{
								if (UUsdSparseVolumeTextureAssetUserData* UserData = Cast<UUsdSparseVolumeTextureAssetUserData>(
										UsdUnreal::ObjectUtils::GetAssetUserData(SparseVolumeTexture)
									))
								{
									SourceOpenVDBAssetPrimPaths = &UserData->SourceOpenVDBAssetPrimPaths;
									TimeSamplePaths = &UserData->TimeSamplePaths;
								}
							}
						}
					}
				}

				// Collect the Attrs we'll need to write out to.
				// Realistically this is a single Attr, but there could be more in case multiple OpenVDBAsset prims refer to the
				// exact same VDB file paths
				if (SourceOpenVDBAssetPrimPaths && SourceOpenVDBAssetPrimPaths->Num() > 0)
				{
					Attrs.Reset(SourceOpenVDBAssetPrimPaths->Num());

					for (const FString& OpenVDBPrimPath : *SourceOpenVDBAssetPrimPaths)
					{
						pxr::UsdPrim OpenVDBPrim = UsdStage->GetPrimAtPath(UnrealToUsd::ConvertPath(*OpenVDBPrimPath).Get());
						if (pxr::UsdVolOpenVDBAsset OpenVDBAsset{OpenVDBPrim})
						{
							Attrs.Add(OpenVDBAsset.CreateFilePathAttr());
						}
					}

					Attr = Attrs[0];
				}

				if (TimeSamplePaths)
				{
					TArray<pxr::SdfAssetPath> FrameIndexToPath;
					FrameIndexToPath.Reserve(TimeSamplePaths->Num());
					for (const FString& TimeSamplePath : *TimeSamplePaths)
					{
						FrameIndexToPath.Add(pxr::SdfAssetPath{UnrealToUsd::ConvertString(*TimeSamplePath).Get()});
					}

					Result.FloatWriter = [Attrs,
										  FrameIndexToPath = MoveTemp(FrameIndexToPath),
										  LastPath = TUsdStore<pxr::SdfAssetPath>{}](float UEValue, double UsdTimeCode) mutable
					{
						// The UEValue here corresponds to a frame index into the SVT (with constant interpolation).
						// Regardless of what the change was on the track, we can assume here that our TimeSampleX arrays are up
						// to date with the generated SVT. This means we essentially just need to author a timeSample at UsdTimeCode
						// that points at the file path that corresponds to that frame

						const int32 FrameIndex = FMath::FloorToInt32(UEValue);
						const pxr::SdfAssetPath& TimeSamplePath = FrameIndexToPath[FrameIndex];

						// This check prevents us from writing the same identical path on every single bake tick
						if (TimeSamplePath != LastPath.Get())
						{
							for (const pxr::UsdAttribute& Attr : Attrs)
							{
								Attr.Set(TimeSamplePath, UsdTimeCode);
							}
						}

						LastPath = TimeSamplePath;
					};
				}
			}
		}
		else if (const UUsdDrawModeComponent* DrawModeComponent = Cast<const UUsdDrawModeComponent>(&Component))
		{
			Attr = {};
			if (pxr::UsdGeomModelAPI GeomModelAPI{UsdPrim})
			{
				// For whatever reason there is no CreateExtentsHintAttr, so here we copy how the
				// attr is created from within SetExtentsHint
				Attr = UsdPrim.CreateAttribute(pxr::UsdGeomTokens->extentsHint, pxr::SdfValueTypeNames->Float3Array, /* custom = */ false);
			}
			else if (pxr::UsdGeomBoundable Boundable{UsdPrim})
			{
				Attr = Boundable.GetExtentAttr();
			}

			// If we still don't have an attr try applying the schema.
			// Not entirely sure how this can possibly happen at this point, but this is more for "parity" with ConvertDrawModeComponent
			if (!Attr)
			{
				if (pxr::UsdGeomModelAPI GeomModelAPI = pxr::UsdGeomModelAPI::Apply(UsdPrim))
				{
					Attr = UsdPrim.CreateAttribute(pxr::UsdGeomTokens->extentsHint, pxr::SdfValueTypeNames->Float3Array, /* custom = */ false);
				}
			}

			if (Attr)
			{
				Result.TwoVectorWriter = [Attr, StageInfo](const FVector& UEMinValue, const FVector& UEMaxValue, double UsdTimeCode)
				{
					FScopedUsdAllocs Allocs;

					pxr::GfVec3f UEBoundsMinUsdSpace = UnrealToUsd::ConvertVectorFloat(StageInfo, UEMinValue);
					pxr::GfVec3f UEBoundsMaxUsdSpace = UnrealToUsd::ConvertVectorFloat(StageInfo, UEMaxValue);
					pxr::GfVec3f UsdMin{
						FMath::Min(UEBoundsMinUsdSpace[0], UEBoundsMaxUsdSpace[0]),
						FMath::Min(UEBoundsMinUsdSpace[1], UEBoundsMaxUsdSpace[1]),
						FMath::Min(UEBoundsMinUsdSpace[2], UEBoundsMaxUsdSpace[2])
					};
					pxr::GfVec3f UsdMax{
						FMath::Max(UEBoundsMinUsdSpace[0], UEBoundsMaxUsdSpace[0]),
						FMath::Max(UEBoundsMinUsdSpace[1], UEBoundsMaxUsdSpace[1]),
						FMath::Max(UEBoundsMinUsdSpace[2], UEBoundsMaxUsdSpace[2])
					};
					pxr::VtArray<pxr::GfVec3f> Extents{UsdMin, UsdMax};

					Attr.Set(Extents, UsdTimeCode);
				};
			}
		}
	}

	for (const pxr::UsdAttribute& SomeAttr : Attrs)
	{
		// Note that we must do this only after the change block is destroyed!
		// This is important because if we don't have spec for this attribute on the current edit target, we're relying
		// on the previous code to create it, and we need to let USD emit its internal notices and fully commit the
		// "attribute creation" spec first. This because PrepareAttributeForAnimationBake will call NotifyIfOverriddenOpinion,
		// which will go through the attribute's spec stack and consider our attribute overriden if it finds a stronger
		// opinion than the one on the edit target. Well if our own spec hasn't been created yet it will misfire when
		// it runs into any other spec
		UE::USDPrimConversion::Private::PrepareAttributeForAnimationBake(SomeAttr);
	}

	return Result;
}

bool UnrealToUsd::ConvertXformable(
	const UMovieScene3DTransformTrack& MovieSceneTrack,
	pxr::UsdPrim& UsdPrim,
	const FMovieSceneSequenceTransform& SequenceTransform
)
{
	if (!UsdPrim)
	{
		return false;
	}

	const FUsdStageInfo StageInfo(UsdPrim.GetStage());

	const UMovieScene* MovieScene = MovieSceneTrack.GetTypedOuter<UMovieScene>();
	if (!MovieScene)
	{
		return false;
	}

	FScopedUsdAllocs UsdAllocs;

	pxr::UsdGeomXformable Xformable(UsdPrim);
	if (!Xformable)
	{
		return false;
	}

	UMovieScene3DTransformSection* TransformSection = Cast<UMovieScene3DTransformSection>(
		const_cast<UMovieScene3DTransformTrack&>(MovieSceneTrack).FindSection(0)
	);
	if (!TransformSection)
	{
		return false;
	}

	const TRange<FFrameNumber> PlaybackRange = MovieScene->GetPlaybackRange();
	const FFrameRate Resolution = MovieScene->GetTickResolution();
	const FFrameRate DisplayRate = MovieScene->GetDisplayRate();

	const double StageTimeCodesPerSecond = UsdPrim.GetStage()->GetTimeCodesPerSecond();
	const FFrameRate StageFrameRate(StageTimeCodesPerSecond, 1);

	FMovieSceneInverseSequenceTransform SequenceToRootTransform = SequenceTransform.Inverse();

	auto EvaluateChannel = [&PlaybackRange, &Resolution, &DisplayRate, &SequenceToRootTransform](
							   const FMovieSceneDoubleChannel* Channel,
							   double DefaultValue
						   ) -> TArray<TPair<FFrameNumber, float>>
	{
		TArray<TPair<FFrameNumber, float>> Values;

		if (PlaybackRange.HasLowerBound() && PlaybackRange.HasUpperBound())
		{
			const FFrameTime Interval = FFrameRate::TransformTime(1, DisplayRate, Resolution);
			const FFrameNumber StartFrame = UE::MovieScene::DiscreteInclusiveLower(PlaybackRange);
			const FFrameNumber EndFrame = UE::MovieScene::DiscreteExclusiveUpper(PlaybackRange);

			for (FFrameTime EvalTime = StartFrame; EvalTime < EndFrame; EvalTime += Interval)
			{
				FFrameNumber KeyTime = FFrameRate::Snap(EvalTime, Resolution, DisplayRate).FloorToFrame();

				TOptional<FFrameTime> RootTime = SequenceToRootTransform.TryTransformTime(KeyTime);
				if (!RootTime)
				{
					continue;
				}

				double Result = DefaultValue;
				if (Channel)
				{
					Result = Channel->GetDefault().Get(DefaultValue);
					Channel->Evaluate(KeyTime, Result);
				}

				Values.Emplace(RootTime->GetFrame(), Result);
			}
		}

		return Values;
	};

	TArrayView<FMovieSceneDoubleChannel*> Channels = TransformSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();
	check(Channels.Num() >= 9);

	auto GetChannel = [&Channels](const int32 ChannelIndex) -> const FMovieSceneDoubleChannel*
	{
		if (Channels.IsValidIndex(ChannelIndex))
		{
			return Channels[ChannelIndex];
		}
		else
		{
			return nullptr;
		}
	};

	// Translation
	TArray<TPair<FFrameNumber, float>> LocationValuesX = EvaluateChannel(GetChannel(0), 0.0);
	TArray<TPair<FFrameNumber, float>> LocationValuesY = EvaluateChannel(GetChannel(1), 0.0);
	TArray<TPair<FFrameNumber, float>> LocationValuesZ = EvaluateChannel(GetChannel(2), 0.0);

	// Rotation
	TArray<TPair<FFrameNumber, float>> RotationValuesX = EvaluateChannel(GetChannel(3), 0.0);
	TArray<TPair<FFrameNumber, float>> RotationValuesY = EvaluateChannel(GetChannel(4), 0.0);
	TArray<TPair<FFrameNumber, float>> RotationValuesZ = EvaluateChannel(GetChannel(5), 0.0);

	// Scale
	TArray<TPair<FFrameNumber, float>> ScaleValuesX = EvaluateChannel(GetChannel(6), 1.0);
	TArray<TPair<FFrameNumber, float>> ScaleValuesY = EvaluateChannel(GetChannel(7), 1.0);
	TArray<TPair<FFrameNumber, float>> ScaleValuesZ = EvaluateChannel(GetChannel(8), 1.0);

	bool bIsDataOutOfSync = false;
	{
		int32 ValueIndex = 0;

		FFrameTime UsdStartTime = FFrameRate::TransformTime(PlaybackRange.GetLowerBoundValue(), Resolution, StageFrameRate);
		FFrameTime UsdEndTime = FFrameRate::TransformTime(PlaybackRange.GetUpperBoundValue(), Resolution, StageFrameRate);

		std::vector<double> UsdTimeSamples;
		if (LocationValuesX.Num() > 0 || (Xformable.GetTimeSamples(&UsdTimeSamples) && UsdTimeSamples.size() > 0))
		{
			bIsDataOutOfSync = (UsdTimeSamples.size() != LocationValuesX.Num());

			if (!bIsDataOutOfSync)
			{
				for (const TPair<FFrameNumber, float>& Value : LocationValuesX)
				{
					FFrameTime UsdFrameTime = FFrameRate::TransformTime(Value.Key, Resolution, StageFrameRate);

					FVector Location(LocationValuesX[ValueIndex].Value, LocationValuesY[ValueIndex].Value, LocationValuesZ[ValueIndex].Value);
					FRotator Rotation(RotationValuesY[ValueIndex].Value, RotationValuesZ[ValueIndex].Value, RotationValuesX[ValueIndex].Value);
					FVector Scale(ScaleValuesX[ValueIndex].Value, ScaleValuesY[ValueIndex].Value, ScaleValuesZ[ValueIndex].Value);

					FTransform Transform(Rotation, Location, Scale);
					pxr::GfMatrix4d UsdTransform = UnrealToUsd::ConvertTransform(StageInfo, Transform);

					pxr::GfMatrix4d UsdMatrix;
					bool bResetXFormStack = false;
					Xformable.GetLocalTransformation(&UsdMatrix, &bResetXFormStack, UsdFrameTime.AsDecimal());

					if (!pxr::GfIsClose(UsdMatrix, UsdTransform, THRESH_POINTS_ARE_NEAR))
					{
						bIsDataOutOfSync = true;
						break;
					}

					++ValueIndex;
				}
			}
		}
	}

	if (bIsDataOutOfSync)
	{
		if (pxr::UsdGeomXformOp TransformOp = UE::USDPrimConversion::Private::ForceMatrixXform(Xformable))
		{
			TransformOp.GetAttr().Clear();	  // Clear existing transform data
		}

		pxr::SdfChangeBlock ChangeBlock;

		// Compensate different orientation for light or camera components
		FTransform CameraCompensation = FTransform::Identity;
		if (UsdPrim.IsA<pxr::UsdGeomCamera>() || UsdPrim.HasAPI<pxr::UsdLuxLightAPI>())
		{
			CameraCompensation = FTransform(FRotator(0.0f, 90.0f, 0.0f));

			if (StageInfo.UpAxis == EUsdUpAxis::ZAxis)
			{
				CameraCompensation *= FTransform(FRotator(90.0f, 0.0f, 0.0f));
			}
		}

		// Invert compensation applied to parent if it's a light or camera component
		FTransform InverseCameraCompensation = FTransform::Identity;
		if (pxr::UsdPrim ParentPrim = UsdPrim.GetParent())
		{
			if (ParentPrim.IsA<pxr::UsdGeomCamera>() || ParentPrim.HasAPI<pxr::UsdLuxLightAPI>())
			{
				InverseCameraCompensation = FTransform(FRotator(0.0f, 90.f, 0.0f));

				if (StageInfo.UpAxis == EUsdUpAxis::ZAxis)
				{
					InverseCameraCompensation *= FTransform(FRotator(90.0f, 0.f, 0.0f));
				}
			}
		}

		int32 ValueIndex = 0;
		for (const TPair<FFrameNumber, float>& Value : LocationValuesX)
		{
			FFrameTime UsdFrameTime = FFrameRate::TransformTime(Value.Key, Resolution, StageFrameRate);

			FVector Location(LocationValuesX[ValueIndex].Value, LocationValuesY[ValueIndex].Value, LocationValuesZ[ValueIndex].Value);
			FRotator Rotation(RotationValuesY[ValueIndex].Value, RotationValuesZ[ValueIndex].Value, RotationValuesX[ValueIndex].Value);
			FVector Scale(ScaleValuesX[ValueIndex].Value, ScaleValuesY[ValueIndex].Value, ScaleValuesZ[ValueIndex].Value);

			FTransform Transform(Rotation, Location, Scale);
			ConvertXformable(CameraCompensation * Transform * InverseCameraCompensation.Inverse(), UsdPrim, UsdFrameTime.AsDecimal());

			++ValueIndex;
		}
	}

	return true;
}

TArray<UE::FUsdAttribute> UsdUtils::GetAttributesForProperty(const UE::FUsdPrim& Prim, const FName& PropertyPath)
{
	using namespace UnrealIdentifiers;

	FScopedUsdAllocs Allocs;
	pxr::UsdPrim UsdPrim{Prim};

	// Common attributes
	if (PropertyPath == TransformPropertyName)
	{
		TArray<UE::FUsdAttribute> Attrs;

		if (pxr::UsdGeomXformable Xformable{UsdPrim})
		{
			Attrs.Emplace(Xformable.GetXformOpOrderAttr());

			bool bResetsXformStack = false;
			std::vector<pxr::UsdGeomXformOp> Ops = Xformable.GetOrderedXformOps(&bResetsXformStack);
			for (const pxr::UsdGeomXformOp& Op : Ops)
			{
				Attrs.Emplace(Op.GetAttr());
			}
		}

		// This function returns all attributes that can affect a property. For the Gprim primitives and the Transform property
		// this will include their heights, widths, etc. too, as we also handle those with just the component transform.
		if (pxr::UsdGeomCapsule Capsule{UsdPrim})
		{
			Attrs.Emplace(Capsule.GetHeightAttr());
			Attrs.Emplace(Capsule.GetRadiusAttr());
		}
		if (pxr::UsdGeomCapsule_1 Capsule_1{UsdPrim})
		{
			Attrs.Emplace(Capsule_1.GetHeightAttr());
			Attrs.Emplace(Capsule_1.GetRadiusTopAttr());
		}
		else if (pxr::UsdGeomCone Cone{UsdPrim})
		{
			Attrs.Emplace(Cone.GetHeightAttr());
			Attrs.Emplace(Cone.GetRadiusAttr());
		}
		else if (pxr::UsdGeomCube Cube{UsdPrim})
		{
			Attrs.Emplace(Cube.GetSizeAttr());
		}
		else if (pxr::UsdGeomCylinder Cylinder{UsdPrim})
		{
			Attrs.Emplace(Cylinder.GetHeightAttr());
			Attrs.Emplace(Cylinder.GetRadiusAttr());
		}
		else if (pxr::UsdGeomCylinder_1 Cylinder_1{UsdPrim})
		{
			Attrs.Emplace(Cylinder_1.GetHeightAttr());
			Attrs.Emplace(Cylinder_1.GetRadiusTopAttr());
		}
		else if (pxr::UsdGeomPlane Plane{UsdPrim})
		{
			Attrs.Emplace(Plane.GetLengthAttr());
			Attrs.Emplace(Plane.GetWidthAttr());
		}
		else if (pxr::UsdGeomSphere Sphere{UsdPrim})
		{
			Attrs.Emplace(Sphere.GetRadiusAttr());
		}

		return Attrs;
	}
	if (PropertyPath == HiddenInGamePropertyName)
	{
		return {UE::FUsdAttribute{UsdPrim.GetAttribute(pxr::UsdGeomTokens->visibility)}};
	}

	// Camera attributes
	else if (PropertyPath == CurrentFocalLengthPropertyName)
	{
		return {UE::FUsdAttribute{UsdPrim.GetAttribute(pxr::UsdGeomTokens->focalLength)}};
	}
	else if (PropertyPath == ManualFocusDistancePropertyName)
	{
		return {UE::FUsdAttribute{UsdPrim.GetAttribute(pxr::UsdGeomTokens->focusDistance)}};
	}
	else if (PropertyPath == CurrentAperturePropertyName)
	{
		return {UE::FUsdAttribute{UsdPrim.GetAttribute(pxr::UsdGeomTokens->exposureFStop)}};
	}
	else if (PropertyPath == SensorWidthPropertyName)
	{
		return {UE::FUsdAttribute{UsdPrim.GetAttribute(pxr::UsdGeomTokens->horizontalAperture)}};
	}
	else if (PropertyPath == SensorHeightPropertyName)
	{
		return {UE::FUsdAttribute{UsdPrim.GetAttribute(pxr::UsdGeomTokens->verticalAperture)}};
	}
	else if (PropertyPath == SensorHorizontalOffsetPropertyName)
	{
		return {UE::FUsdAttribute{UsdPrim.GetAttribute(pxr::UsdGeomTokens->horizontalApertureOffset)}};
	}
	else if (PropertyPath == SensorVerticalOffsetPropertyName)
	{
		return {UE::FUsdAttribute{UsdPrim.GetAttribute(pxr::UsdGeomTokens->verticalApertureOffset)}};
	}
	else if (PropertyPath == ExposureCompensationPropertyName)
	{
		return {UE::FUsdAttribute{UsdPrim.GetAttribute(pxr::UsdGeomTokens->exposure)}};
	}
	else if (PropertyPath == ProjectionModePropertyName)
	{
		return {UE::FUsdAttribute{UsdPrim.GetAttribute(pxr::UsdGeomTokens->projection)}};
	}
	else if (PropertyPath == OrthoFarClipPlanePropertyName		  //
			 || PropertyPath == OrthoNearClipPlanePropertyName	  //
			 || PropertyPath == CustomNearClipppingPlanePropertyName)
	{
		return {UE::FUsdAttribute{UsdPrim.GetAttribute(pxr::UsdGeomTokens->clippingRange)}};
	}
	else if (PropertyPath == DepthOfFieldFstopPropertyName)
	{
		return {UE::FUsdAttribute{UsdPrim.GetAttribute(pxr::UsdGeomTokens->fStop)}};
	}
	else if (PropertyPath == CameraISOPropertyName)
	{
		return {UE::FUsdAttribute{UsdPrim.GetAttribute(pxr::UsdGeomTokens->exposureIso)}};
	}
	else if (PropertyPath == CameraShutterSpeedPropertyName)
	{
		return {UE::FUsdAttribute{UsdPrim.GetAttribute(pxr::UsdGeomTokens->exposureTime)}};
	}

	// Light attributes
	else if (PropertyPath == IntensityPropertyName)
	{
		return {
			UE::FUsdAttribute{UsdPrim.GetAttribute(pxr::UsdLuxTokens->inputsIntensity)},
			UE::FUsdAttribute{UsdPrim.GetAttribute(pxr::UsdLuxTokens->inputsExposure)}
		};
	}
	else if (PropertyPath == LightColorPropertyName)
	{
		return {UE::FUsdAttribute{UsdPrim.GetAttribute(pxr::UsdLuxTokens->inputsColor)}};
	}
	else if (PropertyPath == UseTemperaturePropertyName)
	{
		return {UE::FUsdAttribute{UsdPrim.GetAttribute(pxr::UsdLuxTokens->inputsEnableColorTemperature)}};
	}
	else if (PropertyPath == TemperaturePropertyName)
	{
		return {UE::FUsdAttribute{UsdPrim.GetAttribute(pxr::UsdLuxTokens->inputsColorTemperature)}};
	}
	else if (PropertyPath == SourceWidthPropertyName)
	{
		if (UsdPrim.IsA<pxr::UsdLuxDiskLight>())
		{
			return {UE::FUsdAttribute{UsdPrim.GetAttribute(pxr::UsdLuxTokens->inputsRadius)}};
		}
		else
		{
			return {UE::FUsdAttribute{UsdPrim.GetAttribute(pxr::UsdLuxTokens->inputsWidth)}};
		}
	}
	else if (PropertyPath == SourceHeightPropertyName)
	{
		if (UsdPrim.IsA<pxr::UsdLuxDiskLight>())
		{
			return {UE::FUsdAttribute{UsdPrim.GetAttribute(pxr::UsdLuxTokens->inputsRadius)}};
		}
		else
		{
			return {UE::FUsdAttribute{UsdPrim.GetAttribute(pxr::UsdLuxTokens->inputsHeight)}};
		}
	}
	else if (PropertyPath == SourceRadiusPropertyName)
	{
		return {UE::FUsdAttribute{UsdPrim.GetAttribute(pxr::UsdLuxTokens->inputsRadius)}};
	}
	else if (PropertyPath == OuterConeAnglePropertyName)
	{
		return {UE::FUsdAttribute{UsdPrim.GetAttribute(pxr::UsdLuxTokens->inputsShapingConeAngle)}};
	}
	else if (PropertyPath == InnerConeAnglePropertyName)
	{
		return {
			UE::FUsdAttribute{UsdPrim.GetAttribute(pxr::UsdLuxTokens->inputsShapingConeAngle)},
			UE::FUsdAttribute{UsdPrim.GetAttribute(pxr::UsdLuxTokens->inputsShapingConeSoftness)}
		};
	}
	else if (PropertyPath == LightSourceAnglePropertyName)
	{
		return {UE::FUsdAttribute{UsdPrim.GetAttribute(pxr::UsdLuxTokens->inputsAngle)}};
	}

	// Bounds component properties
	// For now we only support bounds animations, but there's nothing preventing us from supporting color/texture/draw mode animations here,
	// especially since our component works with a "dynamic mesh" approach that already rebuilds its proxy on-demand anyway
	else if (PropertyPath == GET_MEMBER_NAME_CHECKED(UUsdDrawModeComponent, BoundsMin)
			 || PropertyPath == GET_MEMBER_NAME_CHECKED(UUsdDrawModeComponent, BoundsMax))
	{
		// If we have a model API, let's direct the caller to `extentsHint` instead, as that has priority over
		// `extent` in case the model API is applied to a boundable and both are authored
		if (pxr::UsdGeomModelAPI GeomModelAPI{UsdPrim})
		{
			return {UE::FUsdAttribute{GeomModelAPI.GetExtentsHintAttr()}};
		}
		else if (pxr::UsdGeomBoundable Boundable{UsdPrim})
		{
			return {UE::FUsdAttribute(Boundable.GetExtentAttr())};
		}
	}
	else if (PropertyPath == GET_MEMBER_NAME_CHECKED(UHeterogeneousVolumeComponent, Frame))
	{
		if (pxr::UsdVolVolume Volume{Prim})
		{
			pxr::UsdStageRefPtr Stage = Prim.GetStage();

			TArray<UE::FUsdAttribute> Attrs;

			const std::map<pxr::TfToken, pxr::SdfPath>& FieldMap = Volume.GetFieldPaths();
			Attrs.Reserve(FieldMap.size());

			for (std::map<pxr::TfToken, pxr::SdfPath>::const_iterator Iter = FieldMap.cbegin(); Iter != FieldMap.cend(); ++Iter)
			{
				const pxr::SdfPath& AssetPrimPath = Iter->second;

				if (pxr::UsdVolOpenVDBAsset OpenVDBAsset{Stage->GetPrimAtPath(AssetPrimPath)})
				{
					if (pxr::UsdAttribute FilePathAttr = OpenVDBAsset.GetFilePathAttr())
					{
						Attrs.Add(UE::FUsdAttribute{FilePathAttr});
					}
				}
			}

			return Attrs;
		}
	}
	else if (PropertyPath == GET_MEMBER_NAME_CHECKED(UAudioComponent, VolumeMultiplier))
	{
		if (pxr::UsdMediaSpatialAudio Audio{Prim})
		{
			return {UE::FUsdAttribute{Audio.GetGainAttr()}};
		}
	}

	return {};
}

TArray<FName> UsdUtils::GetPropertiesForAttribute(const UE::FUsdPrim& Prim, const FString& AttrName)
{
	using namespace UnrealIdentifiers;

	// Note: This function may seem confusing and "backwards", but it is correct.
	// The logic here is essentially "If this attribute is animated, which tracks do I need to generate?"

	FScopedUsdAllocs Allocs;
	pxr::UsdPrim UsdPrim{Prim};

	pxr::TfToken AttrToken = UnrealToUsd::ConvertToken(*AttrName).Get();

	// First check the standard cases, which should be the most common
	// clang-format off
	const static std::unordered_map<pxr::TfToken, TArray<FName>, pxr::TfHash> StandardMapping = {
		{pxr::UsdGeomTokens->visibility, 					{HiddenInGamePropertyName}},

		{pxr::UsdGeomTokens->focalLength, 					{CurrentFocalLengthPropertyName}},
		{pxr::UsdGeomTokens->focusDistance, 				{ManualFocusDistancePropertyName}},
		{pxr::UsdGeomTokens->fStop, 						{DepthOfFieldFstopPropertyName}},
		{pxr::UsdGeomTokens->horizontalAperture, 			{SensorWidthPropertyName}},
		{pxr::UsdGeomTokens->verticalAperture, 				{SensorHeightPropertyName}},
		{pxr::UsdGeomTokens->horizontalApertureOffset, 		{SensorHorizontalOffsetPropertyName}},
		{pxr::UsdGeomTokens->verticalApertureOffset, 		{SensorVerticalOffsetPropertyName}},
		{pxr::UsdGeomTokens->exposure, 						{ExposureCompensationPropertyName}},
		{pxr::UsdGeomTokens->projection, 					{ProjectionModePropertyName}},
		{pxr::UsdGeomTokens->clippingRange, 				{OrthoFarClipPlanePropertyName, OrthoNearClipPlanePropertyName, CustomNearClipppingPlanePropertyName}},
		{pxr::UsdGeomTokens->exposureFStop, 				{CurrentAperturePropertyName}},
		{pxr::UsdGeomTokens->exposureIso, 					{CameraISOPropertyName}},
		{pxr::UsdGeomTokens->exposureTime, 					{CameraShutterSpeedPropertyName}},

		{pxr::UsdLuxTokens->inputsIntensity, 				{IntensityPropertyName}},
		{pxr::UsdLuxTokens->inputsExposure, 				{IntensityPropertyName}}, // In USD true intensity is just a function of "intensity" and "exposure"
		{pxr::UsdLuxTokens->inputsColor, 					{LightColorPropertyName}},
		{pxr::UsdLuxTokens->inputsWidth, 					{SourceWidthPropertyName}},
		{pxr::UsdLuxTokens->inputsHeight, 					{SourceHeightPropertyName}},
		{pxr::UsdLuxTokens->inputsAngle, 					{LightSourceAnglePropertyName}},
		{pxr::UsdLuxTokens->inputsShapingConeAngle, 		{OuterConeAnglePropertyName}},
		{pxr::UsdLuxTokens->inputsShapingConeSoftness, 		{InnerConeAnglePropertyName}},
		{pxr::UsdLuxTokens->inputsColorTemperature, 		{TemperaturePropertyName}},
		{pxr::UsdLuxTokens->inputsEnableColorTemperature, 	{UseTemperaturePropertyName}},
	};
	// clang-format on
	std::unordered_map<pxr::TfToken, TArray<FName>, pxr::TfHash>::const_iterator Iter = StandardMapping.find(AttrToken);
	if (Iter != StandardMapping.end())
	{
		return Iter->second;
	}

	// Check to see if it's one of the edge cases that depends on the prim type
	if (AttrToken == pxr::UsdLuxTokens->inputsRadius)
	{
		if (UsdPrim.IsA<pxr::UsdLuxSphereLight>())
		{
			return {SourceRadiusPropertyName};
		}
		else if (UsdPrim.IsA<pxr::UsdLuxDiskLight>())
		{
			return {SourceWidthPropertyName, SourceHeightPropertyName};
		}
	}

	return {};
}

void UsdUtils::DensifyTimeSamplesIfNeeded(
	const UE::FUsdPrim& Prim,
	const FName& PropertyPath,
	TArray<double>& InOutTimeSamples
)
{
	using namespace UnrealIdentifiers;

	// For now, only light intensity needs densification, because the final USD intensity
	// is intensity * 2^exposure, which is non-linear in exposure. If we only sample at
	// authored keyframe times, linearly interpolating the resulting UE keyframes won't
	// match the non-linear USD interpolation of the separate attributes.
	if (PropertyPath != IntensityPropertyName)
	{
		return;
	}

	if (InOutTimeSamples.Num() < 2)
	{
		return;
	}

	FScopedUsdAllocs Allocs;
	pxr::UsdPrim UsdPrim{Prim};

	pxr::UsdAttribute ExposureAttr = UsdPrim.GetAttribute(pxr::UsdLuxTokens->inputsExposure);
	if (!ExposureAttr || !ExposureAttr.ValueMightBeTimeVarying())
	{
		return;
	}

	// Check if all exposure time samples are zero (effectively no exposure animation)
	std::vector<double> ExposureSamples;
	ExposureAttr.GetTimeSamples(&ExposureSamples);
	if (ExposureSamples.empty())
	{
		return;
	}

	bool bAllZero = true;
	for (double SampleTime : ExposureSamples)
	{
		float ExposureValue = 0.0f;
		if (ExposureAttr.Get<float>(&ExposureValue, SampleTime) && !FMath::IsNearlyZero(ExposureValue))
		{
			bAllZero = false;
			break;
		}
	}
	if (bAllZero)
	{
		return;
	}

	// Densify: generate every integer time code between the first and last sample
	USD_LOG_INFO(
		TEXT("Baking light intensity animation frame-by-frame for prim '%s' because it has animated exposure, "
			 "which combines non-linearly with intensity (intensity * 2^exposure)"),
		*Prim.GetPrimPath().GetString()
	);

	const double First = FMath::FloorToDouble(InOutTimeSamples[0]);
	const double Last = FMath::CeilToDouble(InOutTimeSamples.Last());

	TSet<double> TimeCodeSet;
	TimeCodeSet.Reserve(static_cast<int32>(Last - First) + 1 + InOutTimeSamples.Num());

	// Preserve original samples (they may not be on integer boundaries)
	for (double Sample : InOutTimeSamples)
	{
		TimeCodeSet.Add(Sample);
	}

	// Add every integer time code in the range
	for (double TimeCode = First; TimeCode <= Last; TimeCode += 1.0)
	{
		TimeCodeSet.Add(TimeCode);
	}

	InOutTimeSamples = TimeCodeSet.Array();
	InOutTimeSamples.Sort();
}

bool UnrealToUsd::ConvertDrawModeComponent(
	const UUsdDrawModeComponent& DrawModeComponent,
	pxr::UsdPrim& UsdPrim,
	bool bWriteExtents,
	double UsdTimeCode
)
{
	if (!UsdPrim)
	{
		return false;
	}

	FScopedUsdAllocs Allocs;

	// If we have a bounds component on an opened stage then we know our prim must satisfy the requirements
	// to have an alternate draw mode (being a "model", having the API schema, etc.).
	// However, this function may be used to export imported components onto newly defined prims on a new stage though
	// (e.g. during level export) so let's enforce those requirements anyway
	pxr::UsdGeomModelAPI GeomModelAPI = pxr::UsdGeomModelAPI::Apply(UsdPrim);

	// To have bounds, the prim must be a "model". To be a model the prim must have a model kind (i.e. basically any of the
	// standard "kind"s set) and all of its ancestors up to the pseudoroot must have some "group" kind (like "assembly")
	if (!UsdPrim.IsModel())
	{
		pxr::TfToken Kind = IUsdPrim::GetKind(UsdPrim);
		if (Kind.IsEmpty())
		{
			IUsdPrim::SetKind(UsdPrim, pxr::KindTokens->component);
		}

		pxr::UsdPrim Parent = UsdPrim.GetParent();
		while (Parent && !Parent.IsPseudoRoot() && !Parent.IsGroup())
		{
			pxr::TfToken ParentKind = IUsdPrim::GetKind(Parent);
			if (ParentKind.IsEmpty())
			{
				IUsdPrim::SetKind(Parent, pxr::KindTokens->group);
			}

			Parent = Parent.GetParent();
		}
	}

	// Author the actual extents.
	// If we're a boundable, we can just author this as an `extent` opinion, but otherwise we'll need to author
	// an `extentsHint` via the pxr::UsdGeomModelAPI. However, if a prim has *both* `extentsHint` and `extent`, USD
	// will favor `extentsHint`, so in order to "affect the stage" we should author to `extentsHint` in that case
	if (bWriteExtents)
	{
		FUsdStageInfo StageInfo{UsdPrim.GetStage()};

		const FVector& UEBoundsMin = DrawModeComponent.BoundsMin;
		const FVector& UEBoundsMax = DrawModeComponent.BoundsMax;
		pxr::GfVec3f UEBoundsMinUsdSpace = UnrealToUsd::ConvertVectorFloat(StageInfo, UEBoundsMin);
		pxr::GfVec3f UEBoundsMaxUsdSpace = UnrealToUsd::ConvertVectorFloat(StageInfo, UEBoundsMax);
		pxr::GfVec3f UsdMin{
			FMath::Min(UEBoundsMinUsdSpace[0], UEBoundsMaxUsdSpace[0]),
			FMath::Min(UEBoundsMinUsdSpace[1], UEBoundsMaxUsdSpace[1]),
			FMath::Min(UEBoundsMinUsdSpace[2], UEBoundsMaxUsdSpace[2])
		};
		pxr::GfVec3f UsdMax{
			FMath::Max(UEBoundsMinUsdSpace[0], UEBoundsMaxUsdSpace[0]),
			FMath::Max(UEBoundsMinUsdSpace[1], UEBoundsMaxUsdSpace[1]),
			FMath::Max(UEBoundsMinUsdSpace[2], UEBoundsMaxUsdSpace[2])
		};

		pxr::VtArray<pxr::GfVec3f> Extents{UsdMin, UsdMax};

		pxr::UsdAttribute ExtentsHintAttr = GeomModelAPI.GetExtentsHintAttr();
		if (ExtentsHintAttr && ExtentsHintAttr.HasAuthoredValue())
		{
			ensure(GeomModelAPI.SetExtentsHint(Extents, UsdTimeCode));
		}
		else if (pxr::UsdGeomBoundable Boundable{UsdPrim})
		{
			if (pxr::UsdAttribute ExtentAttr = Boundable.CreateExtentAttr())
			{
				ExtentAttr.Set(Extents, UsdTimeCode);
			}
		}
		else
		{
			ensure(GeomModelAPI.SetExtentsHint(Extents, UsdTimeCode));
		}
	}

	if (pxr::UsdAttribute Attr = GeomModelAPI.CreateModelDrawModeAttr())
	{
		Attr.Set(
			DrawModeComponent.DrawMode == EUsdDrawMode::Origin		? pxr::UsdGeomTokens->origin
			: DrawModeComponent.DrawMode == EUsdDrawMode::Bounds	? pxr::UsdGeomTokens->bounds
			: DrawModeComponent.DrawMode == EUsdDrawMode::Cards		? pxr::UsdGeomTokens->cards
			: DrawModeComponent.DrawMode == EUsdDrawMode::Inherited ? pxr::UsdGeomTokens->inherited
																	: pxr::UsdGeomTokens->default_,
			UsdTimeCode
		);
	}

	if (pxr::UsdAttribute Attr = GeomModelAPI.CreateModelCardGeometryAttr())
	{
		Attr.Set(
			DrawModeComponent.CardGeometry == EUsdModelCardGeometry::Cross ? pxr::UsdGeomTokens->cross
			: DrawModeComponent.CardGeometry == EUsdModelCardGeometry::Box ? pxr::UsdGeomTokens->box
																		   : pxr::UsdGeomTokens->fromTexture,
			UsdTimeCode
		);
	}

	// Technically we don't need this when we're making the prim into a component, but it's probably best to do it anyway
	// for consistency, and it may be weird for the user if they happen to tweak the prim kind after this and have the bounds disappear
	if (pxr::UsdAttribute Attr = GeomModelAPI.CreateModelApplyDrawModeAttr())
	{
		Attr.Set(true, UsdTimeCode);
	}

	if (pxr::UsdAttribute Attr = GeomModelAPI.CreateModelDrawModeColorAttr())
	{
		// This color is just a vec3f and our color is already linear anyway, so just convert it directly instead of going through
		// UnrealToUsd::ConvertColor
		pxr::GfVec3f UsdColor{DrawModeComponent.BoundsColor.R, DrawModeComponent.BoundsColor.G, DrawModeComponent.BoundsColor.B};
		Attr.Set(UsdColor, UsdTimeCode);
	}

	// Author the actual card face texture references.
	// The logic surrounding when a texture is "authored" is complex (see https://openusd.org/release/api/class_usd_geom_model_a_p_i.html#details),
	// and when reading we must differentiate between not having a texture because something wasn't authored, and not having a texture because
	// something was authored but we failed to find it. This "AuthoredFaces" member is not exposed to blueprint/details panel though, and in general
	// when setting/clearing textures via blueprint/details panels we will also tweak AuthoredFaces. The combined effect is that when a user sets a
	// new texture in a property we will assume that means it became "authored", and when they clear a texture property we will assume that ceases to
	// be "authored"
	EUsdModelCardFace AuthoredFaces = DrawModeComponent.GetAuthoredFaces();

	using CreateAttrFuncType = decltype(&pxr::UsdGeomModelAPI::CreateModelCardTextureXPosAttr);
	using GetAttrFuncType = decltype(&pxr::UsdGeomModelAPI::GetModelCardTextureXPosAttr);

	TFunction<void(EUsdModelCardFace, GetAttrFuncType, CreateAttrFuncType)> ExportCardFace =
		[&DrawModeComponent,
		 AuthoredFaces,
		 UsdTimeCode,
		 &GeomModelAPI](EUsdModelCardFace Face, GetAttrFuncType GetAttr, CreateAttrFuncType CreateAttr)
	{
		UTexture2D* FaceTexture = DrawModeComponent.GetTextureForFace(Face);

		if (EnumHasAllFlags(AuthoredFaces, Face))
		{
			const pxr::VtValue DefaultValue;
			const bool bWriteSparsely = false;
			if (pxr::UsdAttribute Attr = (GeomModelAPI.*CreateAttr)(DefaultValue, bWriteSparsely))
			{
				// We have an existing texture and it's marked as "authored". Try exporting a path to it
				if (FaceTexture)
				{
#if WITH_EDITOR
					if (FaceTexture->AssetImportData)
					{
						FString TextureSourcePath = FaceTexture->AssetImportData->GetFirstFilename();
						FString ResolvedPath = UsdUtils::GetResolvedAssetPath(Attr);

						// Avoid authoring anything unless they point at different files because in the general case the
						// asset import data will have an absolute path, while the path on the attribute may be currently relative.
						// We still want to implement some kind of larger feature to let the user pick whether we should be authoring
						// relative or absolute paths all over, but for now the least we can do is try not changing relative paths
						// to absolute unnecessarily (or vice versa)
						if (!FPaths::IsSamePath(ResolvedPath, TextureSourcePath))
						{
							if (!FPaths::FileExists(TextureSourcePath))
							{
								USD_LOG_USERWARNING(FText::Format(
									LOCTEXT(
										"CardTextureNotOnDisk",
										"Authoring card texture path '{0}' for texture '{1}' onto attribute '{2}', but the source image file doesn't "
										"exist on disk! It may not be possible to display this card texture if the stage is reloaded or reopened. "
										"UTexture assets can't be automatically exported in this manner just yet. If you want to assign a new "
										"texture, make sure the UTexture asset's source image file exists on disk."
									),
									FText::FromString(TextureSourcePath),
									FText::FromString(FaceTexture->GetPathName()),
									FText::FromString(UsdToUnreal::ConvertPath(Attr.GetPath()))
								));
							}

							pxr::SdfAssetPath AssetPath = pxr::SdfAssetPath{UnrealToUsd::ConvertString(*TextureSourcePath).Get()};
							Attr.Set(AssetPath, UsdTimeCode);
						}
					}
					else
					{
						USD_LOG_USERWARNING(FText::Format(
							LOCTEXT(
								"NoAssetImportData",
								"Not authoring card texture for attribute '{0}' because the assigned texture '{1}' has no AssetImportData! UTexture "
								"assets can't be automatically exported in this manner just yet. If you want to assign a new texture, make sure the "
								"UTexture asset's source image file exists on disk."
							),
							FText::FromString(UsdToUnreal::ConvertPath(Attr.GetPath())),
							FText::FromString(FaceTexture->GetPathName())
						));
					}
#endif	  // WITH_EDITOR
				}
				else
				{
					// This face is marked as "authored" and yet we have no texture for it. The only reason it could end up this way is if the source
					// prim for this component originally had an authored texture there that didn't resolve when we parsed it, so let's just leave it
					// alone so that it can still do that if we reload
				}
			}
		}
		// Face is not marked as authored, let's actively clear our opinion for this attribute on the current edit target and time code
		else
		{
			pxr::UsdAttribute Attr = (GeomModelAPI.*GetAttr)();
			if (Attr && Attr.HasAuthoredValue())
			{
				// This is capable of clearing the default opinion too
				Attr.ClearAtTime(UsdTimeCode);
			}
		}
	};

	GetAttrFuncType XPosGet = &pxr::UsdGeomModelAPI::GetModelCardTextureXPosAttr;
	CreateAttrFuncType XPosCreate = &pxr::UsdGeomModelAPI::CreateModelCardTextureXPosAttr;
	GetAttrFuncType YPosGet = &pxr::UsdGeomModelAPI::GetModelCardTextureYPosAttr;
	CreateAttrFuncType YPosCreate = &pxr::UsdGeomModelAPI::CreateModelCardTextureYPosAttr;
	GetAttrFuncType ZPosGet = &pxr::UsdGeomModelAPI::GetModelCardTextureZPosAttr;
	CreateAttrFuncType ZPosCreate = &pxr::UsdGeomModelAPI::CreateModelCardTextureZPosAttr;
	GetAttrFuncType XNegGet = &pxr::UsdGeomModelAPI::GetModelCardTextureXNegAttr;
	CreateAttrFuncType XNegCreate = &pxr::UsdGeomModelAPI::CreateModelCardTextureXNegAttr;
	GetAttrFuncType YNegGet = &pxr::UsdGeomModelAPI::GetModelCardTextureYNegAttr;
	CreateAttrFuncType YNegCreate = &pxr::UsdGeomModelAPI::CreateModelCardTextureYNegAttr;
	GetAttrFuncType ZNegGet = &pxr::UsdGeomModelAPI::GetModelCardTextureZNegAttr;
	CreateAttrFuncType ZNegCreate = &pxr::UsdGeomModelAPI::CreateModelCardTextureZNegAttr;

	// We swap these when importing so that they look right in UE (e.g. ZPos is always pointing at UE +Z axis),
	// so when writing back we need to swap back too
	FUsdStageInfo StageInfo{UsdPrim.GetStage()};
	if (StageInfo.UpAxis == EUsdUpAxis::ZAxis)
	{
		Swap(YPosGet, YNegGet);
		Swap(YPosCreate, YNegCreate);
	}
	else
	{
		Swap(YPosGet, ZPosGet);
		Swap(YPosCreate, ZPosCreate);

		Swap(YNegGet, ZNegGet);
		Swap(YNegCreate, ZNegCreate);
	}

	ExportCardFace(EUsdModelCardFace::XPos, XPosGet, XPosCreate);
	ExportCardFace(EUsdModelCardFace::YPos, YPosGet, YPosCreate);
	ExportCardFace(EUsdModelCardFace::ZPos, ZPosGet, ZPosCreate);
	ExportCardFace(EUsdModelCardFace::XNeg, XNegGet, XNegCreate);
	ExportCardFace(EUsdModelCardFace::YNeg, YNegGet, YNegCreate);
	ExportCardFace(EUsdModelCardFace::ZNeg, ZNegGet, ZNegCreate);

	return true;
}

namespace UE::USDPrimConversion::Private
{
	FString PrimPathToNamespace(FString PrimPath)
	{
		const TCHAR* CharsToReplace = TEXT("/{}[]");
		while (*CharsToReplace)
		{
			PrimPath.ReplaceCharInline(*CharsToReplace, **UnrealIdentifiers::UsdNamespaceDelimiter, ESearchCase::CaseSensitive);
			++CharsToReplace;
		}

		// Make sure we don't start with a delimiter
		while (PrimPath.RemoveFromStart(UnrealIdentifiers::UsdNamespaceDelimiter))
		{
		}
		return PrimPath;
	}

	bool ConvertMetadataInternal(
		const FUsdPrimMetadata& PrimMetadata,
		const pxr::UsdPrim& Prim,
		const TArray<FString>& BlockedPrefixFilters,
		bool bInvertFilters,
		const FString& NamespacePrefix = {}
	)
	{
		using namespace UE::USDPrimConversion::Private;

		if (!Prim || PrimMetadata.Metadata.Num() == 0 || (bInvertFilters && BlockedPrefixFilters.Num() == 0))
		{
			return false;
		}

		FScopedUsdAllocs Allocs;

		bool bSuccess = true;
		for (const TPair<FString, FUsdMetadataValue>& MetadataPair : PrimMetadata.Metadata)
		{
			FString FullKeyPath = MetadataPair.Key;
			const FUsdMetadataValue& MetadataValue = MetadataPair.Value;

			if (MetadataValue.StringifiedValue.IsEmpty() || MetadataValue.TypeName.IsEmpty())
			{
				continue;
			}

			// It's likely always a bad idea to author these as they are automatically authored by just the prim
			// definition itself and we'll likely run into trouble if we try writing anything that differs from it
			const static TSet<FString> FieldsToSkip = {
				UsdToUnreal::ConvertToken(pxr::SdfFieldKeys->Specifier),
				UsdToUnreal::ConvertToken(pxr::SdfFieldKeys->TypeName)
			};
			if (FieldsToSkip.Contains(FullKeyPath))
			{
				continue;
			}

			// Note that here we always have full key paths, as we store these paths flattened out when we're in UE
			if (ShouldSkipField(FullKeyPath, BlockedPrefixFilters, bInvertFilters))
			{
				continue;
			}

			const FString* TypeNameToUse = &MetadataValue.TypeName;

			// Add the prim path prefix if we have any
			// Only add the prefix now as we need to check the original key path against the filters
			if (!NamespacePrefix.IsEmpty())
			{
				const static FString CustomDataPrefix = UsdToUnreal::ConvertToken(pxr::SdfFieldKeys->CustomData);

				// e.g. "customData:fromSourcePrims:Root:MyXform:CollapsedMesh1:customData:myIntMetadataValue"
				FullKeyPath = CustomDataPrefix + UnrealIdentifiers::UsdNamespaceDelimiter + NamespacePrefix + UnrealIdentifiers::UsdNamespaceDelimiter
							  + FullKeyPath;

				// USD is fine with us authoring apiSchemas directly as top level metadata, but it can't understand the typename
				// if it's in a nested dictionary. We're never going to be actively using that value as actual apiSchemas
				// after that point anyway, so we may as well just keep that as a string and let the data make it to USD at
				// least in some form if we need to.
				// Note that we're already filtering apiSchemas when reading data from child prims, so this is mostly just for
				// safety (given that the user can author all this manually) and edge cases (when assets are shared via the
				// asset cache).
				const static FString ApiSchemasToken = UsdToUnreal::ConvertToken(pxr::UsdTokens->apiSchemas);
				const static FString StringTypeName = UsdToUnreal::ConvertToken(pxr::SdfValueTypeNames->String.GetAsToken());
				if (MetadataPair.Key == ApiSchemasToken)
				{
					TypeNameToUse = &StringTypeName;
				}
			}

			pxr::VtValue UnstringifiedValue;
			bSuccess &= UsdUtils::Unstringify(MetadataValue.StringifiedValue, *TypeNameToUse, UnstringifiedValue);

			if (!bSuccess)
			{
				USD_LOG_WARNING(
					TEXT("Failed to set metadata '%s' on prim '%s' as the value '%s' could not be parsed from string!"),
					*FullKeyPath,
					*UsdToUnreal::ConvertPath(Prim.GetPrimPath()),
					*MetadataValue.StringifiedValue
				);
				break;
			}

			// If this is a key-value pair inside at least one dictionary we need to split the key into top-level
			// dictionary name and "the rest"
			pxr::TfToken TopLevelKeyName;
			pxr::TfToken KeyPath;
			int32 FirstColonIndex = FullKeyPath.Find(UnrealIdentifiers::UsdNamespaceDelimiter);
			if (FirstColonIndex != INDEX_NONE)
			{
				// If our path was "first:second:third", this will put "first" on TopLevelKeyName, and "second:third" on KeyPath
				FString UETopLevelKeyName = FullKeyPath.Left(FirstColonIndex);
				FString UEKeyPath = FullKeyPath.Right(FullKeyPath.Len() - FirstColonIndex - 1);

				TopLevelKeyName = UnrealToUsd::ConvertToken(*UETopLevelKeyName).Get();
				KeyPath = UnrealToUsd::ConvertToken(*UEKeyPath).Get();
			}
			// If this is a top-level key-value pair we can use the MetadataFullPath directly as the key
			else
			{
				TopLevelKeyName = UnrealToUsd::ConvertToken(*FullKeyPath).Get();
			}

			const bool bWillOverwrite = Prim.HasMetadataDictKey(TopLevelKeyName, KeyPath);
			if (bWillOverwrite)
			{
				USD_LOG_INFO(TEXT("Overwriting metadata field '%s' on prim '%s'"), *FullKeyPath, *UsdToUnreal::ConvertPath(Prim.GetPrimPath()));
			}

			// This will also nicely create the nested dictionaries that it needs on-demand
			bSuccess &= Prim.SetMetadataByDictKey(TopLevelKeyName, KeyPath, UnstringifiedValue);

			if (!bSuccess)
			{
				break;
			}
		}
		return bSuccess;
	}
}	 // namespace UE::USDPrimConversion::Private

bool UnrealToUsd::ConvertMetadata(
	const FUsdCombinedPrimMetadata& CombinedPrimMetadata,
	const pxr::UsdPrim& Prim,
	const TArray<FString>& BlockedPrefixFilters,
	bool bInvertFilters
)
{
	using namespace UE::USDPrimConversion::Private;

	if (!Prim || CombinedPrimMetadata.PrimPathToMetadata.Num() == 0 || (bInvertFilters && BlockedPrefixFilters.Num() == 0))
	{
		return false;
	}

	FScopedUsdAllocs Allocs;

	// In order to roundtrip metadata, we should try to output the metadata collected from the "main source prim" back
	// out to the "Prim" we were provided here. In simple cases like a simple Mesh prim or a simple SkelRoot, this means we
	// will output to the exported Mesh prim or SkelRoot the *exact* same metadata fields that were on the source prim,
	// including "apiSchemas" and "kind" and so on, which is great! There are some other edge cases though.
	//
	// We use FUsdCombinedPrimMetadata to store metadata from multiple prims that ended up sharing the same generated asset
	// (e.g. hash collision), but also to store metadata from all prims in the subtree that contributed to the asset, hash
	// collision or not (LOD Mesh prims, skinned mesh prims for a Skeletal Mesh, collapsed Mesh prims, etc.).
	//
	// If we're in the latter case (source prim subtree), we can still try to find the "main source prim" by checking for
	// a common ancestor to all, outputting the metadata from that common ancestor directly to "Prim", and outputting all
	// the metadata for other prims in different metadata namespaces. That makes sure all the metadata makes it back out
	// *somewhere*, but it also ensures that we fully roundtrip the metadata on the "main source prim".
	//
	// In the former case though (source prim hash collision generating single asset), there really is no "common ancestor",
	// so the best we can do is to output *all* prim metadata in different namespaces. This should be a rare edge case,
	// and the caller/user can always prevent that from happening by just ensuring FUsdCombinedPrimMetadata has a single
	// PrimPath stored though.
	FString CommonAncestor;
	if (CombinedPrimMetadata.PrimPathToMetadata.Num() > 1)
	{
		TArray<FString> MetadataPrims;
		CombinedPrimMetadata.PrimPathToMetadata.GetKeys(MetadataPrims);

		// If we have multiple prims just because we collected metadata from an entire subtree, then our root prim will
		// be the first one after we sort.
		// Note that there is still the chance that in the process of collecting metadata from the source prim subtree,
		// only one or more random child prim(s) had any metadata, while the actual "main source prim" didn't have any.
		// In that we'll either end up "promoting" that child prim's metadata, or handling that case as if we were in the
		// hash collision case mentioned above. Those are edge cases of edge cases though, and hopefully shouldn't cause
		// any trouble either way (all the metadata is still going to be output just fine)
		MetadataPrims.Sort();
		const FString& PotentialAncestor = MetadataPrims[0];

		bool bHasCommonAncestor = true;
		for (int32 Index = 1; Index < MetadataPrims.Num(); ++Index)
		{
			const FString& SomeMetadataPrim = MetadataPrims[Index];
			if (!SomeMetadataPrim.StartsWith(PotentialAncestor))
			{
				bHasCommonAncestor = false;
				break;
			}
		}

		if (bHasCommonAncestor)
		{
			CommonAncestor = PotentialAncestor;
		}
	}

	bool bSuccess = true;
	for (const TPair<FString, FUsdPrimMetadata>& PrimPathToMetadata : CombinedPrimMetadata.PrimPathToMetadata)
	{
		const FString& PrimPath = PrimPathToMetadata.Key;
		const FUsdPrimMetadata& PrimMetadata = PrimPathToMetadata.Value;

		// If this prim is not the common ancestor we need to output its metadata inside of a nested namespace
		const bool bIsTopLevelPrim = (CombinedPrimMetadata.PrimPathToMetadata.Num() == 1) || (PrimPath == CommonAncestor);
		const FString NamespacePrefix = bIsTopLevelPrim
											? TEXT("")
											: TEXT("fromSourcePrims") + UnrealIdentifiers::UsdNamespaceDelimiter + PrimPathToNamespace(PrimPath);

		bSuccess &= ConvertMetadataInternal(PrimMetadata, Prim, BlockedPrefixFilters, bInvertFilters, NamespacePrefix);

		if (!bSuccess)
		{
			break;
		}
	}

	return bSuccess;
}

bool UnrealToUsd::ConvertMetadata(
	const FUsdPrimMetadata& PrimMetadata,
	const pxr::UsdPrim& Prim,
	const TArray<FString>& BlockedPrefixFilters,
	bool bInvertFilters
)
{
	return UE::USDPrimConversion::Private::ConvertMetadataInternal(PrimMetadata, Prim, BlockedPrefixFilters, bInvertFilters);
}

bool UnrealToUsd::ConvertMetadata(
	const UUsdAssetUserData* AssetUserData,
	const pxr::UsdPrim& Prim,
	const TArray<FString>& BlockedPrefixFilters,
	bool bInvertFilters
)
{
	if (!AssetUserData || !Prim)
	{
		return false;
	}

	// In the general case we'll have a single stage in here, and also a single FUsdPrimMetadata entry inside of it.
	// Here we coalesce all the metadata entries we have though. The inner ConvertMetadata call will warn
	// about any overwriting metadata keys
	for (const TPair<FString, FUsdCombinedPrimMetadata>& StageMetadataPair : AssetUserData->StageIdentifierToMetadata)
	{
		const bool bSuccess = ConvertMetadata(StageMetadataPair.Value, Prim, BlockedPrefixFilters, bInvertFilters);
		if (!bSuccess)
		{
			return false;
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE

#endif	  // #if USE_USD_SDK
