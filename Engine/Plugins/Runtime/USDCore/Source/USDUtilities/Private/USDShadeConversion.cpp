// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDShadeConversion.h"

#if USE_USD_SDK

#include "UnrealUSDWrapper.h"
#include "USDAssetCache2.h"
#include "USDAssetCache3.h"
#include "USDAssetImportData.h"
#include "USDAssetUserData.h"
#include "USDAttributeUtils.h"
#include "USDConversionUtils.h"
#include "USDErrorUtils.h"
#include "USDExportUtils.h"
#include "USDLayerUtils.h"
#include "USDMemory.h"
#include "USDObjectUtils.h"
#include "USDTypesConversion.h"

#include "UsdWrappers/UsdAttribute.h"
#include "UsdWrappers/UsdPrim.h"

#include "Algo/AllOf.h"
#include "Engine/Texture2D.h"
#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "ImageUtils.h"
#include "Internationalization/Regex.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpressionConstant.h"
#include "Materials/MaterialExpressionConstant4Vector.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "Materials/MaterialExpressionVertexColor.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialExpressionUtils.h"
#include "MaterialShared.h"
#include "Math/TransformCalculus2D.h"
#include "Misc/FileHelper.h"
#include "Modules/ModuleManager.h"
#include "RenderUtils.h"
#include "TextureResource.h"
#include "UDIMUtilities.h"

#if WITH_EDITOR
#include "Factories/TextureFactory.h"
#include "IMaterialBakingModule.h"
#include "MaterialBakingStructures.h"
#include "MaterialEditingLibrary.h"
#include "MaterialOptions.h"
#include "MaterialUtilities.h"
#endif	  // WITH_EDITOR

#include "USDIncludesStart.h"
#include "pxr/base/tf/token.h"
#include "pxr/usd/ar/asset.h"
#include "pxr/usd/ar/resolvedPath.h"
#include "pxr/usd/ar/resolver.h"
#include "pxr/usd/ar/resolverScopedCache.h"
#include "pxr/usd/sdf/layerUtils.h"
#include "pxr/usd/usd/primCompositionQuery.h"
#include "pxr/usd/usdShade/material.h"
#include "pxr/usd/usdShade/materialBindingAPI.h"
#include "pxr/usd/usdShade/nodeDefAPI.h"

#include "USDIncludesEnd.h"

// Required for packaging
class UTextureFactory;

#define LOCTEXT_NAMESPACE "USDShadeConversion"

namespace UE::UsdShadeConversion::Private
{
#if WITH_EDITOR
	using FlattenToMatEntry = TPairInitializer<const EFlattenMaterialProperties&, const EMaterialProperty&>;
	const static TMap<EFlattenMaterialProperties, EMaterialProperty> FlattenToMaterialProperty({
		FlattenToMatEntry(EFlattenMaterialProperties::Diffuse, EMaterialProperty::MP_BaseColor),
		FlattenToMatEntry(EFlattenMaterialProperties::Metallic, EMaterialProperty::MP_Metallic),
		FlattenToMatEntry(EFlattenMaterialProperties::Specular, EMaterialProperty::MP_Specular),
		FlattenToMatEntry(EFlattenMaterialProperties::Roughness, EMaterialProperty::MP_Roughness),
		FlattenToMatEntry(EFlattenMaterialProperties::Anisotropy, EMaterialProperty::MP_Anisotropy),
		FlattenToMatEntry(EFlattenMaterialProperties::Normal, EMaterialProperty::MP_Normal),
		FlattenToMatEntry(EFlattenMaterialProperties::Tangent, EMaterialProperty::MP_Tangent),
		FlattenToMatEntry(EFlattenMaterialProperties::Opacity, EMaterialProperty::MP_Opacity),
		FlattenToMatEntry(EFlattenMaterialProperties::Emissive, EMaterialProperty::MP_EmissiveColor),
		FlattenToMatEntry(EFlattenMaterialProperties::SubSurface, EMaterialProperty::MP_SubsurfaceColor),
		FlattenToMatEntry(EFlattenMaterialProperties::OpacityMask, EMaterialProperty::MP_OpacityMask),
		FlattenToMatEntry(EFlattenMaterialProperties::AmbientOcclusion, EMaterialProperty::MP_AmbientOcclusion),
	});

	/** Simple wrapper around the FBakeOutput data that we can reuse for data coming in from a FFlattenMaterial */
	struct FBakedMaterialView
	{
		TMap<EMaterialProperty, TArray<FColor>*> PropertyData;
		TMap<EMaterialProperty, TArray<FFloat16Color>*> HDRPropertyData;
		TMap<EMaterialProperty, FIntPoint> PropertySizes;

		explicit FBakedMaterialView(FBakeOutput& BakeOutput)
			: PropertySizes(BakeOutput.PropertySizes)
		{
			HDRPropertyData.Reserve(BakeOutput.HDRPropertyData.Num());
			for (TPair<EMaterialProperty, TArray<FFloat16Color>>& BakedPair : BakeOutput.HDRPropertyData)
			{
				HDRPropertyData.Add(BakedPair.Key, &BakedPair.Value);
			}

			PropertyData.Reserve(BakeOutput.PropertyData.Num());
			for (TPair<EMaterialProperty, TArray<FColor>>& BakedPair : BakeOutput.PropertyData)
			{
				PropertyData.Add(BakedPair.Key, &BakedPair.Value);
			}
		}

		explicit FBakedMaterialView(FFlattenMaterial& FlattenMaterial)
		{
			PropertyData.Reserve(static_cast<uint8>(EFlattenMaterialProperties::NumFlattenMaterialProperties));
			for (const TPair<EFlattenMaterialProperties, EMaterialProperty>& FlattenToProperty : FlattenToMaterialProperty)
			{
				PropertyData.Add(FlattenToProperty.Value, &FlattenMaterial.GetPropertySamples(FlattenToProperty.Key));
			}

			PropertySizes.Reserve(static_cast<uint8>(EFlattenMaterialProperties::NumFlattenMaterialProperties));
			for (const TPair<EFlattenMaterialProperties, EMaterialProperty>& FlattenToProperty : FlattenToMaterialProperty)
			{
				PropertySizes.Add(FlattenToProperty.Value, FlattenMaterial.GetPropertySize(FlattenToProperty.Key));
			}
		}
	};
#endif	  // WITH_EDITOR

	// Given an AssetPath, resolve it to an actual file path
	FString ResolveAssetPath(const pxr::SdfLayerHandle& LayerHandle, const FString& AssetPath)
	{
		// TODO: Most of this stuff is incompatible with custom resolvers, as these asset paths may be URLs
		// or just GUIDs or anything like that, where relative vs absolute path make no sense. We will need
		// a different approach whenever we properly handle USD resolvers

		FScopedUsdAllocs UsdAllocs;

		FString PathToResolve = LayerHandle ? UsdToUnreal::ConvertString(
												  pxr::SdfComputeAssetPathRelativeToLayer(LayerHandle, UnrealToUsd::ConvertString(*AssetPath).Get())
											  )
											: AssetPath;

		// We need to provide absolute paths to the resolver later: It has no idea what to do with a relative
		// path relative to some random location
		if (FPaths::IsRelative(PathToResolve) && LayerHandle)
		{
			FString LayerDirectory = FPaths::GetPath(UsdToUnreal::ConvertString(LayerHandle->GetRealPath()));
			PathToResolve = FPaths::Combine(LayerDirectory, PathToResolve);
		}

		// If this path has an UDIM placeholder in it (e.g. "textures/red.<UDIM>.exr"), let's try to find an
		// actual existing UDIM tile instead, or else Resolver.Resolve will just give us the empty string.
		// Previously we directly tried to find tile 1001, but it seems there is no guarantee that the
		// particular 1001 tile will exist.
		// Also note that for UDIMs the UE texture factory expects to receive path to any one tile, and it
		// will itself discover the remaining tiles (need to also keep that in mind when doing the support
		// for custom resolvers...)
		if (PathToResolve.Contains(TEXT("<UDIM>")))
		{
			FString UdimFileFilter = PathToResolve.Replace(TEXT("<UDIM>"), TEXT("*"));

			TArray<FString> UDIMFiles;
			IFileManager::Get().FindFiles(UDIMFiles, *UdimFileFilter, /*files*/ true, /*directories*/ false);
			if (UDIMFiles.Num() > 0)
			{
				// Sort here to enforce some sort of consistency between repeated calls
				UDIMFiles.Sort();

				// FindFiles will just return the filename with no path info, so lets put the file in the same
				// location our original <UDIM> path was
				PathToResolve = FPaths::Combine(FPaths::GetPath(PathToResolve), UDIMFiles[0]);
			}
		}

		pxr::ArResolverScopedCache ResolverCache;
		pxr::ArResolver& Resolver = pxr::ArGetResolver();
		std::string AssetIdentifier = Resolver.Resolve(UnrealToUsd::ConvertString(*PathToResolve).Get().c_str());

		// Don't normalize an empty path as the result will be "."
		if (AssetIdentifier.size() > 0)
		{
			AssetIdentifier = Resolver.CreateIdentifier(AssetIdentifier);
		}

		FString ResolvedAssetPath = UsdToUnreal::ConvertString(AssetIdentifier);
		return ResolvedAssetPath;
	}

	// If ResolvedTexturePath is in a format like "C:/MyFiles/scene.usdz[0/texture.png]", this will place "png" in OutExtensionNoDot, and
	// return true
	bool IsInsideUsdzArchive(const FString& ResolvedTexturePath, FString& OutExtensionNoDot)
	{
		FString USDZFilePath;
		FString InnerAssetPath;
		const bool bSplit = UsdUtils::SplitUSDZAssetPath(ResolvedTexturePath, USDZFilePath, InnerAssetPath);
		if (!bSplit)
		{
			return false;
		}

		// Should be something like "png"
		OutExtensionNoDot = FPaths::GetExtension(InnerAssetPath);
		return true;
	}

	TUsdStore<std::shared_ptr<const char>> ReadTextureBufferFromUsdzArchive(const FString& ResolvedTexturePath, uint64& OutBufferSize)
	{
		pxr::ArResolver& Resolver = pxr::ArGetResolver();
		std::shared_ptr<pxr::ArAsset> Asset = Resolver.OpenAsset(pxr::ArResolvedPath(UnrealToUsd::ConvertString(*ResolvedTexturePath).Get()));

		TUsdStore<std::shared_ptr<const char>> Buffer;

		if (Asset)
		{
			OutBufferSize = static_cast<uint64>(Asset->GetSize());
			{
				FScopedUsdAllocs Allocs;
				Buffer = Asset->GetBuffer();
			}
		}

		return Buffer;
	}

	/** If ResolvedTexturePath points at a texture inside an usdz file, this will use USD to pull the asset from the file, and TextureFactory
	 * to import it directly from the binary buffer */
	UTexture* ReadTextureFromUsdzArchiveEditor(
		const FString& ResolvedTexturePath,
		const FString& TextureExtension,
		UTextureFactory* TextureFactory,
		UObject* Outer,
		FName TextureName,
		EObjectFlags ObjectFlags
	)
	{
#if WITH_EDITOR
		uint64 BufferSize = 0;
		TUsdStore<std::shared_ptr<const char>> Buffer = ReadTextureBufferFromUsdzArchive(ResolvedTexturePath, BufferSize);
		const uint8* BufferStart = reinterpret_cast<const uint8*>(Buffer.Get().get());

		if (BufferSize == 0 || BufferStart == nullptr)
		{
			return nullptr;
		}

		FScopedUnrealAllocs UEAllocs;

		UObject* Context = nullptr;
		FFeedbackContext* FeedbackContext = GWarn;
		return Cast<UTexture>(TextureFactory->FactoryCreateBinary(
			UTexture::StaticClass(),
			Outer,
			TextureName,
			ObjectFlags,
			Context,
			*TextureExtension,
			BufferStart,
			BufferStart + BufferSize,
			FeedbackContext
		));
#else
		return nullptr;
#endif	  // WITH_EDITOR
	}

	UTexture* ReadTextureFromUsdzArchiveRuntime(const FString& ResolvedTexturePath)
	{
		uint64 BufferSize = 0;
		TUsdStore<std::shared_ptr<const char>> Buffer = ReadTextureBufferFromUsdzArchive(ResolvedTexturePath, BufferSize);
		const uint8* BufferStart = reinterpret_cast<const uint8*>(Buffer.Get().get());

		if (BufferSize == 0 || BufferStart == nullptr)
		{
			return nullptr;
		}

		FScopedUnrealAllocs UEAllocs;

		// Copied from FImageUtils::ImportBufferAsTexture2D( Buffer ) so that we can avoid a copy into the TArray<uint8> that it takes as
		// parameter
		IImageWrapperModule& ImageWrapperModule = FModuleManager::Get().LoadModuleChecked<IImageWrapperModule>(TEXT("ImageWrapper"));
		EImageFormat Format = ImageWrapperModule.DetectImageFormat(BufferStart, BufferSize);

		UTexture2D* NewTexture = nullptr;
		EPixelFormat PixelFormat = PF_Unknown;
		if (Format != EImageFormat::Invalid)
		{
			TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule.CreateImageWrapper(Format);

			int32 BitDepth = 0;
			int32 Width = 0;
			int32 Height = 0;

			if (ImageWrapper->SetCompressed((void*)BufferStart, BufferSize))
			{
				PixelFormat = PF_Unknown;

				ERGBFormat RGBFormat = ERGBFormat::Invalid;

				BitDepth = ImageWrapper->GetBitDepth();

				Width = ImageWrapper->GetWidth();
				Height = ImageWrapper->GetHeight();

				if (BitDepth == 16)
				{
					PixelFormat = PF_FloatRGBA;
					RGBFormat = ERGBFormat::BGRA;
				}
				else if (BitDepth == 8)
				{
					PixelFormat = PF_B8G8R8A8;
					RGBFormat = ERGBFormat::BGRA;
				}
				else
				{
					USD_LOG_WARNING(TEXT("Error creating texture. Bit depth is unsupported. (%d)"), BitDepth);
					return nullptr;
				}

				TArray64<uint8> UncompressedData;
				ImageWrapper->GetRaw(RGBFormat, BitDepth, UncompressedData);

				NewTexture = UTexture2D::CreateTransient(Width, Height, PixelFormat);
				if (NewTexture)
				{
					NewTexture->bNotOfflineProcessed = true;
					uint8* MipData = static_cast<uint8*>(NewTexture->GetPlatformData()->Mips[0].BulkData.Lock(LOCK_READ_WRITE));

					// Bulk data was already allocated for the correct size when we called CreateTransient above
					FMemory::Memcpy(MipData, UncompressedData.GetData(), NewTexture->GetPlatformData()->Mips[0].BulkData.GetBulkDataSize());

					NewTexture->GetPlatformData()->Mips[0].BulkData.Unlock();

					NewTexture->UpdateResource();
				}
			}
		}
		else
		{
			USD_LOG_WARNING(TEXT("Error creating texture. Couldn't determine the file format"));
		}

		return NewTexture;
	}

	// Will traverse the shade material graph backwards looking for a string/token value and return it.
	// Returns the empty string if it didn't find anything.
	FString RecursivelySearchForStringValue(pxr::UsdShadeInput Input)
	{
		if (!Input)
		{
			return {};
		}

		FScopedUsdAllocs Allocs;

		if (Input.HasConnectedSource())
		{
			pxr::UsdShadeConnectableAPI Souce;
			pxr::TfToken SourceName;
			pxr::UsdShadeAttributeType SourceType;
			if (pxr::UsdShadeConnectableAPI::GetConnectedSource(Input.GetAttr(), &Souce, &SourceName, &SourceType))
			{
				for (const pxr::UsdShadeInput& DeeperInput : Souce.GetInputs())
				{
					FString Result = RecursivelySearchForStringValue(DeeperInput);
					if (!Result.IsEmpty())
					{
						return Result;
					}
				}
			}
		}
		else
		{
			std::string StringValue;
			if (Input.Get<std::string>(&StringValue))
			{
				return UsdToUnreal::ConvertString(StringValue);
			}

			pxr::TfToken TokenValue;
			if (Input.Get<pxr::TfToken>(&TokenValue))
			{
				return UsdToUnreal::ConvertToken(TokenValue);
			}
		}

		return {};
	}

	/**
	 * Receives a UsdUVTexture shader, and returns the name of the primvar that it is using as 'st',
	 * plus the USD-space UV transforms that should be applied to that primvar when sampling this texture with
	 * it.
	 */
	void GetSTPrimvarAndTransform(const pxr::UsdShadeConnectableAPI& UsdUVTexture, UsdToUnreal::FTextureParameterValue& OutTextureValue)
	{
		FScopedUsdAllocs Allocs;

		pxr::UsdShadeInput StInput = UsdUVTexture.GetInput(UnrealIdentifiers::St);
		if (!StInput)
		{
			return;
		}

		pxr::UsdShadeConnectableAPI Connectable;
		pxr::TfToken ConnectableOutput;
		pxr::UsdShadeAttributeType AttributeType;

		FTransform2f ConcatenatedTransform;
		bool bFoundPrimvarReader = false;

		// Traverse through potentially N UV transform nodes
		while (pxr::UsdShadeConnectableAPI::GetConnectedSource(StInput.GetAttr(), &Connectable, &ConnectableOutput, &AttributeType))
		{
			pxr::UsdPrim ConnectedPrim = Connectable.GetPrim();
			pxr::UsdShadeNodeDefAPI ConnectedShadeNode{ConnectedPrim};
			if (!ConnectedShadeNode)
			{
				// Deadend, should never really happen
				return;
			}

			pxr::TfToken ConnectedNodeId;
			ConnectedShadeNode.GetShaderId(&ConnectedNodeId);

			// UV transform shader node
			if (ConnectedNodeId == UnrealIdentifiers::UsdTransform2d)
			{
				FScale2f Scale{1.0f, 1.0f};
				if (pxr::UsdShadeInput ScaleInput = Connectable.GetInput(UnrealIdentifiers::Scale))
				{
					pxr::GfVec2f VecValue;
					if (ScaleInput.Get(&VecValue))
					{
						Scale = FScale2f{VecValue[0], VecValue[1]};
					}
				}

				float Rotation = 0.0f;
				if (pxr::UsdShadeInput RotationInput = Connectable.GetInput(UnrealIdentifiers::Rotation))
				{
					float FloatValue;
					if (RotationInput.Get(&FloatValue))
					{
						Rotation = FloatValue;
					}
				}

				FVector2f Translation{0.0f, 0.0f};
				if (pxr::UsdShadeInput TranslationInput = Connectable.GetInput(UnrealIdentifiers::Translation))
				{
					pxr::GfVec2f VecValue;
					if (TranslationInput.Get(&VecValue))
					{
						Translation = FVector2f{VecValue[0], VecValue[1]};
					}
				}

				// Concat transform (scale, then rotation, then translation)
				FTransform2f NewTransform = FTransform2f{Scale}.Concatenate(FTransform2f{FQuat2f{FMath::DegreesToRadians(Rotation)}, Translation});
				ConcatenatedTransform = ConcatenatedTransform.Concatenate(NewTransform);

				// Step through to whatever is *this* node's input
				StInput = Connectable.GetInput(UnrealIdentifiers::In);
			}
			// Directly connected to primvar reader
			else if (ConnectedNodeId == UnrealIdentifiers::UsdPrimvarReader_float2)
			{
				bFoundPrimvarReader = true;
				break;
			}
			else
			{
				USD_LOG_WARNING(
					TEXT("Unexpected shader node id '%s' when traversing texture node '%s' for primvars!"),
					*UsdToUnreal::ConvertToken(ConnectedNodeId),
					*UsdToUnreal::ConvertPath(UsdUVTexture.GetPrim().GetPrimPath())
				);
				return;
			}
		}

		// Ideally after running through the UsdTransform2d nodes we'd run into a primvar reader node
		if (bFoundPrimvarReader)
		{
			if (pxr::UsdShadeInput VarnameInput = Connectable.GetInput(UnrealIdentifiers::Varname))
			{
				// PrimvarReader may have a string literal with the primvar name, or it may be connected to
				// e.g. an attribute defined elsewhere
				FString Primvar = RecursivelySearchForStringValue(VarnameInput);
				if (!Primvar.IsEmpty())
				{
					// This stuff will end up as arguments for the UsdTransform2d UE material function, which
					// will do a bunch of conversions inside. We could precompute some of that here, but instead
					// we're choosing not to, because this means that these values will show up on the material
					// instance exactly as they show up in the USD shader prims (e.g. if in USD the rotation is
					// 30 (degrees) we'll see "30" on the material instance too)
					OutTextureValue.Primvar = Primvar;
					OutTextureValue.UVScale = ConcatenatedTransform.GetMatrix().GetScale();
					OutTextureValue.UVRotation = -FMath::RadiansToDegrees(ConcatenatedTransform.GetMatrix().GetRotationAngle());
					OutTextureValue.UVTranslation = FVector2f{ConcatenatedTransform.GetTranslation()};
				}
			}
		}
	}

	// This works for both pxr::UsdShadeInput and pxr::UsdShadeOutput
	template<typename InputOrOutputType>
	bool GetTextureParameterValue(InputOrOutputType& InputOrOutput, TextureGroup Group, UsdToUnreal::FParameterValue& OutValue)
	{
		FScopedUsdAllocs UsdAllocs;

		const bool bIsNormalInput = (InputOrOutput.GetTypeName() == pxr::SdfValueTypeNames->Normal3f 
								  || InputOrOutput.GetTypeName() == pxr::SdfValueTypeNames->Normal3fArray
			 					  || InputOrOutput.GetBaseName() == UnrealIdentifiers::Normal);

		pxr::UsdShadeConnectableAPI UsdUVTextureSource;
		pxr::TfToken UsdUVTextureSourceName;
		pxr::UsdShadeAttributeType UsdUVTextureAttributeType;

		// Clear it, as we'll signal that it has a valid texture bound by setting it with an FTextureParameterValue
		OutValue.Set<float>(0.0f);

		if (pxr::UsdShadeConnectableAPI::GetConnectedSource(
				InputOrOutput.GetAttr(),
				&UsdUVTextureSource,
				&UsdUVTextureSourceName,
				&UsdUVTextureAttributeType
			))
		{
			pxr::UsdShadeInput FileInput;

			// UsdUVTexture: Get its file input
			if (UsdUVTextureAttributeType == pxr::UsdShadeAttributeType::Output)
			{
				FileInput = UsdUVTextureSource.GetInput(UnrealIdentifiers::File);
			}
			// Check if we are being directly passed an asset
			else
			{
				FileInput = UsdUVTextureSource.GetInput(UsdUVTextureSourceName);
			}

			// Recursively traverse "inputs:file" connections until we stop finding other connected prims
			pxr::UsdShadeConnectableAPI TextureFileSource;
			pxr::TfToken TextureFileSourceName;
			pxr::UsdShadeAttributeType TextureFileAttributeType;
			while (FileInput)
			{
				if (pxr::UsdShadeConnectableAPI::GetConnectedSource(
						FileInput.GetAttr(),
						&TextureFileSource,
						&TextureFileSourceName,
						&TextureFileAttributeType
					))
				{
					// Another connection, get its file input
					if (TextureFileAttributeType == pxr::UsdShadeAttributeType::Output)
					{
						FileInput = TextureFileSource.GetInput(UnrealIdentifiers::File);
					}
					// Check if we are being directly passed an asset
					else
					{
						FileInput = TextureFileSource.GetInput(TextureFileSourceName);
					}
				}
				else
				{
					break;
				}
			}

			if (FileInput && FileInput.GetTypeName() == pxr::SdfValueTypeNames->Asset)	  // Check that FileInput is of type Asset
			{
				// Get desired texture wrapping modes
				TextureAddress AddressX = TextureAddress::TA_Wrap;
				TextureAddress AddressY = TextureAddress::TA_Wrap;
				if (pxr::UsdAttribute WrapSAttr = UsdUVTextureSource.GetInput(UnrealIdentifiers::WrapS))
				{
					pxr::TfToken WrapS;
					if (WrapSAttr.Get(&WrapS))
					{
						AddressX = WrapS == UnrealIdentifiers::Repeat	? TextureAddress::TA_Wrap
								   : WrapS == UnrealIdentifiers::Mirror ? TextureAddress::TA_Mirror
																		: TextureAddress::TA_Clamp;	   // We also consider the "black" wrap
																									   // mode as clamp as that is the closest
																									   // we can get
					}
				}
				if (pxr::UsdAttribute WrapTAttr = UsdUVTextureSource.GetInput(UnrealIdentifiers::WrapT))
				{
					pxr::TfToken WrapT;
					if (WrapTAttr.Get(&WrapT))
					{
						AddressY = WrapT == UnrealIdentifiers::Repeat	? TextureAddress::TA_Wrap
								   : WrapT == UnrealIdentifiers::Mirror ? TextureAddress::TA_Mirror
																		: TextureAddress::TA_Clamp;	   // We also consider the "black" wrap
																									   // mode as clamp as that is the closest
																									   // we can get
					}
				}

				const FString TexturePath = UsdUtils::GetResolvedAssetPath(FileInput.GetAttr());

				// We will assume the texture is valid, and show a warning if we fail to parse this later.
				// Note that we don't even check that the file exists: If we have a texture bound to opacity then we
				// assume this material is meant to be translucent, even if the texture path is invalid (or points inside an USDZ archive)
				if (!TexturePath.IsEmpty())
				{
					OutValue.Set<UsdToUnreal::FTextureParameterValue>(UsdToUnreal::FTextureParameterValue{});
				}
				else
				{
					// Prefer to retrieve the original filepath string as this TexturePath will likely just
					// be the empty string if we failed to resolve
					FString RawFilePath = TexturePath;
					if (FileInput)
					{
						pxr::SdfAssetPath AssetPath;
						FileInput.Get<pxr::SdfAssetPath>(&AssetPath);
						RawFilePath = UsdToUnreal::ConvertString(AssetPath.GetAssetPath());
					}

					USD_LOG_WARNING(
						TEXT("Failed to find the file '%s', used by shader prim '%s'."),
						*RawFilePath,
						*UsdToUnreal::ConvertPath(InputOrOutput.GetPrim().GetPrimPath())
					);
					return false;
				}

				// If the texture is UDIM, it will contain the "<UDIM>" tag. UsdUtils::GetResolvedAssetPath will already
				// replace the tag with one of the tiles so that it plugs nicely into an resolver call that it has, but
				// (for Interchange) we also need to know whether this path was originally flagged to be interpreted as UDIM
				// or not, so here we just do a simple check on the string value that is actually on the attribute
				bool bIsUDIM = false;
				if (pxr::UsdAttribute Attr = FileInput.GetAttr())
				{
					pxr::SdfAssetPath Path;
					if (Attr.Get(&Path))
					{
						FString RawFilePath = UsdToUnreal::ConvertString(Path.GetAssetPath());
						bIsUDIM = RawFilePath.Contains(TEXT("<UDIM>"));
					}
				}

				// Collect desired sRGB, if anything is authored
				TOptional<bool> bSRGB;
				if (pxr::UsdAttribute SourceColorSpaceAttr = UsdUVTextureSource.GetInput(UnrealIdentifiers::SourceColorSpaceToken))
				{
					pxr::TfToken SourceColorSpaceValue;
					if (SourceColorSpaceAttr.HasAuthoredValue() && SourceColorSpaceAttr.Get(&SourceColorSpaceValue))
					{
						bSRGB = SourceColorSpaceValue == UnrealIdentifiers::RawColorSpaceToken	  ? false
								: SourceColorSpaceValue == UnrealIdentifiers::SRGBColorSpaceToken ? true
																								  : bSRGB;
					}
				}

				UsdToUnreal::FTextureParameterValue& OutTextureValue = OutValue.Get<UsdToUnreal::FTextureParameterValue>();
				OutTextureValue.TextureFilePath = TexturePath;
				OutTextureValue.AddressX = AddressX;
				OutTextureValue.AddressY = AddressY;
				OutTextureValue.Group = Group;
				OutTextureValue.bSRGB = bSRGB;
				OutTextureValue.bIsUDIM = bIsUDIM;
				GetSTPrimvarAndTransform(UsdUVTextureSource, OutTextureValue);

				// The UsdUVTexture Shader itself prim has multiple standard outputs, but this is not
				// full swizzle support (check
				// https://github.com/PixarAnimationStudios/USD/blob/5c5ebddff35012461a2b0ba773c47f05730cbab4/pxr/usdImaging/plugin/usdShaders/shaders/shaderDefs.usda#L198)
				if (UsdUVTextureAttributeType == pxr::UsdShadeAttributeType::Output)
				{
					const FName OutputName(UsdToUnreal::ConvertToken(UsdUVTextureSourceName));

					if (OutputName == TEXT("rgb"))
					{
						OutTextureValue.OutputIndex = 0;
					}
					else if (OutputName == TEXT("r"))
					{
						OutTextureValue.OutputIndex = 1;
					}
					else if (OutputName == TEXT("g"))
					{
						OutTextureValue.OutputIndex = 2;
					}
					else if (OutputName == TEXT("b"))
					{
						OutTextureValue.OutputIndex = 3;
					}
					else if (OutputName == TEXT("a"))
					{
						OutTextureValue.OutputIndex = 4;
					}
				}
			}
		}

		// We may be calling this from IsMaterialTranslucent, when we have no TexturesCache: We wouldn't be able to import textures without
		// the cache as we may repeatedly parse the same texture. Because of this, and how we clear OutValue to float before we begin, we can
		// say that if we have a FTextureParameterValue at all, then there is a valid texture that *can* be parsed, and so we return true.
		return OutValue.IsType<UsdToUnreal::FTextureParameterValue>();
	}

	bool GetFloatParameterValue(
		pxr::UsdShadeConnectableAPI& Connectable,
		const pxr::TfToken& InputName,
		float DefaultValue,
		UsdToUnreal::FParameterValue& OutValue
	)
	{
		FScopedUsdAllocs Allocs;

		auto ProcessPort = [&InputName, &OutValue, &DefaultValue]<typename PortType>(const PortType& Port) -> bool
		{
			// If we have another shader/node connected
			pxr::UsdShadeConnectableAPI Source;
			pxr::TfToken SourceName;
			pxr::UsdShadeAttributeType AttributeType;
			if (pxr::UsdShadeConnectableAPI::GetConnectedSource(Port.GetAttr(), &Source, &SourceName, &AttributeType))
			{
				if (GetTextureParameterValue(Port, TEXTUREGROUP_WorldSpecular, OutValue))
				{
					return true;
				}

				// Check if we have a fallback input that we can use instead, since we don't have a valid texture value
				if (const pxr::UsdShadeInput FallbackInput = Source.GetInput(UnrealIdentifiers::Fallback))
				{
					float UsdFallbackFloat;
					if (FallbackInput.Get<float>(&UsdFallbackFloat))
					{
						OutValue.Set<float>(UsdFallbackFloat);
						return true;
					}
				}

				// Recurse because the attribute may just be pointing at some other attribute that has the data
				// (e.g. when shader input is just "hoisted" and connected to the parent material input)
				return GetFloatParameterValue(Source, SourceName, DefaultValue, OutValue);
			}
			// No other node connected, so we must have some value
			else
			{
				float InputValue = DefaultValue;

				if constexpr (std::is_same_v<PortType, pxr::UsdShadeInput>)
				{
					Port.template Get<float>(&InputValue);
				}

				OutValue.Set<float>(InputValue);
				return true;
			}

			return false;
		};

		pxr::UsdShadeInput Input = Connectable.GetInput(InputName);
		if (Input)
		{
			return ProcessPort(Input);
		}

		// Check if we're connected to another NodeGraph output instead, and try climbing through that
		pxr::UsdShadeOutput Output = Connectable.GetOutput(InputName);
		if (Output)
		{
			return ProcessPort(Output);
		}

		return false;
	}

	template<typename InputOrOutputType>
	bool GetPrimvarReaderParameterValue(
		const InputOrOutputType& Input,
		const pxr::TfToken& PrimvarReaderShaderId,
		const FLinearColor& DefaultValue,
		UsdToUnreal::FParameterValue& OutValue
	)
	{
		FScopedUsdAllocs Allocs;

		if (!Input)
		{
			return false;
		}

		const bool bShaderOutputsOnly = true;
		const pxr::UsdShadeAttributeVector ValProdAttrs = Input.GetValueProducingAttributes(bShaderOutputsOnly);
		for (const pxr::UsdAttribute& ValProdAttr : ValProdAttrs)
		{
			const pxr::UsdShadeShader ValProdShader(ValProdAttr.GetPrim());
			if (!ValProdShader)
			{
				continue;
			}

			pxr::TfToken ShaderId;
			if (!ValProdShader.GetShaderId(&ShaderId) || ShaderId != PrimvarReaderShaderId)
			{
				continue;
			}

			const pxr::UsdShadeInput VarnameInput = ValProdShader.GetInput(UnrealIdentifiers::Varname);
			if (!VarnameInput)
			{
				continue;
			}

			// The schema for UsdPrimvarReader specifies that the "varname" input should be
			// string-typed, but some assets might be set up token-typed, so we'll consider
			// either type.
			std::string PrimvarName;
			if (VarnameInput.GetTypeName() == pxr::SdfValueTypeNames->String)
			{
				if (!VarnameInput.Get(&PrimvarName))
				{
					continue;
				}
			}
			else if (VarnameInput.GetTypeName() == pxr::SdfValueTypeNames->Token)
			{
				pxr::TfToken PrimvarNameToken;
				if (!VarnameInput.Get(&PrimvarNameToken))
				{
					continue;
				}
				PrimvarName = PrimvarNameToken.GetString();
			}

			if (PrimvarName.empty())
			{
				continue;
			}

			FLinearColor FallbackColor = DefaultValue;
			const pxr::UsdShadeInput FallbackInput = ValProdShader.GetInput(UnrealIdentifiers::Fallback);
			pxr::GfVec3f UsdFallbackColor;
			if (FallbackInput && FallbackInput.Get<pxr::GfVec3f>(&UsdFallbackColor))
			{
				FallbackColor = UsdToUnreal::ConvertColor(UsdFallbackColor);
			}

			OutValue.Set<UsdToUnreal::FPrimvarReaderParameterValue>(
				UsdToUnreal::FPrimvarReaderParameterValue{UsdToUnreal::ConvertString(PrimvarName), FVector(FallbackColor)}
			);
			return true;
		}

		return false;
	}

	bool GetVec3ParameterValue(
		pxr::UsdShadeConnectableAPI& Connectable,
		const pxr::TfToken& InputName,
		const FLinearColor& DefaultValue,
		UsdToUnreal::FParameterValue& OutValue,
		TextureGroup Group = TextureGroup::TEXTUREGROUP_World
	)
	{
		FScopedUsdAllocs Allocs;

		auto ProcessPort = [Group, &InputName, &OutValue, &DefaultValue]<typename PortType>(const PortType& Port) -> bool
		{
			// Check if we're connected to something
			pxr::UsdShadeConnectableAPI Source;
			pxr::TfToken SourceName;
			pxr::UsdShadeAttributeType AttributeType;
			if (pxr::UsdShadeConnectableAPI::GetConnectedSource(Port.GetAttr(), &Source, &SourceName, &AttributeType))
			{
				// Check if we're connected to a UsdUVTexture prim
				if (GetTextureParameterValue(Port, Group, OutValue))
				{
					return true;
				}

				// Check whether this input receives its value through a connection to a primvar reader shader
				if (GetPrimvarReaderParameterValue(Port, UnrealIdentifiers::UsdPrimvarReader_float3, DefaultValue, OutValue))
				{
					return true;
				}

				// Check if we have a fallback input on the connection, since we don't have a valid texture value
				if (const pxr::UsdShadeInput FallbackInput = Source.GetInput(UnrealIdentifiers::Fallback))
				{
					pxr::GfVec3f UsdFallbackVec3;
					if (FallbackInput.Get<pxr::GfVec3f>(&UsdFallbackVec3))
					{
						OutValue.Set<FVector>(UsdToUnreal::ConvertVector(UsdFallbackVec3));
						return true;
					}

					pxr::GfVec4f UsdFallbackVec4;
					if (FallbackInput.Get<pxr::GfVec4f>(&UsdFallbackVec4))
					{
						if (!FMath::IsNearlyEqual(UsdFallbackVec4[3], 1.0f))
						{
							USD_LOG_WARNING(
								TEXT("Ignoring alpha value from fallback GfVec4f [%f, %f, %f, %f] used for Shader '%s'"),
								UsdFallbackVec4[0],
								UsdFallbackVec4[1],
								UsdFallbackVec4[2],
								UsdFallbackVec4[3],
								*UsdToUnreal::ConvertPath(Source.GetPrim().GetPrimPath())
							);
						}

						OutValue.Set<FVector>(FVector{UsdFallbackVec4[0], UsdFallbackVec4[1], UsdFallbackVec4[2]});
						return true;
					}
				}

				// We're connected to something else, so try traversing into the input connectable itself
				return GetVec3ParameterValue(Source, SourceName, DefaultValue, OutValue, Group);
			}
			// Has a value directly on the port
			else
			{
				FLinearColor DiffuseColor = DefaultValue;

				if constexpr (std::is_same_v<PortType, pxr::UsdShadeInput>)
				{
					pxr::GfVec3f UsdDiffuseColor;
					if (Port.template Get<pxr::GfVec3f>(&UsdDiffuseColor))
					{
						DiffuseColor = UsdToUnreal::ConvertColor(UsdDiffuseColor);
					}
				}

				OutValue.Set<FVector>(FVector(DiffuseColor));
				return true;
			}

			return false;
		};

		pxr::UsdShadeInput Input = Connectable.GetInput(InputName);
		if (Input)
		{
			return ProcessPort(Input);
		}

		// Check if we're connected to another NodeGraph output instead, and try climbing through that
		pxr::UsdShadeOutput Output = Connectable.GetOutput(InputName);
		if (Output)
		{
			return ProcessPort(Output);
		}

		return false;
	}

	bool GetBoolParameterValue(
		pxr::UsdShadeConnectableAPI& Connectable,
		const pxr::TfToken& InputName,
		bool DefaultValue,
		UsdToUnreal::FParameterValue& OutValue
	)
	{
		FScopedUsdAllocs Allocs;

		pxr::UsdShadeInput Input = Connectable.GetInput(InputName);
		if (!Input)
		{
			return false;
		}

		// If we have another shader/node connected
		pxr::UsdShadeConnectableAPI Source;
		pxr::TfToken SourceName;
		pxr::UsdShadeAttributeType AttributeType;
		if (pxr::UsdShadeConnectableAPI::GetConnectedSource(Input.GetAttr(), &Source, &SourceName, &AttributeType))
		{
			// Recurse because the attribute may just be pointing at some other attribute that has the data
			// (e.g. when shader input is just "hoisted" and connected to the parent material input)
			return GetBoolParameterValue(Source, SourceName, DefaultValue, OutValue);
		}
		// No other node connected, so we must have some value
		else
		{
			bool InputValue = DefaultValue;
			Input.Get<bool>(&InputValue);

			OutValue.Set<bool>(InputValue);
		}

		return true;
	}

	bool GetIntParameterValue(
		pxr::UsdShadeConnectableAPI& Connectable,
		const pxr::TfToken& InputName,
		int32 DefaultValue,
		UsdToUnreal::FParameterValue& OutValue
	)
	{
		FScopedUsdAllocs Allocs;

		pxr::UsdShadeInput Input = Connectable.GetInput(InputName);
		if (!Input)
		{
			return false;
		}

		// If we have another shader/node connected
		pxr::UsdShadeConnectableAPI Source;
		pxr::TfToken SourceName;
		pxr::UsdShadeAttributeType AttributeType;
		if (pxr::UsdShadeConnectableAPI::GetConnectedSource(Input.GetAttr(), &Source, &SourceName, &AttributeType))
		{
			// Recurse because the attribute may just be pointing at some other attribute that has the data
			// (e.g. when shader input is just "hoisted" and connected to the parent material input)
			return GetIntParameterValue(Source, SourceName, DefaultValue, OutValue);
		}
		// No other node connected, so we must have some value
		else
		{
			int32 InputValue = DefaultValue;
			Input.Get<int32>(&InputValue);

			OutValue.Set<int32>(InputValue);
		}

		return true;
	}

	// No recursion for token value
	bool GetTokenAsFloatParameterValue(
		pxr::UsdShadeConnectableAPI& Connectable,
		const pxr::TfToken& InputName,
		const char* DefaultValue,
		const TMap<FString, float>& Values,
		UsdToUnreal::FParameterValue& OutValue
	)
	{
		FScopedUsdAllocs Allocs;

		pxr::UsdShadeInput Input = Connectable.GetInput(InputName);
		if (!Input)
		{
			return false;
		}

		pxr::TfToken TokenInputValue{ DefaultValue };
		Input.Get<pxr::TfToken>(&TokenInputValue);
		float InputValue = 0;
		if (const float* FoundValue = Values.Find(UsdToUnreal::ConvertToken(TokenInputValue)))
		{
			InputValue = *FoundValue;
		}
		OutValue.Set<float>(InputValue);

		return true;
	}

#if WITH_EDITOR
	/**
	 * Intended to be used with Visit(), this will create a UMaterialExpression in InMaterial, set it with the
	 * value stored in the current variant of an FParameterValue, and return it.
	 */
	struct FGetExpressionForValueVisitor
	{
		FGetExpressionForValueVisitor(UMaterial& InMaterial)
			: Material(InMaterial)
		{
		}

		UMaterialExpression* operator()(const float FloatValue) const
		{
			UMaterialExpressionConstant* Expression = Cast<UMaterialExpressionConstant>(
				UMaterialEditingLibrary::CreateMaterialExpression(&Material, UMaterialExpressionConstant::StaticClass())
			);
			Expression->R = FloatValue;

			return Expression;
		}

		UMaterialExpression* operator()(const FVector& VectorValue) const
		{
			UMaterialExpressionConstant4Vector* Expression = Cast<UMaterialExpressionConstant4Vector>(
				UMaterialEditingLibrary::CreateMaterialExpression(&Material, UMaterialExpressionConstant4Vector::StaticClass())
			);
			Expression->Constant = FLinearColor(VectorValue);

			return Expression;
		}

		UMaterialExpression* operator()(const UsdToUnreal::FTextureParameterValue& TextureValue) const
		{
			UMaterialExpressionTextureSample* Expression = Cast<UMaterialExpressionTextureSample>(
				UMaterialEditingLibrary::CreateMaterialExpression(&Material, UMaterialExpressionTextureSample::StaticClass())
			);
			Expression->Texture = TextureValue.Texture;
			Expression->SamplerType = MaterialExpressionUtils::GetSamplerTypeForTexture(TextureValue.Texture);

			return Expression;
		}

		UMaterialExpression* operator()(const UsdToUnreal::FPrimvarReaderParameterValue& PrimvarReaderValue) const
		{
			UMaterialExpression* Expression = nullptr;

			// We currently only import the "displayColor" primvar in USD as vertex colors
			// on UE meshes, so that's the only primvar we can support here.
			if (PrimvarReaderValue.PrimvarName == "displayColor")
			{
				Expression = UMaterialEditingLibrary::CreateMaterialExpression(&Material, UMaterialExpressionVertexColor::StaticClass());
			}
			else
			{
				Expression = UMaterialEditingLibrary::CreateMaterialExpression(&Material, UMaterialExpressionConstant4Vector::StaticClass());

				UMaterialExpressionConstant4Vector* FallbackExpression = Cast<UMaterialExpressionConstant4Vector>(Expression);
				if (FallbackExpression)
				{
					FallbackExpression->Constant = FLinearColor(PrimvarReaderValue.FallbackValue);
				}
			}

			return Expression;
		}

	private:
		UMaterial& Material;
	};
#endif	  // WITH_EDITOR

	UMaterialExpression* GetExpressionForValue(UMaterial& Material, const UsdToUnreal::FParameterValue& ParameterValue)
	{
#if WITH_EDITOR
		FGetExpressionForValueVisitor Visitor{Material};
		return Visit(Visitor, ParameterValue);
#else
		return nullptr;
#endif	  // WITH_EDITOR
	}

	/**
	 * Intended to be used with Visit(), this will set an FParameterValue into InMaterial using InParameterName,
	 * depending on the variant active in FParameterValue
	 */
	struct FSetParameterValueVisitor
	{
		explicit FSetParameterValueVisitor(UMaterialInstance& InMaterial, const TCHAR* InParameterName)
			: Material(InMaterial)
			, ParameterName(InParameterName)
		{
		}

		void operator()(const float FloatValue) const
		{
			UsdUtils::SetScalarParameterValue(Material, ParameterName, FloatValue);
		}

		void operator()(const FVector& VectorValue) const
		{
			UsdUtils::SetVectorParameterValue(Material, ParameterName, FLinearColor(VectorValue));
		}

		void operator()(const UsdToUnreal::FTextureParameterValue& TextureValue) const
		{
			UsdUtils::SetTextureParameterValue(Material, ParameterName, TextureValue.Texture);
		}

		void operator()(const UsdToUnreal::FPrimvarReaderParameterValue& PrimvarReaderValue) const
		{
			UsdUtils::SetVectorParameterValue(Material, ParameterName, FLinearColor(PrimvarReaderValue.FallbackValue));
		}

		void operator()(const bool BoolValue) const
		{
			UsdUtils::SetBoolParameterValue(Material, ParameterName, BoolValue);
		}

		// we handle int as scalar parameters
		void operator()(const int32 Int32Value) const
		{
			UsdUtils::SetScalarParameterValue(Material, ParameterName, Int32Value);
		}

	protected:
		UMaterialInstance& Material;
		const TCHAR* ParameterName;
	};

	/**
	 * Specialized version of FSetParameterValueVisitor for UE's UsdPreviewSurface reference materials.
	 */
	struct FSetPreviewSurfaceParameterValueVisitor : private FSetParameterValueVisitor
	{
		explicit FSetPreviewSurfaceParameterValueVisitor(
			UMaterialInstance& InMaterial,
			const TCHAR* InParameterName,
			const TMap<FString, int32>& InPrimvarToUVIndex
		)
			: FSetParameterValueVisitor(InMaterial, InParameterName)
			, PrimvarToUVIndex(InPrimvarToUVIndex)
		{
		}

		void operator()(const float FloatValue) const
		{
			FSetParameterValueVisitor::operator()(FloatValue);
			UsdUtils::SetScalarParameterValue(Material, *FString::Printf(TEXT("Use%sTexture"), ParameterName), 0.0f);
		}

		void operator()(const FVector& VectorValue) const
		{
			FSetParameterValueVisitor::operator()(VectorValue);
			UsdUtils::SetScalarParameterValue(Material, *FString::Printf(TEXT("Use%sTexture"), ParameterName), 0.0f);
		}

		void operator()(const UsdToUnreal::FTextureParameterValue& TextureValue) const
		{
			UsdUtils::SetTextureParameterValue(Material, *FString::Printf(TEXT("%sTexture"), ParameterName), TextureValue.Texture);
			UsdUtils::SetScalarParameterValue(Material, *FString::Printf(TEXT("Use%sTexture"), ParameterName), 1.0f);

			FLinearColor ScaleAndTranslation = FLinearColor{
				TextureValue.UVScale.GetVector()[0],
				TextureValue.UVScale.GetVector()[1],
				TextureValue.UVTranslation[0],
				TextureValue.UVTranslation[1]
			};
			UsdUtils::SetVectorParameterValue(Material, *FString::Printf(TEXT("%sScaleTranslation"), ParameterName), ScaleAndTranslation);

			UsdUtils::SetScalarParameterValue(Material, *FString::Printf(TEXT("%sRotation"), ParameterName), TextureValue.UVRotation);

			if (const int32* FoundIndex = PrimvarToUVIndex.Find(TextureValue.Primvar))
			{
				UsdUtils::SetScalarParameterValue(Material, *FString::Printf(TEXT("%sUVIndex"), ParameterName), *FoundIndex);
			}
			else
			{
				USD_LOG_USERWARNING(FText::Format(
					LOCTEXT(
						"MissingPrimvar",
						"Failed to find primvar '{0}' when setting material parameter '{1}' on material '{2}'. Available primvars and UV indices: {3}.{4}"
					),
					FText::FromString(TextureValue.Primvar),
					FText::FromString(ParameterName),
					FText::FromString(Material.GetPathName()),
					FText::FromString(UsdUtils::StringifyMap(PrimvarToUVIndex)),
					FText::FromString(
						TextureValue.Primvar.IsEmpty() ? TEXT(" Is your UsdUVTexture Shader missing the 'inputs:st' attribute? (It specifies "
															  "which UV set to sample the texture with)")
													   : TEXT("")
					)
				));
			}

			FLinearColor ComponentMask = FLinearColor::Black;
			switch (TextureValue.OutputIndex)
			{
				case 0:	   // RGB
					ComponentMask = FLinearColor{1.f, 1.f, 1.f, 0.f};
					break;
				case 1:	   // R
					ComponentMask = FLinearColor{1.f, 0.f, 0.f, 0.f};
					break;
				case 2:	   // G
					ComponentMask = FLinearColor{0.f, 1.f, 0.f, 0.f};
					break;
				case 3:	   // B
					ComponentMask = FLinearColor{0.f, 0.f, 1.f, 0.f};
					break;
				case 4:	   // A
					ComponentMask = FLinearColor{0.f, 0.f, 0.f, 1.f};
					break;
			}
			UsdUtils::SetVectorParameterValue(Material, *FString::Printf(TEXT("%sTextureComponent"), ParameterName), ComponentMask);
		}

		void operator()(const UsdToUnreal::FPrimvarReaderParameterValue& PrimvarReaderValue) const
		{
			FSetParameterValueVisitor::operator()(PrimvarReaderValue.FallbackValue);

			// We currently only import the "displayColor" primvar in USD as vertex colors
			// on UE meshes, so that's the only primvar we can support here.
			if (PrimvarReaderValue.PrimvarName == "displayColor")
			{
				UsdUtils::SetScalarParameterValue(Material, TEXT("UseVertexColorForBaseColor"), 1.0f);
			}
		}

	protected:
		const TMap<FString, int32>& PrimvarToUVIndex;
	};

	void SetParameterValue(
		UMaterialInstance& Material,
		const TCHAR* ParameterName,
		const UsdToUnreal::FParameterValue& ParameterValue,
		bool bForUsdPreviewSurface,
		const TMap<FString, int32>& PrimvarToUVIndex
	)
	{
		if (bForUsdPreviewSurface)
		{
			FSetPreviewSurfaceParameterValueVisitor Visitor{Material, ParameterName, PrimvarToUVIndex};
			Visit(Visitor, ParameterValue);
		}
		else
		{
			FSetParameterValueVisitor Visitor{Material, ParameterName};
			Visit(Visitor, ParameterValue);
		}
	}

	UTexture* CreateTextureWithEditor(
		const FString& ResolvedTexturePath,
		FName SanitizedName,
		TextureGroup Group,
		EObjectFlags ObjectFlags,
		UObject* Outer,
		bool bForceLinear,
		bool bIsUDIM
	)
	{
		UTexture* Texture = nullptr;
#if WITH_EDITOR
		if (ResolvedTexturePath.IsEmpty() || !Outer)
		{
			return nullptr;
		}

		FScopedUnrealAllocs UnrealAllocs;

		UTextureFactory* TextureFactory = NewObject<UTextureFactory>();
		TextureFactory->SuppressImportOverwriteDialog();
		TextureFactory->bUseHashAsGuid = true;
		TextureFactory->LODGroup = Group;
		TextureFactory->HDRImportShouldBeLongLatCubeMap = EAppReturnType::YesAll;
		// To maintain existing behavior, ColorSpaceMode is left at auto if linear is not requested
		TextureFactory->ColorSpaceMode = bForceLinear ? ETextureSourceColorSpace::Linear : ETextureSourceColorSpace::Auto;

		if (bIsUDIM)
		{
			const static FRegexPattern RegexPattern(TEXT(R"((.+[._])(\d{4}|<UDIM>)(.*))"));
			const FString BaseFileName = FPaths::GetBaseFilename(ResolvedTexturePath);
			FRegexMatcher RegexMatcher(RegexPattern, BaseFileName);
			if (RegexMatcher.FindNext())
			{
				FString BaseFileNameBeforeUdim = RegexMatcher.GetCaptureGroup(1);
				FString BaseFileNameAfterUdim = RegexMatcher.GetCaptureGroup(3);

				FString UdimRegexPattern = FString::Printf(TEXT(R"((%s)(\d{4})(%s))"), *BaseFileNameBeforeUdim, *BaseFileNameAfterUdim);
				TextureFactory->UdimRegexPattern = MoveTemp(UdimRegexPattern);
			}
		}

		if (!ResolvedTexturePath.IsEmpty())
		{
			FString TextureExtension;
			if (IsInsideUsdzArchive(ResolvedTexturePath, TextureExtension))
			{
				// Always prefer using the TextureFactory if we can, as it may provide compression, which the runtime version never will
				Texture = ReadTextureFromUsdzArchiveEditor(ResolvedTexturePath, TextureExtension, TextureFactory, Outer, SanitizedName, ObjectFlags);
			}
			// Not inside an USDZ archive, just a regular texture
			else
			{
				bool bOutCancelled = false;
				const TCHAR* Params = TEXT("");
				Texture = Cast<UTexture>(TextureFactory->ImportObject(	  //
					UTexture::StaticClass(),
					Outer,
					SanitizedName,
					ObjectFlags,
					ResolvedTexturePath,
					Params,
					bOutCancelled
				));
			}

			if (Texture)
			{
				// We set this even if we're not going to import so that we can track our original texture filepath
				// in case we later do an Actions->Import
				UUsdAssetImportData* ImportData = NewObject<UUsdAssetImportData>(Texture);
				ImportData->UpdateFilenameOnly(ResolvedTexturePath);
				Texture->AssetImportData = ImportData;
			}
		}
#endif	  // WITH_EDITOR
		return Texture;
	}

	UTexture* CreateTextureAtRuntime(const FString& ResolvedTexturePath)
	{
		FScopedUnrealAllocs UnrealAllocs;

		UTexture* Texture = nullptr;

		if (!ResolvedTexturePath.IsEmpty())
		{
			// Try checking if the texture is inside an USDZ archive first, or else TextureFactory throws an error
			FString TextureExtension;
			if (IsInsideUsdzArchive(ResolvedTexturePath, TextureExtension))
			{
				Texture = ReadTextureFromUsdzArchiveRuntime(ResolvedTexturePath);
			}

			// Not inside an USDZ archive, just a regular texture
			if (!Texture)
			{
				Texture = FImageUtils::ImportFileAsTexture2D(ResolvedTexturePath);
			}
		}

		return Texture;
	}

#if WITH_EDITOR
	// Note that we will bake things that aren't supported on the default USD surface shader schema. These could be useful in case the user
	// has a custom renderer though, and they can pick which properties they want anyway
	bool BakeMaterial(
		const UMaterialInterface& Material,
		const TArray<FPropertyEntry>& InMaterialProperties,
		const FIntPoint& InDefaultTextureSize,
		FBakeOutput& OutBakedData,
		bool bInDecayTexturesToSinglePixel
	)
	{
		TArray<UTexture*> MaterialTextures;
		Material.GetUsedTextures(MaterialTextures);

		// Precache all used textures, otherwise could get everything rendered with low-res textures.
		for (UTexture* Texture : MaterialTextures)
		{
			if (UTexture2D* Texture2D = Cast<UTexture2D>(Texture))
			{
				Texture2D->SetForceMipLevelsToBeResident(30.0f);
				Texture2D->WaitForStreaming();
			}
		}

		FMaterialData MatSet;
		MatSet.Material = const_cast<UMaterialInterface*>(&Material);	 // We don't modify it and neither does the material baking module,
																		 // it's just a bad signature
		MatSet.bPerformShrinking = bInDecayTexturesToSinglePixel;

		for (const FPropertyEntry& Entry : InMaterialProperties)
		{
			// No point in asking it to bake if we're going to use the user-supplied value
			if (Entry.bUseConstantValue)
			{
				continue;
			}

			switch (Entry.Property)
			{
				case MP_Normal:
					if (!Material.GetMaterial()->HasNormalConnected() && !Material.GetMaterial()->bUseMaterialAttributes)
					{
						continue;
					}
					break;
				case MP_Tangent:
					if (!Material.GetMaterial()->GetEditorOnlyData()->Tangent.IsConnected() && !Material.GetMaterial()->bUseMaterialAttributes)
					{
						continue;
					}
					break;
				case MP_EmissiveColor:
					if (!Material.GetMaterial()->GetEditorOnlyData()->EmissiveColor.IsConnected() && !Material.GetMaterial()->bUseMaterialAttributes)
					{
						continue;
					}
					break;
				case MP_OpacityMask:
					if (!Material.IsPropertyActive(MP_OpacityMask) || !IsMaskedBlendMode(Material))
					{
						continue;
					}
					break;
				case MP_Opacity:
					if (!Material.IsPropertyActive(MP_Opacity) || !IsTranslucentBlendMode(Material))
					{
						continue;
					}
					break;
				case MP_MAX:
					continue;
				default:
					if (!Material.IsPropertyActive(Entry.Property))
					{
						continue;
					}
					break;
			}

			MatSet.PropertySizes.Add(Entry.Property, Entry.bUseCustomSize ? Entry.CustomSize : InDefaultTextureSize);
		}

		FMeshData MeshSettings;
		MeshSettings.MeshDescription = nullptr;
		MeshSettings.TextureCoordinateBox = FBox2D(FVector2D(0.0f, 0.0f), FVector2D(1.0f, 1.0f));
		MeshSettings.TextureCoordinateIndex = 0;

		TArray<FBakeOutput> BakeOutputs;
		IMaterialBakingModule& Module = FModuleManager::Get().LoadModuleChecked<IMaterialBakingModule>("MaterialBaking");
		const bool bLinearBake = true;
		Module.SetLinearBake(bLinearBake);
		const bool bEmissiveHDR = true;
		Module.SetEmissiveHDR(bEmissiveHDR);
		Module.BakeMaterials({&MatSet}, {&MeshSettings}, BakeOutputs);

		if (BakeOutputs.Num() < 1)
		{
			return false;
		}

		OutBakedData = BakeOutputs[0];
		return true;
	}

	// Writes textures for all baked channels in BakedSamples that are larger than 1x1, and returns the filenames of the emitted textures for
	// each channel
	TMap<EMaterialProperty, FString> WriteTextures(
		FBakedMaterialView& BakedSamples,
		const FString& MaterialName,
		const FDirectoryPath& TexturesFolder
	)
	{
		IImageWrapperModule& ImageWrapperModule = FModuleManager::Get().LoadModuleChecked<IImageWrapperModule>(TEXT("ImageWrapper"));
		TSharedPtr<IImageWrapper> EXRImageWrapper = ImageWrapperModule.CreateImageWrapper(EImageFormat::EXR);
		TSharedPtr<IImageWrapper> PNGImageWrapper = ImageWrapperModule.CreateImageWrapper(EImageFormat::PNG);

		TMap<EMaterialProperty, FString> WrittenTexturesPerChannel;

		TSet<FString> UsedFilePaths;
		auto MakeUniqueTexturePath = [&UsedFilePaths, &MaterialName, &TexturesFolder](const FString& PropertyName, const FString& ExtensionWithDot)
		{
			const FString TextureFileName = UsdUnreal::ObjectUtils::SanitizeObjectName(MaterialName + TEXT("_") + PropertyName);
			const FString TextureFilePathPrefix = FPaths::Combine(TexturesFolder.Path, TextureFileName);
			return UsdUnreal::ExportUtils::GetUniqueFilePathForExport(TextureFilePathPrefix + ExtensionWithDot);
		};

		// Write textures for HDR baked properties larger than 1x1
		for (TPair<EMaterialProperty, TArray<FFloat16Color>*>& BakedDataPair : BakedSamples.HDRPropertyData)
		{
			const EMaterialProperty Property = BakedDataPair.Key;

			TArray<FFloat16Color>* Samples = BakedDataPair.Value;
			if (!Samples || Samples->Num() < 2)
			{
				continue;
			}

			const FIntPoint& FinalSize = BakedSamples.PropertySizes.FindChecked(Property);
			if (FinalSize.GetMin() < 2)
			{
				continue;
			}

			const UEnum* PropertyEnum = StaticEnum<EMaterialProperty>();
			FName PropertyName = PropertyEnum->GetNameByValue(Property);
			FString TrimmedPropertyName = PropertyName.ToString();
			TrimmedPropertyName.RemoveFromStart(TEXT("MP_"));

			const FString ExtensionWithDot = TEXT(".exr");
			const FString TextureFilePath = MakeUniqueTexturePath(TrimmedPropertyName, ExtensionWithDot);

			// For some reason the baked samples always have zero alpha and there is nothing we can do about it... It seems like the material
			// baking module is made with the intent that the data ends up in UTexture2Ds, where they can be set to be compressed without
			// alpha and have the value ignored. Since we need to write these to file, we must set them back up to full alpha. This is
			// potentially useless as USD handles these at most as color3f, but it could be annoying for the user if they intend on using the
			// textures for anything else
			for (FFloat16Color& Sample : *Samples)
			{
				Sample.A = 1.0f;
			}

			EXRImageWrapper->SetRaw(Samples->GetData(), Samples->GetAllocatedSize(), FinalSize.X, FinalSize.Y, ERGBFormat::RGBAF, 16);
			const TArray64<uint8> Data = EXRImageWrapper->GetCompressed(100);

			bool bWroteFile = FFileHelper::SaveArrayToFile(Data, *TextureFilePath);
			if (bWroteFile)
			{
				WrittenTexturesPerChannel.Add(Property, TextureFilePath);
			}
			else
			{
				USD_LOG_WARNING(TEXT("Failed to write texture '%s', baked channel will be ignored."), *TextureFilePath);
			}
		}

		// Write textures for baked properties larger than 1x1
		for (TPair<EMaterialProperty, TArray<FColor>*>& BakedDataPair : BakedSamples.PropertyData)
		{
			const EMaterialProperty Property = BakedDataPair.Key;

			// The material baking module still generates and sends an SDR version of any HDR channel it also bakes,
			// so here let's skip this one in case we already generated an HDR texture for the channel
			if (WrittenTexturesPerChannel.Contains(Property))
			{
				continue;
			}

			TArray<FColor>* Samples = BakedDataPair.Value;
			if (!Samples || Samples->Num() < 2)
			{
				continue;
			}

			const FIntPoint& FinalSize = BakedSamples.PropertySizes.FindChecked(Property);
			if (FinalSize.GetMin() < 2)
			{
				continue;
			}

			const UEnum* PropertyEnum = StaticEnum<EMaterialProperty>();
			FName PropertyName = PropertyEnum->GetNameByValue(Property);
			FString TrimmedPropertyName = PropertyName.ToString();
			TrimmedPropertyName.RemoveFromStart(TEXT("MP_"));

			const FString ExtensionWithDot = TEXT(".png");
			const FString TextureFilePath = MakeUniqueTexturePath(TrimmedPropertyName, ExtensionWithDot);

			// For some reason the baked samples always have zero alpha and there is nothing we can do about it... It seems like the material
			// baking module is made with the intent that the data ends up in UTexture2Ds, where they can be set to be compressed without
			// alpha and have the value ignored. Since we need to write these to file, we must set them back up to full alpha. This is
			// potentially useless as USD handles these at most as color3f, but it could be annoying for the user if they intend on using the
			// textures for anything else
			for (FColor& Sample : *Samples)
			{
				Sample.A = 255;
			}

			PNGImageWrapper->SetRaw(Samples->GetData(), Samples->GetAllocatedSize(), FinalSize.X, FinalSize.Y, ERGBFormat::BGRA, 8);
			const TArray64<uint8> PNGData = PNGImageWrapper->GetCompressed(100);

			bool bWroteFile = FFileHelper::SaveArrayToFile(PNGData, *TextureFilePath);
			if (bWroteFile)
			{
				WrittenTexturesPerChannel.Add(Property, TextureFilePath);
			}
			else
			{
				USD_LOG_WARNING(TEXT("Failed to write texture '%s', baked channel will be ignored."), *TextureFilePath);
			}
		}

		return WrittenTexturesPerChannel;
	}

	bool ConfigureShadePrim(
		const FBakedMaterialView& BakedData,
		const TMap<EMaterialProperty, FString>& WrittenTextures,
		const TMap<EMaterialProperty, float>& UserConstantValues,
		pxr::UsdShadeMaterial& OutUsdShadeMaterial
	)
	{
		FScopedUsdAllocs Allocs;

		pxr::UsdPrim MaterialPrim = OutUsdShadeMaterial.GetPrim();
		pxr::UsdStageRefPtr Stage = MaterialPrim.GetStage();
		if (!MaterialPrim || !Stage)
		{
			return false;
		}

		FString UsdFilePath = UsdToUnreal::ConvertString(Stage->GetRootLayer()->GetRealPath());

		FUsdStageInfo StageInfo{Stage};

		pxr::SdfPath MaterialPath = MaterialPrim.GetPath();

		// Create surface shader
		pxr::UsdShadeShader ShadeShader = pxr::UsdShadeShader::Define(
			Stage,
			MaterialPath.AppendChild(UnrealToUsd::ConvertToken(TEXT("SurfaceShader")).Get())
		);
		ShadeShader.SetShaderId(UnrealIdentifiers::UsdPreviewSurface);
		pxr::UsdShadeOutput ShaderOutOutput = ShadeShader.CreateOutput(UnrealIdentifiers::Surface, pxr::SdfValueTypeNames->Token);

		// Connect material to surface shader
		pxr::UsdShadeOutput MaterialSurfaceOutput = OutUsdShadeMaterial.CreateSurfaceOutput();
		MaterialSurfaceOutput.ConnectToSource(ShaderOutOutput);

		pxr::UsdShadeShader PrimvarReaderShader;

		const pxr::TfToken PrimvarReaderShaderName = UnrealToUsd::ConvertToken(TEXT("PrimvarReader")).Get();
		const pxr::TfToken PrimvarVariableName = UnrealToUsd::ConvertToken(TEXT("stPrimvarName")).Get();

		// Collect all the properties we'll process, as some data is baked and some comes from values the user input as export options
		TSet<EMaterialProperty> PropertiesToProcess;
		{
			PropertiesToProcess.Reserve(BakedData.PropertyData.Num() + UserConstantValues.Num() + BakedData.HDRPropertyData.Num());
			BakedData.PropertyData.GetKeys(PropertiesToProcess);

			TSet<EMaterialProperty> UsedProperties;
			UserConstantValues.GetKeys(UsedProperties);
			PropertiesToProcess.Append(UsedProperties);

			BakedData.HDRPropertyData.GetKeys(UsedProperties);
			PropertiesToProcess.Append(UsedProperties);
		}

		// We always write BaseColor because the default in UE is full black, but in USD seems to be 0.18. If we left
		// BaseColor unbound, a material that relies on opacity/other channels and leaves BaseColor disconnected for the black
		// value would end up looking gray in usdview/other DCCs
		if (!PropertiesToProcess.Contains(MP_BaseColor))
		{
			PropertiesToProcess.Add(MP_BaseColor);
		}
		// Analogously, the default refraction value in USD is 1.5, which is very distorted, while it's 1.0 in UE.
		// So if we have no other opinion here, let's author a 1.0 refraction value so that it looks like it would in UE.
		// We only care about that if we're also baking opacity though (so that we're in some kind of translucent mode). 
		// If we're fully opaque then we don't really care to author this
		if (!PropertiesToProcess.Contains(MP_Refraction) && (PropertiesToProcess.Contains(MP_Opacity) || PropertiesToProcess.Contains(MP_OpacityMask)))
		{
			PropertiesToProcess.Add(MP_Refraction);
		}

		const float Zero = 0.0f;
		const float One = 1.0f;

		// Fill in outputs
		for (const EMaterialProperty& Property : PropertiesToProcess)
		{
			const FString* TextureFilePath = WrittenTextures.Find(Property);
			const float* UserConstantValue = UserConstantValues.Find(Property);
			const FIntPoint* SampleSize = BakedData.PropertySizes.Find(Property);

			uint64 NumSamples = 0;

			bool bPropertyValueIsConstant = false;
			pxr::GfVec3f ConstantLinearValue;

			// Try HDR first: When baking a channel as HDR (like emissive) the MaterialBaking module
			// will still bake the SDR version of the channel too. We keep both in our FBakedMaterialView
			// because it only does the mechanism of decaying to a single value on the SDR data array
			bool bParsedHDR = false;
			if (BakedData.HDRPropertyData.Contains(Property))
			{
				const TArray<FColor>* SDRSamples = BakedData.PropertyData.FindRef(Property);
				NumSamples = SDRSamples ? SDRSamples->Num() : 0;

				const bool bDecayedToSingleSample = NumSamples == 1;
				bPropertyValueIsConstant = (UserConstantValue != nullptr || bDecayedToSingleSample);

				const TArray<FFloat16Color>* Samples = BakedData.HDRPropertyData.FindRef(Property);
				if (bDecayedToSingleSample && Samples)
				{
					// If it decayed to single sample we know our SDR array only has one value,
					// and so should our HDR array. Get the value from the HDR one to avoid an
					// unnecessary quantization though
					const FFloat16Color& Sample = (*Samples)[0];
					ConstantLinearValue = pxr::GfVec3f{Sample.R, Sample.G, Sample.B};
				}

				if (Samples && Samples->Num() > 0)
				{
					bParsedHDR = true;
				}
			}

			if (!bParsedHDR)
			{
				const TArray<FColor>* Samples = BakedData.PropertyData.FindRef(Property);
				NumSamples = Samples ? Samples->Num() : 0;

				bPropertyValueIsConstant = (UserConstantValue != nullptr || NumSamples == 1);
				if (NumSamples == 1)
				{
					switch (Property)
					{
						case EMaterialProperty::MP_BaseColor:
						case EMaterialProperty::MP_SubsurfaceColor:
						{
							pxr::GfVec4f ConvertedColor = UnrealToUsd::ConvertColor((*Samples)[0]);
							ConstantLinearValue = pxr::GfVec3f{ConvertedColor[0], ConvertedColor[1], ConvertedColor[2]};
							break;
						}
						case EMaterialProperty::MP_Normal:
						case EMaterialProperty::MP_Tangent:
						{
							FVector ConvertedNormal{(*Samples)[0].ReinterpretAsLinear()};
							ConstantLinearValue = UnrealToUsd::ConvertVectorFloat(StageInfo, ConvertedNormal);
							break;
						}
						default:
						{
							const FColor& Sample = (*Samples)[0];
							ConstantLinearValue = pxr::GfVec3f{Sample.R / 255.0f, Sample.G / 255.0f, Sample.B / 255.0f};
							break;
						}
					}
				}
			}

			if (Property == MP_BaseColor && NumSamples == 0 && !UserConstantValue)
			{
				UserConstantValue = &Zero;
				bPropertyValueIsConstant = true;
			}

			if (Property == MP_Refraction && NumSamples == 0 && !UserConstantValue)
			{
				UserConstantValue = &One;
				bPropertyValueIsConstant = true;
			}

			if ((NumSamples == 0 || !SampleSize) && !UserConstantValue)
			{
				USD_LOG_INFO(TEXT("Skipping material property %d as we have no valid data to use."), Property);
				continue;
			}

			if (!UserConstantValue && NumSamples > 0 && SampleSize)
			{
				if (NumSamples != SampleSize->X * SampleSize->Y)
				{
					USD_LOG_WARNING(
						TEXT("Skipping material property %d as it has an unexpected number of samples (%lld instead of %d)."),
						Property,
						NumSamples,
						SampleSize->X * SampleSize->Y
					);
					continue;
				}
			}

			if (!bPropertyValueIsConstant && (TextureFilePath == nullptr || !FPaths::FileExists(*TextureFilePath)))
			{
				USD_LOG_WARNING(
					TEXT("Skipping material property %d as target texture '%s' was not found."),
					Property,
					TextureFilePath ? **TextureFilePath : TEXT("")
				);
				continue;
			}

			pxr::TfToken InputToken;
			pxr::SdfValueTypeName InputType;
			pxr::VtValue ConstantValue;
			pxr::GfVec4f FallbackValue;
			pxr::TfToken ColorSpaceToken = UnrealIdentifiers::RawColorSpaceToken;
			switch (Property)
			{
				case MP_BaseColor:
					InputToken = UnrealIdentifiers::DiffuseColor;
					InputType = pxr::SdfValueTypeNames->Color3f;
					if (bPropertyValueIsConstant)
					{
						if (UserConstantValue)
						{
							ConstantValue = pxr::GfVec3f(*UserConstantValue, *UserConstantValue, *UserConstantValue);
						}
						else
						{
							ConstantValue = ConstantLinearValue;
						}
					}
					FallbackValue = pxr::GfVec4f{0.0, 0.0, 0.0, 1.0f};
					ColorSpaceToken = UnrealIdentifiers::SRGBColorSpaceToken;
					break;
				case MP_Specular:
					InputToken = UnrealIdentifiers::Specular;
					InputType = pxr::SdfValueTypeNames->Float;
					ConstantValue = UserConstantValue ? *UserConstantValue : ConstantLinearValue[0];
					FallbackValue = pxr::GfVec4f{0.5, 0.5, 0.5, 1.0f};
					break;
				case MP_Metallic:
					InputToken = UnrealIdentifiers::Metallic;
					InputType = pxr::SdfValueTypeNames->Float;
					ConstantValue = UserConstantValue ? *UserConstantValue : ConstantLinearValue[0];
					FallbackValue = pxr::GfVec4f{0.0, 0.0, 0.0, 1.0f};
					break;
				case MP_Roughness:
					InputToken = UnrealIdentifiers::Roughness;
					InputType = pxr::SdfValueTypeNames->Float;
					ConstantValue = UserConstantValue ? *UserConstantValue : ConstantLinearValue[0];
					FallbackValue = pxr::GfVec4f{0.5, 0.5, 0.5, 1.0f};
					break;
				case MP_Normal:
					InputToken = UnrealIdentifiers::Normal;
					InputType = pxr::SdfValueTypeNames->Normal3f;
					if (bPropertyValueIsConstant)
					{
						// This doesn't make much sense here but it's an available option, so here we go
						if (UserConstantValue)
						{
							ConstantValue = pxr::GfVec3f(*UserConstantValue, *UserConstantValue, *UserConstantValue);
						}
						else
						{
							ConstantValue = ConstantLinearValue;
						}
					}
					FallbackValue = pxr::GfVec4f{0.0, 0.0, 1.0, 1.0f};
					break;
				case MP_Tangent:
					InputToken = UnrealIdentifiers::Tangent;
					InputType = pxr::SdfValueTypeNames->Normal3f;
					if (bPropertyValueIsConstant)
					{
						// This doesn't make much sense here but it's an available option, so here we go
						if (UserConstantValue)
						{
							ConstantValue = pxr::GfVec3f(*UserConstantValue, *UserConstantValue, *UserConstantValue);
						}
						else
						{
							ConstantValue = ConstantLinearValue;
						}
					}
					FallbackValue = pxr::GfVec4f{1.0, 0.0, 0.0, 1.0f};
					break;
				case MP_EmissiveColor:
					InputToken = UnrealIdentifiers::EmissiveColor;
					InputType = pxr::SdfValueTypeNames->Color3f;
					if (bPropertyValueIsConstant)
					{
						// This doesn't make much sense here but it's an available option, so here we go
						if (UserConstantValue)
						{
							ConstantValue = pxr::GfVec3f(*UserConstantValue, *UserConstantValue, *UserConstantValue);
						}
						else
						{
							ConstantValue = ConstantLinearValue;
						}
					}
					FallbackValue = pxr::GfVec4f{0.0, 0.0, 0.0, 1.0f};
					// Emissive is also written out with RawColorSpaceToken as we write them as EXR files now
					break;
				case MP_Opacity:	// It's OK that we use the same for both as these are mutually exclusive blend modes
				case MP_OpacityMask:
					InputToken = UnrealIdentifiers::Opacity;
					InputType = pxr::SdfValueTypeNames->Float;
					ConstantValue = UserConstantValue ? *UserConstantValue : ConstantLinearValue[0];
					FallbackValue = pxr::GfVec4f{1.0, 1.0, 1.0, 1.0f};
					break;
				case MP_Refraction:
					InputToken = UnrealIdentifiers::Refraction;
					InputType = pxr::SdfValueTypeNames->Float;
					ConstantValue = UserConstantValue ? *UserConstantValue : ConstantLinearValue[0];
					FallbackValue = pxr::GfVec4f{1.0, 1.0, 1.0, 1.0f};
					break;
				case MP_Anisotropy:
					InputToken = UnrealIdentifiers::Anisotropy;
					InputType = pxr::SdfValueTypeNames->Float;
					ConstantValue = UserConstantValue ? *UserConstantValue : ConstantLinearValue[0];
					FallbackValue = pxr::GfVec4f{0.0, 0.0, 0.0, 1.0f};
					break;
				case MP_AmbientOcclusion:
					InputToken = UnrealIdentifiers::Occlusion;
					InputType = pxr::SdfValueTypeNames->Float;
					ConstantValue = UserConstantValue ? *UserConstantValue : ConstantLinearValue[0];
					FallbackValue = pxr::GfVec4f{1.0, 1.0, 1.0, 1.0f};
					break;
				case MP_SubsurfaceColor:
					InputToken = UnrealIdentifiers::SubsurfaceColor;
					InputType = pxr::SdfValueTypeNames->Color3f;
					if (bPropertyValueIsConstant)
					{
						if (UserConstantValue)
						{
							ConstantValue = pxr::GfVec3f(*UserConstantValue, *UserConstantValue, *UserConstantValue);
						}
						else
						{
							ConstantValue = ConstantLinearValue;
						}
					}
					FallbackValue = pxr::GfVec4f{1.0, 1.0, 1.0, 1.0f};
					ColorSpaceToken = UnrealIdentifiers::SRGBColorSpaceToken;
					break;
				default:
					continue;
			}

			pxr::UsdShadeInput ShadeInput = ShadeShader.CreateInput(InputToken, InputType);
			if (bPropertyValueIsConstant)
			{
				ShadeInput.Set(ConstantValue);
			}
			else	// Its a texture
			{
				// Create the primvar/uv set reader on-demand. We'll be using the same UV set for everything for now
				if (!PrimvarReaderShader)
				{
					PrimvarReaderShader = pxr::UsdShadeShader::Define(Stage, MaterialPath.AppendChild(PrimvarReaderShaderName));
					PrimvarReaderShader.SetShaderId(UnrealIdentifiers::UsdPrimvarReader_float2);

					// Create the 'st' input directly on the material, as that seems to be preferred
					pxr::UsdShadeInput MaterialStInput = OutUsdShadeMaterial.CreateInput(PrimvarVariableName, pxr::SdfValueTypeNames->Token);
					MaterialStInput.Set(UnrealIdentifiers::St);

					pxr::UsdShadeInput VarnameInput = PrimvarReaderShader.CreateInput(UnrealIdentifiers::Varname, pxr::SdfValueTypeNames->String);
					VarnameInput.ConnectToSource(MaterialStInput);

					PrimvarReaderShader.CreateOutput(UnrealIdentifiers::Result, pxr::SdfValueTypeNames->Token);
				}

				FString TextureReaderName = UsdToUnreal::ConvertToken(InputToken);
				TextureReaderName.RemoveFromEnd(TEXT("Color"));
				TextureReaderName += TEXT("Texture");

				pxr::UsdShadeShader UsdUVTextureShader = pxr::UsdShadeShader::Define(
					Stage,
					MaterialPath.AppendChild(UnrealToUsd::ConvertToken(*TextureReaderName).Get())
				);
				UsdUVTextureShader.SetShaderId(UnrealIdentifiers::UsdUVTexture);

				pxr::UsdShadeInput TextureFileInput = UsdUVTextureShader.CreateInput(UnrealIdentifiers::File, pxr::SdfValueTypeNames->Asset);
				FString TextureRelativePath = *TextureFilePath;
				if (!UsdFilePath.IsEmpty())
				{
					FPaths::MakePathRelativeTo(TextureRelativePath, *UsdFilePath);
				}
				TextureFileInput.Set(pxr::SdfAssetPath(UnrealToUsd::ConvertString(*TextureRelativePath).Get()));

				pxr::UsdShadeInput TextureStInput = UsdUVTextureShader.CreateInput(UnrealIdentifiers::St, pxr::SdfValueTypeNames->Float2);
				TextureStInput.ConnectToSource(PrimvarReaderShader.GetOutput(UnrealIdentifiers::Result));

				pxr::UsdShadeInput TextureColorSpaceInput = UsdUVTextureShader.CreateInput(
					UnrealIdentifiers::SourceColorSpaceToken,
					pxr::SdfValueTypeNames->Token
				);
				TextureColorSpaceInput.Set(ColorSpaceToken);

				pxr::UsdShadeInput TextureFallbackInput = UsdUVTextureShader.CreateInput(UnrealIdentifiers::Fallback, pxr::SdfValueTypeNames->Float4);
				TextureFallbackInput.Set(FallbackValue);

				// In the general case it's impossible to set a "correct" wrapping value here because the
				// material we just baked may be using 3 different textures with UV transforms an all different
				// texture wrapping modes, and we're forced to pick a single value to wrap the baked texture
				// with, but let's at least write "repeat" out as that is the default for textures in UE and
				// the more likely to be correct, in case the mesh does things like have UVs outside [0, 1]
				pxr::UsdShadeInput TextureFileWrapSInput = UsdUVTextureShader.CreateInput(UnrealIdentifiers::WrapS, pxr::SdfValueTypeNames->Token);
				TextureFileWrapSInput.Set(UnrealIdentifiers::Repeat);
				pxr::UsdShadeInput TextureFileWrapTInput = UsdUVTextureShader.CreateInput(UnrealIdentifiers::WrapT, pxr::SdfValueTypeNames->Token);
				TextureFileWrapTInput.Set(UnrealIdentifiers::Repeat);

				pxr::UsdShadeOutput TextureOutput = UsdUVTextureShader.CreateOutput(
					InputType == pxr::SdfValueTypeNames->Float ? UnrealIdentifiers::R : UnrealIdentifiers::RGB,
					InputType
				);

				ShadeInput.ConnectToSource(TextureOutput);
			}
		}

		return true;
	}
#endif	  // WITH_EDITOR

	void HashShadeInput(const pxr::UsdShadeInput& ShadeInput, FSHA1& InOutHashState)
	{
		if (!ShadeInput)
		{
			return;
		}

		FScopedUsdAllocs UsdAllocs;

		FString InputName = UsdToUnreal::ConvertToken(ShadeInput.GetBaseName());
		InOutHashState.UpdateWithString(*InputName, InputName.Len());

		FString InputTypeName = UsdToUnreal::ConvertToken(ShadeInput.GetTypeName().GetAsToken());
		InOutHashState.UpdateWithString(*InputTypeName, InputTypeName.Len());

		// Connected to something else, recurse
		pxr::UsdShadeConnectableAPI Source;
		pxr::TfToken SourceName;
		pxr::UsdShadeAttributeType AttributeType;
		if (pxr::UsdShadeConnectableAPI::GetConnectedSource(ShadeInput.GetAttr(), &Source, &SourceName, &AttributeType))
		{
			FString SourceOutputName = UsdToUnreal::ConvertToken(SourceName);
			InOutHashState.UpdateWithString(*SourceOutputName, SourceOutputName.Len());

			// Skip in case our input is connected to an output on the same prim, or else we'll recurse forever
			if (Source.GetPrim() == ShadeInput.GetPrim())
			{
				return;
			}

			for (const pxr::UsdShadeInput& ChildInput : Source.GetInputs())
			{
				HashShadeInput(ChildInput, InOutHashState);
			}
		}
		// Not connected to anything, has a value (this could be a texture file path too)
		else
		{
			pxr::VtValue ShadeInputValue;
			ShadeInput.Get(&ShadeInputValue);

			// We have to manually resolve and hash these file paths or else resolvedpaths inside usdz archives will have upper case driver
			// letters when we first open the stage, but will switch to lower case drive letters if we reload them. This is not something
			// we're doing, as it happens with a pure USD python script. (this with USD 21.05 in June 2021)
			if (ShadeInputValue.IsHolding<pxr::SdfAssetPath>())
			{
				FString ResolvedPath = UsdUtils::GetResolvedAssetPath(ShadeInput.GetAttr());
				InOutHashState.UpdateWithString(*ResolvedPath, ResolvedPath.Len());
			}
			else if (ShadeInputValue.IsHolding<pxr::TfToken>())
			{
				// Stringify instead of using GetHash because if ShadeInputValue contains a pxr::TfToken then
				// it will actually just contain some non-deterministic integer IDs
				FString Stringified = UsdToUnreal::ConvertString(pxr::TfStringify(ShadeInputValue));
				InOutHashState.UpdateWithString(*Stringified, Stringified.Len());
			}
			else
			{
				uint64 ValueHash = (uint64)ShadeInputValue.GetHash();
				InOutHashState.Update(reinterpret_cast<uint8*>(&ValueHash), sizeof(uint64));
			}
		}
	}

	// Fills out the Texture parameters within InOutMaterialData, spawning new UTexture2Ds within
	// the TexturesCache and updating the FTextureParameterValue::Texture member to point at them
	void ResolveTextureParameters(
		const pxr::UsdPrim& MaterialPrim,
		UMaterialInterface& Material,
		UUsdAssetCache3* TexturesCache,
		UsdToUnreal::FUsdPreviewSurfaceMaterialData& InOutMaterialData,
		bool bShareAssetsForIdenticalPrims
	)
	{
		if (!TexturesCache)
		{
			return;
		}

		UUsdAssetUserData* MaterialAssetUserData = UsdUnreal::ObjectUtils::GetAssetUserData(&Material);

		for (TPair<FString, UsdToUnreal::FParameterValue>& Pair : InOutMaterialData.Parameters)
		{
			UsdToUnreal::FTextureParameterValue* TextureParameter = Pair.Value.TryGet<UsdToUnreal::FTextureParameterValue>();
			if (!TextureParameter)
			{
				continue;
			}

			// We'll add these to the hash because the materials are built to try and reuse the same textures for multiple channels,
			// and those may expect linear or sRGB values. Without this we may parse a texture as linear because we hit the opacity
			// channel first, and then reuse it as linear for the base color channel even though it should have been sRGB. Plus we may
			// have something weird like a normal map being plugged into the base color and the normal channel.
			TextureCompressionSettings CompressionSettings = TextureParameter->Group == TEXTUREGROUP_WorldNormalMap
																 ? TextureCompressionSettings::TC_Normalmap
																 : TextureCompressionSettings::TC_Default;
			const bool bSRGB = TextureParameter->GetSRGBValue();

			// Note: We used to use the ShadeInput itself here on the asset hash prefix, which is now a bit awkward to get after the
			// refactor for Interchange. Using the material prim itself should hopefully still be fine though, as it will provide a
			// separate texture for each Material prim itself
			const FString PrefixedTextureHash = UsdUtils::GetAssetHashPrefix(MaterialPrim, bShareAssetsForIdenticalPrims)
												+ UsdUtils::GetTextureHash(	   //
													TextureParameter->TextureFilePath,
													bSRGB,
													CompressionSettings,
													TextureParameter->AddressX,
													TextureParameter->AddressY
												);

			const FString& DesiredTextureName = FPaths::GetBaseFilename(TextureParameter->TextureFilePath);

			const EObjectFlags DesiredFlags = RF_Public | RF_Standalone | RF_Transactional;

			const bool bIsUDIM = TextureParameter->bIsUDIM || TextureParameter->TextureFilePath.Contains(TEXT("<UDIM>"));

			bool bCreatedTexture = false;
			UTexture* Texture = TexturesCache->GetOrCreateCustomCachedAsset<UTexture2D>(
				PrefixedTextureHash,
				DesiredTextureName,
				DesiredFlags,
				[TextureParameter, TexturesCache, bSRGB, bIsUDIM](UPackage* Outer, FName SanitizedName, EObjectFlags DesiredFlags)
				{
					return UsdUtils::CreateTexture(
						TextureParameter->TextureFilePath,
						SanitizedName,
						TextureParameter->Group,
						DesiredFlags,
						Outer,
						!bSRGB,
						bIsUDIM
					);
				},
				&bCreatedTexture
			);

			if (Texture)
			{
				TextureParameter->Texture = Texture;

				const bool bNeedVirtualTextureStreaming = bIsUDIM;

				// The texture resource needs to be updated only if the following settings have changed from their default values
				bool bNeedUpdateResource = false;
				bNeedUpdateResource |= bNeedVirtualTextureStreaming != Texture->VirtualTextureStreaming;
				bNeedUpdateResource |= TextureParameter->AddressX != TextureAddress::TA_Wrap;
				bNeedUpdateResource |= TextureParameter->AddressY != TextureAddress::TA_Wrap;
#if !WITH_EDITOR
				// In editor, these settings are already set on the factory when the texture is created so no further update is needed
				bNeedUpdateResource |= bSRGB != true;
				// TC_Normalmap is set via the TEXTUREGROUP_WorldNormalMap Group
				bNeedUpdateResource |= CompressionSettings != TextureCompressionSettings::TC_Default;
#endif	  // !WITH_EDITOR
				if (bCreatedTexture && bNeedUpdateResource)
				{
#if WITH_EDITOR
					Texture->PreEditChange(nullptr);
#endif	  // WITH_EDITOR
					if (bNeedVirtualTextureStreaming)
					{
						Texture->VirtualTextureStreaming = true;
					}

					Texture->SRGB = bSRGB;
					Texture->CompressionSettings = CompressionSettings;
					if (UTexture2D* Texture2D = Cast<UTexture2D>(Texture))
					{
						Texture2D->AddressX = TextureParameter->AddressX;
						Texture2D->AddressY = TextureParameter->AddressY;
					}
#if WITH_EDITOR
					Texture->PostEditChange();
#else
					Texture->UpdateResource();
#endif	  // WITH_EDITOR
				}

				// Give the same prim paths to the texture, so that it ends up imported right next to the material
				if (MaterialAssetUserData)
				{
					if (UUsdAssetUserData* AssetUserData = UsdUnreal::ObjectUtils::GetOrCreateAssetUserData(Texture))
					{
						for (const FString& MaterialPrimPath : MaterialAssetUserData->PrimPaths)
						{
							AssetUserData->PrimPaths.AddUnique(MaterialPrimPath);
						}
					}
				}
			}
			else
			{
				USD_LOG_USERWARNING(FText::Format(
					LOCTEXT("FailedToParseTexture", "Failed to parse texture at path '{0}'"),
					FText::FromString(TextureParameter->TextureFilePath)
				));
			}
		}
	}
}	 // namespace UE::UsdShadeConversion::Private
namespace UsdShadeConversionImpl = UE::UsdShadeConversion::Private;

bool UsdToUnreal::FTextureParameterValue::GetSRGBValue() const
{
	bool bFallbackSRGB = true;
	// Disable SRGB when parsing float textures, as they're likely specular/roughness maps
	if (Group == TEXTUREGROUP_WorldNormalMap || Group == TEXTUREGROUP_WorldSpecular)
	{
		bFallbackSRGB = false;
	}
	return bSRGB.Get(bFallbackSRGB);
}

bool UsdToUnreal::ConvertMaterial(
	const pxr::UsdPrim& UsdShadeMaterialPrim,
	FUsdPreviewSurfaceMaterialData& OutMaterialData,
	const TCHAR* RenderContext
)
{
	using namespace UsdShadeConversionImpl;

	FScopedUsdAllocs UsdAllocs;

	pxr::UsdShadeMaterial UsdShadeMaterial{UsdShadeMaterialPrim};
	if (!UsdShadeMaterial)
	{
		return false;
	}

	pxr::TfToken RenderContextToken = pxr::UsdShadeTokens->universalRenderContext;
	if (RenderContext)
	{
		pxr::TfToken ProvidedRenderContextToken = UnrealToUsd::ConvertToken(RenderContext).Get();
		RenderContextToken = ProvidedRenderContextToken;
	}

	pxr::UsdShadeShader SurfaceShader = UsdShadeMaterial.ComputeSurfaceSource(RenderContextToken);

	if (!SurfaceShader)
	{
		return false;
	}

	pxr::UsdShadeConnectableAPI Connectable{SurfaceShader};

	UsdToUnreal::FParameterValue ParameterValue;

	// Base color
	if (GetVec3ParameterValue(Connectable, UnrealIdentifiers::DiffuseColor, FLinearColor::Black, ParameterValue))
	{
		OutMaterialData.Parameters.Add(TEXT("BaseColor"), ParameterValue);
	}

	// Emissive color
	if (GetVec3ParameterValue(Connectable, UnrealIdentifiers::EmissiveColor, FLinearColor::Black, ParameterValue))
	{
		OutMaterialData.Parameters.Add(TEXT("EmissiveColor"), ParameterValue);
	}

	// Metallic
	if (GetFloatParameterValue(Connectable, UnrealIdentifiers::Metallic, 0.f, ParameterValue))
	{
		OutMaterialData.Parameters.Add(TEXT("Metallic"), ParameterValue);
	}

	// Roughness
	if (GetFloatParameterValue(Connectable, UnrealIdentifiers::Roughness, 1.f, ParameterValue))
	{
		OutMaterialData.Parameters.Add(TEXT("Roughness"), ParameterValue);
	}

	// SpecularColor
	if (GetVec3ParameterValue(Connectable, UnrealIdentifiers::SpecularColor, FLinearColor::White, ParameterValue))
	{
		OutMaterialData.Parameters.Add(TEXT("SpecularColor"), ParameterValue);
	}

	// Use Specular Workflow
	if (GetIntParameterValue(Connectable, UnrealIdentifiers::UseSpecularWorkflow, 0, ParameterValue))
	{
		OutMaterialData.Parameters.Add(TEXT("UseSpecularWorkflow"), ParameterValue);
	}

	// ClearCoat
	if (GetFloatParameterValue(Connectable, UnrealIdentifiers::ClearCoat, 0, ParameterValue))
	{
		OutMaterialData.Parameters.Add(TEXT("ClearCoat"), ParameterValue);
	}

	// ClearCoat Roughness
	if (GetFloatParameterValue(Connectable, UnrealIdentifiers::ClearCoatRoughness, 0.1f, ParameterValue))
	{
		OutMaterialData.Parameters.Add(TEXT("ClearCoatRoughness"), ParameterValue);
	}

	// Opacity
	if (GetFloatParameterValue(Connectable, UnrealIdentifiers::Opacity, 1.f, ParameterValue))
	{
		OutMaterialData.Parameters.Add(TEXT("Opacity"), ParameterValue);
	}

	// Opacity Mode
	static TMap<FString, float> OpacityModeMap{ {"transparent", 0.f}, {"presence", 1.f} };
	if (GetTokenAsFloatParameterValue(Connectable, UnrealIdentifiers::OpacityMode, "transparent", OpacityModeMap, ParameterValue))
	{
		OutMaterialData.Parameters.Add(TEXT("OpacityMode"), ParameterValue);
	}

	// Opacity Threshold
	if (GetFloatParameterValue(Connectable, UnrealIdentifiers::OpacityThreshold, 0, ParameterValue))
	{
		OutMaterialData.Parameters.Add(TEXT("OpacityThreshold"), ParameterValue);
	}

	// Normal
	if (GetVec3ParameterValue(Connectable, UnrealIdentifiers::Normal, FLinearColor::Blue, ParameterValue, TEXTUREGROUP_WorldNormalMap))
	{
		OutMaterialData.Parameters.Add(TEXT("Normal"), ParameterValue);
	}

	// Refraction
	if (GetFloatParameterValue(Connectable, UnrealIdentifiers::Refraction, 1.5f, ParameterValue))
	{
		OutMaterialData.Parameters.Add(TEXT("Refraction"), ParameterValue);
	}

	// Occlusion
	if (GetFloatParameterValue(Connectable, UnrealIdentifiers::Occlusion, 1.0f, ParameterValue))
	{
		OutMaterialData.Parameters.Add(TEXT("AmbientOcclusion"), ParameterValue);
	}

	// Displacement
	if (GetFloatParameterValue(Connectable, UnrealIdentifiers::Displacement, 0.0f, ParameterValue))
	{
		OutMaterialData.Parameters.Add(TEXT("Displacement"), ParameterValue);
	}

	// UseWorldSpaceNormal
	// This is not part of the UsdPreviewSurface schema, but it's something we export for landscape materials.
	// Unlike the other attributes this is written directly to the Material prim, and not on the surface shader.
	if (pxr::UsdAttribute Attr = UsdShadeMaterialPrim.GetAttribute(UnrealIdentifiers::WorldSpaceNormals))
	{
		bool bValue = false;
		if (Attr.Get<bool>(&bValue) && bValue)
		{
			// Use an int parameter here as actual bool parameters are used for static switches it seems
			UsdToUnreal::FParameterValue IntParameterValue{TInPlaceType<int32>(), static_cast<int32>(bValue)};
			OutMaterialData.Parameters.Add(TEXT("UseWorldSpaceNormals"), IntParameterValue);
		}
	}

	TMap<FString, FString> ParameterToPrimvar;
	TMap<FString, int32> PrimvarToUVIndex;

	// Sort primvars
	TSet<FString> UsedPrimvars;
	for (const TPair<FString, UsdToUnreal::FParameterValue>& Parameter : OutMaterialData.Parameters)
	{
		if (const UsdToUnreal::FTextureParameterValue* TextureParameterValue = Parameter.Value.TryGet<UsdToUnreal::FTextureParameterValue>())
		{
			UsedPrimvars.Add(TextureParameterValue->Primvar);
			ParameterToPrimvar.Add(Parameter.Key, TextureParameterValue->Primvar);
		}
	}
	UsedPrimvars.Remove(TEXT(""));
	TArray<FString> SortedPrimvars = UsedPrimvars.Array();
	SortedPrimvars.Sort();	  // Try for some deterministic ordering (st0 should come before st1, etc.)
	PrimvarToUVIndex.Reserve(SortedPrimvars.Num());
	int32 UVIndex = 0;
	for (const FString& Primvar : SortedPrimvars)
	{
		PrimvarToUVIndex.Add(Primvar, UVIndex++);
	}

	OutMaterialData.PrimvarToUVIndex = PrimvarToUVIndex;

	return true;
}

bool UsdToUnreal::ConvertMaterial(
	const pxr::UsdShadeMaterial& UsdShadeMaterial,
	UMaterialInstance& Material,
	UUsdAssetCache3* TexturesCache,
	const TCHAR* RenderContext,
	bool bShareAssetsForIdenticalPrims
)
{
	pxr::UsdPrim Prim = UsdShadeMaterial.GetPrim();
	if (!Prim)
	{
		return false;
	}

	FUsdPreviewSurfaceMaterialData MaterialData;
	const bool bSuccess = ConvertMaterial(Prim, MaterialData, RenderContext);
	if (!bSuccess)
	{
		return false;
	}

	// Actually create textures if we've been given a place where to put them.
	UsdShadeConversionImpl::ResolveTextureParameters(Prim, Material, TexturesCache, MaterialData, bShareAssetsForIdenticalPrims);

	// Set material parameters on the actual material instance
	const bool bForUsdPreviewSurface = true;
	for (const TPair<FString, UsdToUnreal::FParameterValue>& Parameter : MaterialData.Parameters)
	{
		UsdShadeConversionImpl::SetParameterValue(Material, *Parameter.Key, Parameter.Value, bForUsdPreviewSurface, MaterialData.PrimvarToUVIndex);
	}

	// Record which primvars we used on each UV index. This is important as we'll match this with the analogous member
	// on static/skeletal meshe import data, and create a new material instance with different UV index parameter
	// values if we need to
	if (UUsdMaterialAssetUserData* UserData = Material.GetAssetUserData<UUsdMaterialAssetUserData>())
	{
		// We rebuild this as there's no reason to store this separately on MaterialData itself, as the complete
		// parameter data is stored there now
		TMap<FString, FString> ParameterToPrimvar;
		for (const TPair<FString, UsdToUnreal::FParameterValue>& Parameter : MaterialData.Parameters)
		{
			if (const UsdToUnreal::FTextureParameterValue* TextureParameterValue = Parameter.Value.TryGet<UsdToUnreal::FTextureParameterValue>())
			{
				ParameterToPrimvar.Add(Parameter.Key, TextureParameterValue->Primvar);
			}
		}

		UserData->PrimvarToUVIndex = MaterialData.PrimvarToUVIndex;
		UserData->ParameterToPrimvar = ParameterToPrimvar;
	}

	// We used to only reture true in case we managed to convert at least one parameter, but we don't want
	// callers to interpret that we "failed to convert the material" if we couldn't find any usable parameter
	return true;
}

bool UsdToUnreal::ConvertMaterial(
	const pxr::UsdShadeMaterial& UsdShadeMaterial,
	UMaterial& Material,
	UUsdAssetCache3* TexturesCache,
	const TCHAR* RenderContext,
	bool bShareAssetsForIdenticalPrims
)
{
#if WITH_EDITOR

	pxr::UsdPrim Prim = UsdShadeMaterial.GetPrim();
	if (!Prim)
	{
		return false;
	}

	FUsdPreviewSurfaceMaterialData MaterialData;
	const bool bSuccess = ConvertMaterial(Prim, MaterialData, RenderContext);
	if (!bSuccess)
	{
		return false;
	}

	// Actually create textures if we've been given a place where to put them.
	UsdShadeConversionImpl::ResolveTextureParameters(Prim, Material, TexturesCache, MaterialData, bShareAssetsForIdenticalPrims);

	// auto so we can use the MaterialInput generic parameter for FColorMaterialInput, FScalarMaterialInput and FVectorMaterialInput
	auto ConnectMaterialInput = [&MaterialData](UMaterial& Material, auto& MaterialInput, const FString& ParameterName) -> bool
	{
		const UsdToUnreal::FParameterValue* ParameterValue = MaterialData.Parameters.Find(ParameterName);
		if (!ParameterValue)
		{
			return false;
		}

		UMaterialExpression* Expression = UsdShadeConversionImpl::GetExpressionForValue(Material, *ParameterValue);
		if (!Expression)
		{
			return false;
		}

		MaterialInput.Expression = Expression;

		if (const UsdToUnreal::FTextureParameterValue* TextureParameterValue = ParameterValue->TryGet<UsdToUnreal::FTextureParameterValue>())
		{
			MaterialInput.OutputIndex = TextureParameterValue->OutputIndex;

			if (UMaterialExpressionTextureSample* TextureExpression = Cast<UMaterialExpressionTextureSample>(Expression))
			{
				if (int32* FoundCoordinate = MaterialData.PrimvarToUVIndex.Find(TextureParameterValue->Primvar))
				{
					TextureExpression->ConstCoordinate = *FoundCoordinate;
				}
				else
				{
					USD_LOG_WARNING(
						TEXT("Failed to find primvar '%s' when setting material parameter. Available primvars and UV indices: %s.%s"),
						*TextureParameterValue->Primvar,
						*UsdUtils::StringifyMap(MaterialData.PrimvarToUVIndex),
						TextureParameterValue->Primvar.IsEmpty() ? TEXT(" Is your UsdUVTexture Shader missing the 'inputs:st' attribute? (It "
																		"specifies which UV set to sample the texture with)")
																 : TEXT("")
					);
				}
			}
		}

		return true;
	};

	UMaterialEditorOnlyData* EditorOnly = Material.GetEditorOnlyData();

	ConnectMaterialInput(Material, EditorOnly->BaseColor, TEXT("BaseColor"));
	ConnectMaterialInput(Material, EditorOnly->EmissiveColor, TEXT("EmissiveColor"));
	ConnectMaterialInput(Material, EditorOnly->Metallic, TEXT("Metallic"));
	ConnectMaterialInput(Material, EditorOnly->Roughness, TEXT("Roughness"));
	if (ConnectMaterialInput(Material, EditorOnly->Opacity, TEXT("Opacity")))
	{
		Material.BlendMode = BLEND_Translucent;
	}
	ConnectMaterialInput(Material, EditorOnly->Normal, TEXT("Normal"));
	ConnectMaterialInput(Material, EditorOnly->Refraction, TEXT("Refraction"));
	ConnectMaterialInput(Material, EditorOnly->AmbientOcclusion, TEXT("AmbientOcclusion"));

	// Handle world space normals
	pxr::UsdPrim UsdPrim = UsdShadeMaterial.GetPrim();
	if (pxr::UsdAttribute Attr = UsdPrim.GetAttribute(UnrealIdentifiers::WorldSpaceNormals))
	{
		bool bValue = false;
		if (Attr.Get<bool>(&bValue) && bValue)
		{
			Material.bTangentSpaceNormal = false;
		}
	}

	// Record which primvars we used on each UV index. This is important as we'll match this with the analogous member
	// on static/skeletal meshe import data, and create a new material instance with different UV index parameter
	// values if we need to
	if (UUsdMaterialAssetUserData* UserData = Material.GetAssetUserData<UUsdMaterialAssetUserData>())
	{
		// We rebuild this as there's no reason to store this separately on MaterialData itself, as the complete
		// parameter data is stored there now
		TMap<FString, FString> ParameterToPrimvar;
		for (const TPair<FString, UsdToUnreal::FParameterValue>& Parameter : MaterialData.Parameters)
		{
			if (const UsdToUnreal::FTextureParameterValue* TextureParameterValue = Parameter.Value.TryGet<UsdToUnreal::FTextureParameterValue>())
			{
				ParameterToPrimvar.Add(Parameter.Key, TextureParameterValue->Primvar);
			}
		}

		UserData->PrimvarToUVIndex = MaterialData.PrimvarToUVIndex;
		UserData->ParameterToPrimvar = ParameterToPrimvar;
	}

	return true;
#else
	return false;
#endif	  // WITH_EDITOR
}

bool UsdToUnreal::ConvertShadeInputsToParameters(
	const pxr::UsdShadeMaterial& UsdShadeMaterial,
	UMaterialInstance& MaterialInstance,
	UUsdAssetCache3* TexturesCache,
	const TCHAR* RenderContext,
	bool bShareAssetsForIdenticalPrims
)
{
	using namespace UsdShadeConversionImpl;

	FScopedUsdAllocs UsdAllocs;

	pxr::TfToken RenderContextToken = pxr::UsdShadeTokens->universalRenderContext;
	if (RenderContext)
	{
		RenderContextToken = UnrealToUsd::ConvertToken(RenderContext).Get();
	}

	pxr::UsdShadeShader SurfaceShader = UsdShadeMaterial.ComputeSurfaceSource(RenderContextToken);
	if (!SurfaceShader)
	{
		return false;
	}

	pxr::UsdShadeConnectableAPI Connectable{SurfaceShader};

	bool bHasMaterialInfo = false;
	for (pxr::UsdShadeInput& ShadeInput : SurfaceShader.GetInputs())
	{
		FString InputName = UsdToUnreal::ConvertToken(ShadeInput.GetBaseName());

		pxr::UsdShadeInput ConnectInput = ShadeInput;
		if (ShadeInput.HasConnectedSource())
		{
			pxr::UsdShadeConnectableAPI Source;
			pxr::TfToken SourceName;
			pxr::UsdShadeAttributeType SourceType;
			pxr::UsdShadeConnectableAPI::GetConnectedSource(ShadeInput.GetAttr(), &Source, &SourceName, &SourceType);

			ConnectInput = Source.GetInput(SourceName);
		}

		if (!ConnectInput)
		{
			continue;
		}

		FString DisplayName = UsdToUnreal::ConvertString(ConnectInput.GetAttr().GetDisplayName());
		if (DisplayName.IsEmpty())
		{
			DisplayName = InputName;
		}

		UsdToUnreal::FParameterValue ParameterValue;
		const bool bForUsdPreviewSurface = false;
		const TextureGroup Group = DisplayName.Contains(TEXT("normal")) ? TEXTUREGROUP_WorldNormalMap : TEXTUREGROUP_World;

		// For now it seems we don't set "primvar parameters" anyway, so don't bother building this up
		TMap<FString, int32> Unused;

		if (ShadeInput.GetTypeName() == pxr::SdfValueTypeNames->Bool)
		{
			if (GetBoolParameterValue(Connectable, ShadeInput.GetBaseName(), false, ParameterValue))
			{
				SetParameterValue(MaterialInstance, *DisplayName, ParameterValue, bForUsdPreviewSurface, Unused);
				bHasMaterialInfo = true;
			}
		}
		else if (ShadeInput.GetTypeName() == pxr::SdfValueTypeNames->Float || ShadeInput.GetTypeName() == pxr::SdfValueTypeNames->Double
				 || ShadeInput.GetTypeName() == pxr::SdfValueTypeNames->Half)
		{
			if (GetFloatParameterValue(Connectable, ShadeInput.GetBaseName(), 1.f, ParameterValue))
			{
				SetParameterValue(MaterialInstance, *DisplayName, ParameterValue, bForUsdPreviewSurface, Unused);
				bHasMaterialInfo = true;
			}
		}
		else if (GetVec3ParameterValue(Connectable, ShadeInput.GetBaseName(), FLinearColor::Black, ParameterValue, Group))
		{
			SetParameterValue(MaterialInstance, *DisplayName, ParameterValue, bForUsdPreviewSurface, Unused);
			bHasMaterialInfo = true;
		}
	}

	return true;
}

#if WITH_EDITOR
bool UnrealToUsd::ConvertMaterialToBakedSurface(
	const UMaterialInterface& InMaterial,
	const TArray<FPropertyEntry>& InMaterialProperties,
	const FIntPoint& InDefaultTextureSize,
	const FDirectoryPath& InTexturesDir,
	pxr::UsdPrim& OutUsdShadeMaterialPrim,
	bool bInDecayTexturesToSinglePixel
)
{
	pxr::UsdShadeMaterial OutUsdShadeMaterial{OutUsdShadeMaterialPrim};
	if (!OutUsdShadeMaterial)
	{
		return false;
	}

	FBakeOutput BakedData;
	if (!UsdShadeConversionImpl::BakeMaterial(InMaterial, InMaterialProperties, InDefaultTextureSize, BakedData, bInDecayTexturesToSinglePixel))
	{
		return false;
	}

	UsdShadeConversionImpl::FBakedMaterialView View{BakedData};
	TMap<EMaterialProperty, FString> WrittenTextures = UsdShadeConversionImpl::WriteTextures(View, InMaterial.GetName(), InTexturesDir);

	// Manually add user supplied constant values. Can't place these in InMaterial as they're floats, and baked data is just quantized FColors
	TMap<EMaterialProperty, float> UserConstantValues;
	for (const FPropertyEntry& Entry : InMaterialProperties)
	{
		if (Entry.bUseConstantValue)
		{
			UserConstantValues.Add(Entry.Property, Entry.ConstantValue);
		}
	}

	return UsdShadeConversionImpl::ConfigureShadePrim(View, WrittenTextures, UserConstantValues, OutUsdShadeMaterial);
}

bool UnrealToUsd::ConvertFlattenMaterial(
	const FString& InMaterialName,
	FFlattenMaterial& InMaterial,
	const TArray<FPropertyEntry>& InMaterialProperties,
	const FDirectoryPath& InTexturesDir,
	UE::FUsdPrim& OutUsdShadeMaterialPrim
)
{
	pxr::UsdShadeMaterial OutUsdShadeMaterial{pxr::UsdPrim{OutUsdShadeMaterialPrim}};
	if (!OutUsdShadeMaterial)
	{
		return false;
	}

	UsdShadeConversionImpl::FBakedMaterialView View{InMaterial};
	TMap<EMaterialProperty, FString> WrittenTextures = UsdShadeConversionImpl::WriteTextures(View, InMaterialName, InTexturesDir);

	// Manually add user supplied constant values. Can't place these in InMaterial as they're floats, and baked data is just quantized FColors
	TMap<EMaterialProperty, float> UserConstantValues;
	for (const FPropertyEntry& Entry : InMaterialProperties)
	{
		if (Entry.bUseConstantValue)
		{
			UserConstantValues.Add(Entry.Property, Entry.ConstantValue);
		}
	}

	return UsdShadeConversionImpl::ConfigureShadePrim(View, WrittenTextures, UserConstantValues, OutUsdShadeMaterial);
}

#endif	  // WITH_EDITOR

FString UsdUtils::GetResolvedAssetPath(const pxr::UsdAttribute& AssetPathAttr, pxr::UsdTimeCode TimeCode)
{
	if (!AssetPathAttr)
	{
		return {};
	}

	FScopedUsdAllocs UsdAllocs;

	pxr::SdfAssetPath AssetPath;
	AssetPathAttr.Get<pxr::SdfAssetPath>(&AssetPath, TimeCode);

	std::string AssetIdentifier = AssetPath.GetResolvedPath();
	// Don't normalize an empty path as the result will be "."
	if (AssetIdentifier.size() > 0)
	{
		pxr::ArResolver& Resolver = pxr::ArGetResolver();
		AssetIdentifier = Resolver.CreateIdentifier(AssetIdentifier);
	}

	FString ResolvedPath = UsdToUnreal::ConvertString(AssetIdentifier);

	if (ResolvedPath.IsEmpty())
	{
		FString TexturePath = UsdToUnreal::ConvertString(AssetPath.GetAssetPath());
		FPaths::NormalizeFilename(TexturePath);

		if (!TexturePath.IsEmpty())
		{
			pxr::SdfLayerRefPtr TextureLayer = UsdUtils::FindLayerForAttribute(AssetPathAttr, TimeCode.GetValue());
			ResolvedPath = UsdShadeConversionImpl::ResolveAssetPath(TextureLayer, TexturePath);
		}
	}

	FPaths::NormalizeFilename(ResolvedPath);
	return ResolvedPath;
}

FString UsdUtils::GetTextureHash(
	const FString& ResolvedTexturePath,
	bool bSRGB,
	TextureCompressionSettings CompressionSettings,
	TextureAddress AddressX,
	TextureAddress AddressY
)
{
	using namespace UE::UsdShadeConversion::Private;

	FMD5 MD5;

	// Hash the actual texture
	FString TextureExtension;
	if (IsInsideUsdzArchive(ResolvedTexturePath, TextureExtension))
	{
		uint64 BufferSize = 0;
		TUsdStore<std::shared_ptr<const char>> Buffer = ReadTextureBufferFromUsdzArchive(ResolvedTexturePath, BufferSize);
		const uint8* BufferStart = reinterpret_cast<const uint8*>(Buffer.Get().get());

		if (BufferSize > 0 && BufferStart != nullptr)
		{
			MD5.Update(BufferStart, BufferSize);
		}
	}
	// Copied from FMD5Hash::HashFileFromArchive as it doesn't expose its FMD5
	else if (TUniquePtr<FArchive> Ar{IFileManager::Get().CreateFileReader(*ResolvedTexturePath)})
	{
		TArray<uint8> LocalScratch;
		LocalScratch.SetNumUninitialized(1024 * 64);

		const int64 Size = Ar->TotalSize();
		int64 Position = 0;

		// Read in BufferSize chunks
		while (Position < Size)
		{
			const auto ReadNum = FMath::Min(Size - Position, (int64)LocalScratch.Num());
			Ar->Serialize(LocalScratch.GetData(), ReadNum);
			MD5.Update(LocalScratch.GetData(), ReadNum);

			Position += ReadNum;
		}
	}
	else
	{
		USD_LOG_WARNING(TEXT("Failed to find texture at path '%s' when trying to generate a hash for it"), *ResolvedTexturePath);
	}

	// Hash the additional data
	MD5.Update(reinterpret_cast<uint8*>(&bSRGB), sizeof(bool));
	MD5.Update(reinterpret_cast<uint8*>(&CompressionSettings), sizeof(CompressionSettings));
	MD5.Update(reinterpret_cast<uint8*>(&AddressX), sizeof(AddressX));
	MD5.Update(reinterpret_cast<uint8*>(&AddressY), sizeof(AddressY));

	FMD5Hash Hash;
	Hash.Set(MD5);
	return LexToString(Hash);
}

UTexture* UsdUtils::CreateTexture(
	const FString& ResolvedTexturePath,
	FName SanitizedName,
	TextureGroup Group,
	EObjectFlags Flags,
	UObject* Outer,
	bool bForceLinear,
	bool bIsUDIMPath
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UsdUtils::CreateTexture);

	// Standalone game does have WITH_EDITOR defined, but it can't use the texture factories, so we need to check this instead
	if (GIsEditor)
	{
		return UsdShadeConversionImpl::CreateTextureWithEditor(ResolvedTexturePath, SanitizedName, Group, Flags, Outer, bForceLinear, bIsUDIMPath);
	}
	else
	{
		return UsdShadeConversionImpl::CreateTextureAtRuntime(ResolvedTexturePath);
	}
}

void UsdUtils::NotifyIfVirtualTexturesNeeded(UTexture* Texture)
{
	if (!Texture || !Texture->VirtualTextureStreaming)
	{
		return;
	}

	FString TexturePath = Texture->GetName();
	if (UUsdAssetUserData* UserData = Texture->GetAssetUserData<UUsdAssetUserData>())
	{
		if (!UserData->PrimPaths.IsEmpty())
		{
			TexturePath = UserData->PrimPaths[0];
		}
	}

	if (!UseVirtualTexturing(GMaxRHIShaderPlatform))
	{
		USD_LOG_USERWARNING(FText::Format(
			LOCTEXT(
				"DisabledVirtualTexturing",
				"Texture '{0}' (from prim '{1}') requires Virtual Textures, but the feature is disabled for this project"
			),
			FText::FromString(Texture->GetName()),
			FText::FromString(TexturePath)
		));
	}
}

#if WITH_EDITOR
EFlattenMaterialProperties UsdUtils::MaterialPropertyToFlattenProperty(EMaterialProperty MaterialProperty)
{
	const static TMap<EMaterialProperty, EFlattenMaterialProperties> InvertedMap = []()
	{
		TMap<EMaterialProperty, EFlattenMaterialProperties> Result;
		Result.Reserve(UsdShadeConversionImpl::FlattenToMaterialProperty.Num());

		for (const TPair<EFlattenMaterialProperties, EMaterialProperty>& EnumPair : UsdShadeConversionImpl::FlattenToMaterialProperty)
		{
			Result.Add(EnumPair.Value, EnumPair.Key);
		}

		return Result;
	}();

	if (const EFlattenMaterialProperties* FoundConversion = InvertedMap.Find(MaterialProperty))
	{
		return *FoundConversion;
	}

	return EFlattenMaterialProperties::NumFlattenMaterialProperties;
}

EMaterialProperty UsdUtils::FlattenPropertyToMaterialProperty(EFlattenMaterialProperties FlattenProperty)
{
	if (const EMaterialProperty* FoundConversion = UsdShadeConversionImpl::FlattenToMaterialProperty.Find(FlattenProperty))
	{
		return *FoundConversion;
	}

	return EMaterialProperty::MP_MAX;
}

void UsdUtils::CollapseConstantChannelsToSinglePixel(FFlattenMaterial& InMaterial)
{
	auto ConvertSamplesInPlace = [](TArray<FColor>& Samples) -> bool
	{
		if (Samples.Num() < 2)
		{
			return false;
		}

		FColor ConstantValue = Samples[0];
		bool bResize = Algo::AllOf(
			Samples,
			[&ConstantValue](const FColor& Sample)
			{
				return Sample == ConstantValue;
			}
		);
		if (bResize)
		{
			Samples.SetNum(1);
			return true;
		}

		return false;
	};

	for (const TPair<EFlattenMaterialProperties, EMaterialProperty>& EnumPair : UsdShadeConversionImpl::FlattenToMaterialProperty)
	{
		const EFlattenMaterialProperties Property = EnumPair.Key;
		bool bResized = ConvertSamplesInPlace(InMaterial.GetPropertySamples(Property));
		if (bResized)
		{
			InMaterial.SetPropertySize(Property, FIntPoint(1, 1));
		}
	}
}
#endif	  // WITH_EDITOR

bool UsdUtils::MarkMaterialPrimWithWorldSpaceNormals(const UE::FUsdPrim& MaterialPrim)
{
	FScopedUsdAllocs UsdAllocs;

	pxr::UsdPrim UsdPrim{MaterialPrim};
	if (!UsdPrim)
	{
		return false;
	}

	const bool bCustom = true;
	pxr::UsdAttribute Attr = UsdPrim.CreateAttribute(UnrealIdentifiers::WorldSpaceNormals, pxr::SdfValueTypeNames->Bool, bCustom);
	if (!Attr)
	{
		return false;
	}

	Attr.Set<bool>(true);
	UsdUtils::NotifyIfOverriddenOpinion(Attr);
	return true;
}

void UsdUtils::SetScalarParameterValue(UMaterialInstance& Material, const TCHAR* ParameterName, float ParameterValue)
{
	FMaterialParameterInfo Info;
	Info.Name = ParameterName;

	if (UMaterialInstanceDynamic* Dynamic = Cast<UMaterialInstanceDynamic>(&Material))
	{
		Dynamic->SetScalarParameterValueByInfo(Info, ParameterValue);
	}
#if WITH_EDITOR
	else if (UMaterialInstanceConstant* Constant = Cast<UMaterialInstanceConstant>(&Material))
	{
		Constant->SetScalarParameterValueEditorOnly(Info, ParameterValue);
	}
#endif	  // WITH_EDITOR
}

void UsdUtils::SetVectorParameterValue(UMaterialInstance& Material, const TCHAR* ParameterName, FLinearColor ParameterValue)
{
	FMaterialParameterInfo Info;
	Info.Name = ParameterName;

	if (UMaterialInstanceDynamic* Dynamic = Cast<UMaterialInstanceDynamic>(&Material))
	{
		Dynamic->SetVectorParameterValueByInfo(Info, ParameterValue);
	}
#if WITH_EDITOR
	else if (UMaterialInstanceConstant* Constant = Cast<UMaterialInstanceConstant>(&Material))
	{
		Constant->SetVectorParameterValueEditorOnly(Info, ParameterValue);
	}
#endif	  // WITH_EDITOR
}

void UsdUtils::SetTextureParameterValue(UMaterialInstance& Material, const TCHAR* ParameterName, UTexture* ParameterValue)
{
	FMaterialParameterInfo Info;
	Info.Name = ParameterName;

	if (UMaterialInstanceDynamic* Dynamic = Cast<UMaterialInstanceDynamic>(&Material))
	{
		Dynamic->SetTextureParameterValueByInfo(Info, ParameterValue);
	}
#if WITH_EDITOR
	else if (UMaterialInstanceConstant* Constant = Cast<UMaterialInstanceConstant>(&Material))
	{
		Constant->SetTextureParameterValueEditorOnly(Info, ParameterValue);
	}
#endif	  // WITH_EDITOR
}

void UsdUtils::SetBoolParameterValue(UMaterialInstance& Material, const TCHAR* ParameterName, bool bParameterValue)
{
	bool bFound = false;

#if WITH_EDITOR
	// Try the static parameters first
	if (UMaterialInstanceConstant* Constant = Cast<UMaterialInstanceConstant>(&Material))
	{
		bool bNeedsUpdatePermutations = false;

		FStaticParameterSet StaticParameters;
		Constant->GetStaticParameterValues(StaticParameters);

		for (FStaticSwitchParameter& StaticSwitchParameter : StaticParameters.StaticSwitchParameters)
		{
			if (StaticSwitchParameter.ParameterInfo.Name == ParameterName)
			{
				bFound = true;

				if (StaticSwitchParameter.Value != bParameterValue)
				{
					StaticSwitchParameter.Value = bParameterValue;
					StaticSwitchParameter.bOverride = true;
					bNeedsUpdatePermutations = true;
				}

				break;
			}
		}

		if (bNeedsUpdatePermutations)
		{
			FlushRenderingCommands();
			Constant->UpdateStaticPermutation(StaticParameters);
		}
	}
#endif	  // WITH_EDITOR

	// Try it as a scalar parameter
	if (!bFound)
	{
		SetScalarParameterValue(Material, ParameterName, bParameterValue ? 1.0f : 0.0f);
	}
}

#if WITH_EDITOR
TArray<FString> UsdUtils::GetMaterialXFilePaths(const pxr::UsdPrim& Prim)
{
	FScopedUsdAllocs UsdAllocs;

	TArray<FString> Result;

	auto AddMaterialXFilePath =
		[&Result](const pxr::UsdPrimCompositionQueryArc& CompositionArc, auto ReferenceOrPayloadEditor, auto UsdReferenceOrPayload)
	{
		if (CompositionArc.GetIntroducingListEditor(&ReferenceOrPayloadEditor, &UsdReferenceOrPayload))
		{
			// The mtlx file will be the "target layer". It's useful to get its real path via the SdfLayer
			// interface as it already makes sure it is absolute
			pxr::SdfLayerHandle TargetLayer = CompositionArc.GetTargetLayer();
			FString AbsoluteFilePath = UsdToUnreal::ConvertString(TargetLayer->GetRealPath());

			FString Extension = FPaths::GetExtension(AbsoluteFilePath);
			if (Extension == UnrealIdentifiers::MaterialXRenderContext && FPaths::FileExists(AbsoluteFilePath))
			{
				Result.Add(AbsoluteFilePath);
			}
		}
	};

	// We used to just fetch "direct references" here, but stages may compose the .mtlx file reference onto the prim
	// via another sublayer or reference, so it will be marked as an "ancestral arc" and not be included in the "direct references" filter
	pxr::UsdPrimCompositionQuery PrimCompositionQuery = pxr::UsdPrimCompositionQuery{Prim};
	for (const pxr::UsdPrimCompositionQueryArc& CompositionArc : PrimCompositionQuery.GetCompositionArcs())
	{
		if (CompositionArc.GetArcType() == pxr::PcpArcTypeReference)
		{
			AddMaterialXFilePath(CompositionArc, pxr::SdfReferenceEditorProxy{}, pxr::SdfReference{});
		}
		else if (CompositionArc.GetArcType() == pxr::PcpArcType::PcpArcTypePayload)
		{
			AddMaterialXFilePath(CompositionArc, pxr::SdfPayloadEditorProxy{}, pxr::SdfPayload{});
		}
	}

	return Result;
}
#endif	  // WITH_EDITOR

void UsdUtils::AuthorUnrealMaterialBinding(pxr::UsdPrim& MeshOrGeomSubsetPrim, const FString& UnrealMaterialPathName, bool bStrongerThanDescendants)
{
	if (!MeshOrGeomSubsetPrim || UnrealMaterialPathName.IsEmpty())
	{
		return;
	}

	FScopedUsdAllocs UsdAllocs;

	pxr::UsdShadeMaterialBindingAPI BindingAPI = pxr::UsdShadeMaterialBindingAPI::Apply(MeshOrGeomSubsetPrim);

	// If this mesh prim already has a binding to a *child* material with the 'unreal' render context,
	// just write our material there and early out
	if (pxr::UsdShadeMaterial ShadeMaterial = BindingAPI.ComputeBoundMaterial())
	{
		// We need to try reusing these materials or else we'd write a new material prim every time we change
		// the override in UE, but we also run the risk of modifying a material that is used by multiple prims
		// (and here we just want to set the override for this Mesh prim). The compromise is to only reuse the
		// material if it is a child of MeshPrim already, and always to author our material prims as children
		std::string MaterialPath = ShadeMaterial.GetPrim().GetPath().GetString();
		std::string MeshPrimPath = MeshOrGeomSubsetPrim.GetPath().GetString();
		if (MaterialPath.rfind(MeshPrimPath, 0) == 0)
		{
			if (pxr::UsdPrim MaterialPrim = ShadeMaterial.GetPrim())
			{
				UsdUtils::SetUnrealSurfaceOutput(MaterialPrim, UnrealMaterialPathName);
				return;
			}
		}
	}

	// Find a unique name for our child material prim
	// Note how we'll always author these materials as children of the meshes themselves instead of emitting a common
	// Material prim to use for multiple overrides: This because in the future we'll want to have a separate material
	// bake for each mesh (to make sure we get vertex color effects, etc.), and so we'd have multiple baked .usda material
	// asset layers for each UE material, and we'd want each mesh/section/LOD to refer to its own anyway
	FString ChildMaterialName = TEXT("UnrealMaterial");
	if (pxr::UsdPrim ExistingPrim = MeshOrGeomSubsetPrim.GetChild(UnrealToUsd::ConvertToken(*ChildMaterialName).Get()))
	{
		// Get a unique name for a new prim. Don't even try checking if this prim is usable as the material binding,
		// because if it was the material binding for this mesh we would have already used it above, when fetching the ExistingShader.
		// If we're here, we don't know what this prim is about
		TSet<FString> UsedNames;
		for (pxr::UsdPrim Child : MeshOrGeomSubsetPrim.GetFilteredChildren(pxr::UsdTraverseInstanceProxies(pxr::UsdPrimAllPrimsPredicate)))
		{
			UsedNames.Add(UsdToUnreal::ConvertToken(Child.GetName()));
		}

		ChildMaterialName = UsdUnreal::ObjectUtils::GetUniqueName(ChildMaterialName, UsedNames);
	}

	pxr::UsdStageRefPtr Stage = MeshOrGeomSubsetPrim.GetStage();
	pxr::SdfPath MeshPath = MeshOrGeomSubsetPrim.GetPath();
	pxr::SdfPath MaterialPath = MeshPath.AppendChild(UnrealToUsd::ConvertToken(*ChildMaterialName).Get());

	pxr::UsdShadeMaterial ChildMaterial = pxr::UsdShadeMaterial::Define(Stage, MaterialPath);
	if (!ChildMaterial)
	{
		USD_LOG_WARNING(
			TEXT("Failed to author material prim '%s' when trying to write '%s's material assignment '%s' to USD"),
			*UsdToUnreal::ConvertPath(MaterialPath),
			*UsdToUnreal::ConvertPath(MeshOrGeomSubsetPrim.GetPath()),
			*UnrealMaterialPathName
		);
		return;
	}

	if (pxr::UsdPrim MaterialPrim = ChildMaterial.GetPrim())
	{
		UsdUtils::SetUnrealSurfaceOutput(MaterialPrim, UnrealMaterialPathName);

		BindingAPI.Bind(
			ChildMaterial,
			bStrongerThanDescendants ? pxr::UsdShadeTokens->strongerThanDescendants : pxr::UsdShadeTokens->fallbackStrength
		);
	}
}

void UsdUtils::AuthorUnrealCollectionBasedMaterialBinding(
	const pxr::UsdPrim& CollectionPrim,
	const pxr::UsdPrim& TargetMeshOrGeomSubsetPrim,
	const FString& UnrealMaterialPathName
)
{
	if (!CollectionPrim || CollectionPrim.IsInstanceProxy() || !TargetMeshOrGeomSubsetPrim || UnrealMaterialPathName.IsEmpty())
	{
		return;
	}

	FScopedUsdAllocs UsdAllocs;

	// For collection-based bindings to do anything, TargetMeshOrGeomSubsetPrim must be a descendant of CollectionPrim
	pxr::UsdPrim Iter = TargetMeshOrGeomSubsetPrim.GetParent();
	while (Iter != CollectionPrim)
	{
		if (!Iter || Iter.IsPseudoRoot())
		{
			USD_LOG_WARNING(
				TEXT(
					"Failed to author collection-based material bindings on prim '%s' for target prim '%s', as the latter is not a descendent of the former"
				),
				*UsdToUnreal::ConvertPath(CollectionPrim.GetPath()),
				*UsdToUnreal::ConvertPath(TargetMeshOrGeomSubsetPrim.GetPath())
			);
			return;
		}
		Iter = Iter.GetParent();
	}

	pxr::UsdStageRefPtr Stage = CollectionPrim.GetStage();

	const static pxr::TfToken& BindingPurpose = pxr::UsdShadeTokens->allPurpose;
	pxr::UsdShadeMaterialBindingAPI BindingAPI = pxr::UsdShadeMaterialBindingAPI::Apply(CollectionPrim);
	std::vector<pxr::UsdShadeMaterialBindingAPI::CollectionBinding> ExistingBindings = BindingAPI.GetCollectionBindings(BindingPurpose);

	// Check to see if we happen to have a collection based binding for our target material already. This can happen
	// if we're e.g. setting multiple prims inside of an instance with the same material override
	pxr::UsdShadeMaterialBindingAPI::CollectionBinding CollectionBinding;
	pxr::UsdShadeMaterial BoundMaterial;
	{
		for (const pxr::UsdShadeMaterialBindingAPI::CollectionBinding& Binding : ExistingBindings)
		{
			pxr::UsdShadeMaterial Material = Binding.GetMaterial();

			TOptional<FString> ExistingUnrealMaterialPath = UsdUtils::GetUnrealSurfaceOutput(Material.GetPrim());
			if (ExistingUnrealMaterialPath.IsSet() && ExistingUnrealMaterialPath.GetValue() == UnrealMaterialPathName)
			{
				CollectionBinding = Binding;
				BoundMaterial = CollectionBinding.GetMaterial();
				break;
			}
		}
	}

	// If the CollectionPrim is an instance, we won't be able to author any material inside of it and must instead
	// settle for creating sibling Material prims
	pxr::UsdPrim MaterialParent = CollectionPrim.IsInstance() ? CollectionPrim.GetParent() : CollectionPrim;

	// We don't have a collection-based material binding to our target material yet.
	// Double-check we don't have any existing child Material on this CollectionPrim that we can just reuse though
	if (!BoundMaterial)
	{
		for (const pxr::UsdPrim& Child : MaterialParent.GetFilteredChildren(pxr::UsdTraverseInstanceProxies(pxr::UsdPrimAllPrimsPredicate)))
		{
			TOptional<FString> ExistingUnrealMaterialPath = UsdUtils::GetUnrealSurfaceOutput(Child.GetPrim());
			if (ExistingUnrealMaterialPath.IsSet() && ExistingUnrealMaterialPath.GetValue() == UnrealMaterialPathName)
			{
				BoundMaterial = pxr::UsdShadeMaterial{Child};
				break;
			}
		}
	}

	// Need to create a brand new material with an "unreal" surface output that just points at our target material
	if (!BoundMaterial)
	{
		// Find a unique name for our child material prim
		// Note how we'll always author these materials as children of the meshes themselves instead of emitting a common
		// Material prim to use for multiple overrides: This because in the future we'll want to have a separate material
		// bake for each mesh (to make sure we get vertex color effects, etc.), and so we'd have multiple baked .usda material
		// asset layers for each UE material, and we'd want each mesh/section/LOD to refer to its own anyway
		FString ChildMaterialName = TEXT("UnrealMaterial");
		if (pxr::UsdPrim ExistingPrim = MaterialParent.GetChild(UnrealToUsd::ConvertToken(*ChildMaterialName).Get()))
		{
			// Get a unique name for a new prim. Don't even try checking if this prim is usable as the material binding,
			// because if it was we would have already found it above
			TSet<FString> UsedNames;
			for (pxr::UsdPrim Child : MaterialParent.GetFilteredChildren(pxr::UsdTraverseInstanceProxies(pxr::UsdPrimAllPrimsPredicate)))
			{
				UsedNames.Add(UsdToUnreal::ConvertToken(Child.GetName()));
			}

			ChildMaterialName = UsdUnreal::ObjectUtils::GetUniqueName(ChildMaterialName, UsedNames);
		}

		pxr::SdfPath MaterialParentPath = MaterialParent.GetPath();
		pxr::SdfPath MaterialPath = MaterialParentPath.AppendChild(UnrealToUsd::ConvertToken(*ChildMaterialName).Get());

		BoundMaterial = pxr::UsdShadeMaterial::Define(Stage, MaterialPath);
		if (!BoundMaterial)
		{
			USD_LOG_WARNING(
				TEXT("Failed to author material prim '%s' when trying to write '%s's collection-based material assignment '%s' to USD"),
				*UsdToUnreal::ConvertPath(MaterialPath),
				*UsdToUnreal::ConvertPath(CollectionPrim.GetPath()),
				*UnrealMaterialPathName
			);
			return;
		}

		if (pxr::UsdPrim MaterialPrim = BoundMaterial.GetPrim())
		{
			UsdUtils::SetUnrealSurfaceOutput(MaterialPrim, UnrealMaterialPathName);
		}
	}

	const static pxr::TfToken& BindingStrength = pxr::UsdShadeTokens->strongerThanDescendants;

	// Get the target collection name
	pxr::TfToken CollectionName;
	if (CollectionBinding.IsValid())
	{
		// We already have a collection based binding to our target material, let's just use it
		CollectionName = CollectionBinding.GetCollection().GetName();
	}
	else
	{
		// We don't have a collection binding to our target material yet
		pxr::UsdCollectionAPI CollectionToUse;

		// Before we try creating a brand new collection, let's see if our target prim is already the single prim
		// targetted by other existing collections, because if it is we could just make that collection point at
		// our new material instead.
		//
		// This is useful because otherwise swapping between 3 different material overrides on the exact same
		// component would generate 3 separate collections on the prim, author a bunch of "delete" opinions, and
		// overall just make a mess. With this snippet we just update the material on the same collection instead
		for (const pxr::UsdShadeMaterialBindingAPI::CollectionBinding& Binding : ExistingBindings)
		{
			pxr::TfToken ExistingStrength = BindingAPI.GetMaterialBindingStrength(Binding.GetBindingRel());
			if (ExistingStrength != BindingStrength)
			{
				// Only pick collections that looks like the one we'd create though
				continue;
			}

			pxr::UsdCollectionAPI ExistingCollection = Binding.GetCollection();
			std::set<pxr::SdfPath> IncludedPaths = ExistingCollection.ComputeIncludedPaths(ExistingCollection.ComputeMembershipQuery(), Stage);
			if (IncludedPaths.size() == 1 && (*IncludedPaths.begin()) == TargetMeshOrGeomSubsetPrim.GetPrimPath())
			{
				CollectionToUse = ExistingCollection;
				break;
			}
		}

		// We need to create a brand new collection
		if (!CollectionToUse)
		{
			// Find a unique name for our new collection
			pxr::TfToken NewCollectionName;
			{
				TSet<FString> UsedNames;
				for (const pxr::UsdShadeMaterialBindingAPI::CollectionBinding& Binding : ExistingBindings)
				{
					UsedNames.Add(UsdToUnreal::ConvertToken(Binding.GetCollection().GetName()));
				}

				const static FString UnrealOverridesStr = TEXT("unrealOverrides");
				FString NewCollectionNameStr = UsdUnreal::ObjectUtils::GetUniqueName(UnrealOverridesStr, UsedNames);
				NewCollectionName = UnrealToUsd::ConvertToken(*NewCollectionNameStr).Get();
			}

			// Actually create the new collection binding, with our new name and material
			CollectionToUse = pxr::UsdCollectionAPI::Apply(CollectionPrim, NewCollectionName);
		}

		// Set our target material on CollectionToUse
		pxr::TfToken Empty;	   // Will use CollectionToUse's name instead
		const bool bCreatedBinding = BindingAPI.Bind(CollectionToUse, BoundMaterial, Empty, BindingStrength);
		if (!bCreatedBinding)
		{
			USD_LOG_WARNING(
				TEXT("Failed to create collection-based material binding '%s' on prim '%s'"),
				*UsdToUnreal::ConvertToken(CollectionToUse.GetName()),
				*UsdToUnreal::ConvertPath(CollectionPrim.GetPath())
			);
			return;
		}

		CollectionName = CollectionToUse.GetName();
	}

	// Add the prim to the collection with CollectionName
	const bool bAddedPrim = BindingAPI.AddPrimToBindingCollection(TargetMeshOrGeomSubsetPrim, CollectionName, BindingPurpose);
	if (!bAddedPrim)
	{
		USD_LOG_WARNING(
			TEXT("Failed to add prim '%s' to the collection-based material binding '%s' on prim '%s'"),
			*UsdToUnreal::ConvertPath(TargetMeshOrGeomSubsetPrim.GetPath()),
			*UsdToUnreal::ConvertToken(CollectionName),
			*UsdToUnreal::ConvertPath(CollectionPrim.GetPath())
		);
		return;
	}

	// Remove that prim from any other collection-based bindings we may already have on this prim, to make sure the override shows up
	if (bAddedPrim)
	{
		std::vector<pxr::UsdShadeMaterialBindingAPI::CollectionBinding> AllCollections = BindingAPI.GetCollectionBindings(BindingPurpose);
		for (const pxr::UsdShadeMaterialBindingAPI::CollectionBinding& SomeCollectionBinding : AllCollections)
		{
			pxr::UsdCollectionAPI Collection = SomeCollectionBinding.GetCollection();
			pxr::TfToken OtherCollectionName = Collection.GetName();
			if (OtherCollectionName == CollectionName)
			{
				continue;
			}

			BindingAPI.RemovePrimFromBindingCollection(TargetMeshOrGeomSubsetPrim, OtherCollectionName, BindingPurpose);
		}
	}
}

TOptional<FString> UsdUtils::GetUnrealSurfaceOutput(const pxr::UsdPrim& MaterialPrim)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UsdUtils::GetUnrealSurfaceOutput);

	if (!MaterialPrim)
	{
		return {};
	}

	FScopedUsdAllocs UsdAllocs;

	pxr::UsdShadeMaterial ShadeMaterial{MaterialPrim};
	if (!ShadeMaterial)
	{
		return {};
	}

	pxr::UsdShadeShader SurfaceShader = ShadeMaterial.ComputeSurfaceSource(UnrealIdentifiers::Unreal);
	if (!SurfaceShader)
	{
		return {};
	}

	pxr::SdfAssetPath AssetPath;
	if (SurfaceShader.GetSourceAsset(&AssetPath, UnrealIdentifiers::Unreal))
	{
		return UsdToUnreal::ConvertString(AssetPath.GetAssetPath());
	}

	return {};
}

bool UsdUtils::SetUnrealSurfaceOutput(pxr::UsdPrim& MaterialPrim, const FString& UnrealMaterialPathName)
{
	if (!MaterialPrim)
	{
		return false;
	}

	FScopedUsdAllocs UsdAllocs;

	pxr::UsdShadeMaterial Material{MaterialPrim};
	if (!Material)
	{
		return false;
	}

	pxr::UsdStageRefPtr Stage = MaterialPrim.GetStage();
	pxr::SdfPath ShaderPath = MaterialPrim.GetPath().AppendChild(UnrealToUsd::ConvertToken(TEXT("UnrealShader")).Get());

	pxr::UsdShadeShader UnrealShader = pxr::UsdShadeShader::Define(Stage, ShaderPath);
	if (!UnrealShader)
	{
		return false;
	}

	// Let SetSourceAsset call CreateImplementationSourceAttr internally as it will create the attribute with the correct metadata.
	// For some reason, if we try doing this on Linux we get an attribute that seems to always output "id" when we're exporting material bindings.
	ensure(UnrealShader.SetSourceAsset(
		UnrealMaterialPathName.IsEmpty() ? pxr::SdfAssetPath{} : pxr::SdfAssetPath{UnrealToUsd::ConvertString(*UnrealMaterialPathName).Get()},
		UnrealIdentifiers::Unreal
	));
	pxr::UsdShadeOutput ShaderOutput = UnrealShader.CreateOutput(UnrealToUsd::ConvertToken(TEXT("out")).Get(), pxr::SdfValueTypeNames->Token);

	pxr::UsdShadeOutput MaterialOutput = Material.CreateSurfaceOutput(UnrealIdentifiers::Unreal);
	pxr::UsdShadeConnectableAPI::ConnectToSource(MaterialOutput, ShaderOutput);

	return true;
}

bool UsdUtils::HasSurfaceOutput(const pxr::UsdPrim& MaterialPrim, const FName& RenderContext)
{
	if (!MaterialPrim)
	{
		return false;
	}

	FScopedUsdAllocs UsdAllocs;

	pxr::UsdShadeMaterial ShadeMaterial{MaterialPrim};
	if (!ShadeMaterial)
	{
		return false;
	}

	pxr::UsdShadeOutput MaterialOutput = ShadeMaterial.GetSurfaceOutput(UnrealToUsd::ConvertToken(*RenderContext.ToString()).Get());
	return static_cast<bool>(MaterialOutput);
}

bool UsdUtils::HasSurfaceOutput(const pxr::UsdPrim& MaterialPrim, const TArray<FString>& RenderContexts)
{
	if (!MaterialPrim)
	{
		return false;
	}

	FScopedUsdAllocs UsdAllocs;

	pxr::UsdShadeMaterial ShadeMaterial{MaterialPrim};
	if (!ShadeMaterial)
	{
		return false;
	}

	for (const FString& RenderContext : RenderContexts)
	{
		pxr::UsdShadeOutput MaterialOutput = ShadeMaterial.GetSurfaceOutput(UnrealToUsd::ConvertToken(*RenderContext).Get());
		if (MaterialOutput)
		{
			return true;
		}
	}

	return false;
}

bool UsdUtils::HasDisplacementOutput(const pxr::UsdPrim& MaterialPrim, const FName& RenderContext)
{
	if (!MaterialPrim)
	{
		return false;
	}

	FScopedUsdAllocs UsdAllocs;

	pxr::UsdShadeMaterial ShadeMaterial{MaterialPrim};
	if (!ShadeMaterial)
	{
		return false;
	}

	pxr::UsdShadeOutput MaterialOutput = ShadeMaterial.GetDisplacementOutput(UnrealToUsd::ConvertToken(*RenderContext.ToString()).Get());
	return static_cast<bool>(MaterialOutput);
}

bool UsdUtils::HasDisplacementOutput(const pxr::UsdPrim& MaterialPrim, const TArray<FString>& RenderContexts)
{
	if (!MaterialPrim)
	{
		return false;
	}

	FScopedUsdAllocs UsdAllocs;

	pxr::UsdShadeMaterial ShadeMaterial{MaterialPrim};
	if (!ShadeMaterial)
	{
		return false;
	}

	for (const FString& RenderContext : RenderContexts)
	{
		pxr::UsdShadeOutput MaterialOutput = ShadeMaterial.GetDisplacementOutput(UnrealToUsd::ConvertToken(*RenderContext).Get());
		if (MaterialOutput)
		{
			return true;
		}
	}

	return false;
}

bool UsdUtils::HasVolumeOutput(const pxr::UsdPrim& MaterialPrim, const FName& RenderContext)
{
	if (!MaterialPrim)
	{
		return false;
	}

	FScopedUsdAllocs UsdAllocs;

	pxr::UsdShadeMaterial ShadeMaterial{MaterialPrim};
	if (!ShadeMaterial)
	{
		return false;
	}

	pxr::UsdShadeOutput MaterialOutput = ShadeMaterial.GetVolumeOutput(UnrealToUsd::ConvertToken(*RenderContext.ToString()).Get());
	return static_cast<bool>(MaterialOutput);
}

bool UsdUtils::HasVolumeOutput(const pxr::UsdPrim& MaterialPrim, const TArray<FString>& RenderContexts)
{
	if (!MaterialPrim)
	{
		return false;
	}

	FScopedUsdAllocs UsdAllocs;

	pxr::UsdShadeMaterial ShadeMaterial{MaterialPrim};
	if (!ShadeMaterial)
	{
		return false;
	}

	for (const FString& RenderContext : RenderContexts)
	{
		pxr::UsdShadeOutput MaterialOutput = ShadeMaterial.GetVolumeOutput(UnrealToUsd::ConvertToken(*RenderContext).Get());
		if (MaterialOutput)
		{
			return true;
		}
	}

	return false;
}

bool UsdUtils::IsMaterialTranslucent(const pxr::UsdShadeMaterial& UsdShadeMaterial)
{
	FScopedUsdAllocs UsdAllocs;

	pxr::UsdShadeShader SurfaceShader = UsdShadeMaterial.ComputeSurfaceSource();
	if (!SurfaceShader)
	{
		return false;
	}
	pxr::UsdShadeConnectableAPI Connectable{SurfaceShader};

	const UMaterialInterface* Material = nullptr;
	UUsdAssetCache3* TexturesCache = nullptr;
	const bool bShareAssetsForIdenticalPrims = true;
	UsdToUnreal::FParameterValue ParameterValue;
	bool bHasOpacityConnection = UsdShadeConversionImpl::GetFloatParameterValue(Connectable, UnrealIdentifiers::Opacity, 1.f, ParameterValue);

	// Don't check if the texture is nullptr here as we won't actually parse it yet. If the variant has this type we know it's meant to be bound to a
	// texture
	const bool bHasBoundTexture = ParameterValue.IsType<UsdToUnreal::FTextureParameterValue>();
	const bool bIsTranslucentFloat = (ParameterValue.IsType<float>() && !FMath::IsNearlyEqual(ParameterValue.Get<float>(), 1.0f));

	return bHasOpacityConnection && (bIsTranslucentFloat || bHasBoundTexture);
}

bool UsdUtils::IsMaterialTranslucent(const UsdToUnreal::FUsdPreviewSurfaceMaterialData& ConvertedMaterial)
{
	const UsdToUnreal::FParameterValue* OpacityParam = ConvertedMaterial.Parameters.Find(TEXT("Opacity"));
	if (!OpacityParam)
	{
		return false;
	}

	if (OpacityParam->IsType<UsdToUnreal::FTextureParameterValue>())
	{
		return true;
	}

	if (const float* FloatValue = OpacityParam->TryGet<float>())
	{
		if (!FMath::IsNearlyEqual(*FloatValue, 1.0f))
		{
			return true;
		}
	}

	return false;
}

FSHAHash UsdUtils::HashShadeMaterial(const pxr::UsdShadeMaterial& UsdShadeMaterial, const pxr::TfToken& RenderContext)
{
	FSHAHash OutHash;

	FSHA1 SHA1;
	HashShadeMaterial(UsdShadeMaterial, SHA1, RenderContext);
	SHA1.Final();
	SHA1.GetHash(&OutHash.Hash[0]);

	return OutHash;
}

void UsdUtils::HashShadeMaterial(const pxr::UsdShadeMaterial& UsdShadeMaterial, FSHA1& InOutHash, const pxr::TfToken& RenderContext)
{
	FScopedUsdAllocs UsdAllocs;

	pxr::UsdShadeShader SurfaceShader = UsdShadeMaterial.ComputeSurfaceSource({RenderContext});
	if (!SurfaceShader)
	{
		return;
	}

	for (const pxr::UsdShadeInput& ShadeInput : SurfaceShader.GetInputs())
	{
		UsdShadeConversionImpl::HashShadeInput(ShadeInput, InOutHash);
	}

	bool bValue = false;
	if (pxr::UsdAttribute Attr = UsdShadeMaterial.GetPrim().GetAttribute(UnrealIdentifiers::WorldSpaceNormals))
	{
		Attr.Get<bool>(&bValue);
	}
	InOutHash.Update(reinterpret_cast<uint8*>(&bValue), sizeof(bValue));
}

#undef LOCTEXT_NAMESPACE

#endif	  // #if USE_USD_SDK
