// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDSkeletalDataConversion.h"

#include "UnrealUSDWrapper.h"
#include "USDAttributeUtils.h"
#include "USDClassesModule.h"
#include "USDConversionUtils.h"
#include "USDErrorUtils.h"
#include "USDGeomMeshConversion.h"
#include "USDLayerUtils.h"
#include "USDMemory.h"
#include "USDObjectUtils.h"
#include "USDPrimConversion.h"
#include "USDSetUtils.h"
#include "USDShadeConversion.h"
#include "USDTypesConversion.h"

#include "UsdWrappers/SdfLayer.h"
#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdAttribute.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdSkelSkeletonQuery.h"
#include "UsdWrappers/UsdSkelSkinningQuery.h"
#include "UsdWrappers/UsdStage.h"

#include "Animation/AnimCurveTypes.h"
#include "AnimationRuntime.h"
#include "AnimEncoding.h"
#include "Async/ParallelFor.h"
#include "BoneWeights.h"
#include "Components/SkeletalMeshComponent.h"
#include "ControlRig.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkeletalMeshLODSettings.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Evaluation/MovieSceneSequenceTransform.h"
#include "IMovieScenePlayer.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Misc/CoreMisc.h"
#include "Misc/MemStack.h"
#include "Modules/ModuleManager.h"
#include "MovieScene.h"
#include "MovieSceneTimeHelpers.h"
#include "Rendering/SkeletalMeshLODImporterData.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Rigs/RigHierarchyElements.h"
#include "Sequencer/MovieSceneControlRigParameterSection.h"
#include "StaticMeshAttributes.h"
#include "UObject/Package.h"

#if WITH_EDITOR
#include "Animation/DebugSkelMeshComponent.h"
#include "Factories/FbxSkeletalMeshImportData.h"
#include "ImportUtils/SkeletalMeshImportUtils.h"
#include "MeshUtilities.h"
#endif	  // WITH_EDITOR

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/sdf/types.h"
#include "pxr/usd/usd/editContext.h"
#include "pxr/usd/usd/variantSets.h"
#include "pxr/usd/usdGeom/mesh.h"
#include "pxr/usd/usdGeom/primvarsAPI.h"
#include "pxr/usd/usdGeom/subset.h"
#include "pxr/usd/usdGeom/tokens.h"
#include "pxr/usd/usdGeom/xformable.h"
#include "pxr/usd/usdShade/tokens.h"
#include "pxr/usd/usdSkel/animation.h"
#include "pxr/usd/usdSkel/animMapper.h"
#include "pxr/usd/usdSkel/binding.h"
#include "pxr/usd/usdSkel/bindingAPI.h"
#include "pxr/usd/usdSkel/blendShape.h"
#include "pxr/usd/usdSkel/cache.h"
#include "pxr/usd/usdSkel/root.h"
#include "pxr/usd/usdSkel/skeletonQuery.h"
#include "pxr/usd/usdSkel/skinningQuery.h"
#include "pxr/usd/usdSkel/topology.h"
#include "pxr/usd/usdSkel/utils.h"
#include "USDIncludesEnd.h"
#endif	  // USE_USD_SDK

#define LOCTEXT_NAMESPACE "UsdSkeletalDataConversion"

// Default to true because as of Apr 2023 baking animations with FKControlRigs will only consider morph targets if the curve metadata is on the
// skeleton for some reason
static bool bAddCurveMetadataToSkeleton = true;
static FAutoConsoleVariableRef CVarAddCurveMetadataToSkeleton(
	TEXT("USD.AddCurveMetadataToSkeleton"),
	bAddCurveMetadataToSkeleton,
	TEXT("When true will cause blend shape / morph target float curve data to be added to generated USkeleton assets. When false, this curve data "
		 "will be added to the generated USkeletalMesh assets instead.")
);

#if USE_USD_SDK && WITH_EDITOR
namespace SkelDataConversionImpl
{
	// Adapted from LODUtilities.cpp
	struct FMeshDataBundle
	{
		TArray<FVector3f> Vertices;
		TArray<FVector3f> NormalsPerVertex;
		TArray<uint32> Indices;
		TArray<FVector2f> UVs;
		TArray<uint32> SmoothingGroups;
		TArray<SkeletalMeshImportData::FTriangle> Faces;
		TMap<uint32, TArray<uint32>> VertexIndexToFaceIndices;
	};

	struct FMorphedMeshBundle
	{
		TArray<FVector3f> Vertices;
		TArray<FVector3f> NormalsPerIndex;
		TArray<uint32> Indices;
		TArray<FVector2f> UVs;
		TArray<uint32> SmoothingGroups;
		TArray<uint32> MorphedIndexToSourceIndex;
	};

	/** Converts from wedge-based vertex format into a flat format we can give to MeshUtilities */
	void ConvertImportDataToMeshData(const FSkeletalMeshImportData& ImportData, FMeshDataBundle& MeshDataBundle)
	{
		MeshDataBundle.VertexIndexToFaceIndices.Reserve(ImportData.Points.Num());

		for (const SkeletalMeshImportData::FTriangle& Face : ImportData.Faces)
		{
			SkeletalMeshImportData::FTriangle FaceTriangle;
			FaceTriangle = Face;
			for (int32 Index = 0; Index < 3; ++Index)
			{
				const SkeletalMeshImportData::FVertex& Wedge = ImportData.Wedges[Face.WedgeIndex[Index]];
				FaceTriangle.WedgeIndex[Index] = Wedge.VertexIndex;
				MeshDataBundle.Indices.Add(Wedge.VertexIndex);
				MeshDataBundle.UVs.Add(Wedge.UVs[0]);

				MeshDataBundle.VertexIndexToFaceIndices.FindOrAdd(Wedge.VertexIndex).Add(MeshDataBundle.Faces.Num());
			}
			MeshDataBundle.Faces.Add(FaceTriangle);
			MeshDataBundle.SmoothingGroups.Add(Face.SmoothingGroups);
		}

		MeshDataBundle.Vertices = ImportData.Points;
	}

	/**
	 * Creates a FMorphedMeshBundle by applying the InOutDeltas to InMeshDataBundle, also creating additional deltas.
	 * The point of this function is to prepare OutBundle for computing normals with MeshUtilities. We create new deltas because
	 * the skeletal mesh shares vertices between faces, so if a vertex is morphed, not only does its normal need to be recomputed, but also
	 * the normals of all vertices of triangles that the vertex is a part of.
	 */
	void MorphMeshData(const FMeshDataBundle& InMeshDataBundle, TArray<FMorphTargetDelta>& InOutDeltas, FMorphedMeshBundle& OutBundle)
	{
		OutBundle.Vertices.Reserve(InOutDeltas.Num());
		OutBundle.Indices.Reserve(InOutDeltas.Num());
		OutBundle.UVs.Reserve(InOutDeltas.Num());
		OutBundle.SmoothingGroups.Reserve(InOutDeltas.Num());
		OutBundle.MorphedIndexToSourceIndex.Reserve(InOutDeltas.Num());

		TSet<uint32> AddedFaces;
		TArray<FMorphTargetDelta> NewDeltas;
		TMap<uint32, uint32> SourceIndexToMorphedIndex;

		// Add the existing deltas to the vertices array first
		// Don't add indices yet as we can't guarantee these come in triangle order (they're straight from USD)
		for (const FMorphTargetDelta& Delta : InOutDeltas)
		{
			uint32 SourceIndex = Delta.SourceIdx;
			uint32 MorphedIndex = OutBundle.Vertices.Add(InMeshDataBundle.Vertices[SourceIndex] + Delta.PositionDelta);

			OutBundle.MorphedIndexToSourceIndex.Add(SourceIndex);
			SourceIndexToMorphedIndex.Add(SourceIndex, MorphedIndex);
		}

		// Add all indices, creating any missing deltas/vertices
		for (const FMorphTargetDelta& Delta : InOutDeltas)
		{
			if (const TArray<uint32>* FoundFaceIndices = InMeshDataBundle.VertexIndexToFaceIndices.Find(Delta.SourceIdx))
			{
				for (uint32 FaceIndex : *FoundFaceIndices)
				{
					if (AddedFaces.Contains(FaceIndex))
					{
						continue;
					}
					AddedFaces.Add(FaceIndex);

					const SkeletalMeshImportData::FTriangle& Face = InMeshDataBundle.Faces[FaceIndex];
					OutBundle.SmoothingGroups.Add(Face.SmoothingGroups);

					for (uint32 Index = 0; Index < 3; ++Index)
					{
						uint32 SourceIndex = Face.WedgeIndex[Index];
						uint32 MorphedIndex = INDEX_NONE;

						if (uint32* FoundMorphedIndex = SourceIndexToMorphedIndex.Find(SourceIndex))
						{
							MorphedIndex = *FoundMorphedIndex;
						}
						else
						{
							// Add a new vertex and delta if we don't have one for this vertex yet
							FMorphTargetDelta& NewDelta = NewDeltas.Emplace_GetRef();
							NewDelta.PositionDelta = FVector3f::ZeroVector;
							NewDelta.TangentZDelta = FVector3f::ZeroVector;
							NewDelta.SourceIdx = SourceIndex;

							MorphedIndex = OutBundle.Vertices.Add(InMeshDataBundle.Vertices[SourceIndex]);

							OutBundle.MorphedIndexToSourceIndex.Add(SourceIndex);
							SourceIndexToMorphedIndex.Add(SourceIndex, MorphedIndex);
						}

						OutBundle.Indices.Add(MorphedIndex);
						OutBundle.UVs.Add(InMeshDataBundle.UVs[SourceIndex]);
					}
				}
			}
		}

		InOutDeltas.Append(NewDeltas);
	}

	/**
	 * Updates the TangentZDelta for the vertices within BlendShape with the correct value, so that lighting is correct
	 * when the morph target is applied to the skeletal mesh.
	 * Note: This may add deltas to the blend shape: See MorphMeshData
	 */
	bool ComputeTangentDeltas(const FMeshDataBundle& MeshDataBundle, UsdUtils::FUsdBlendShape& BlendShape)
	{
		if (BlendShape.bHasAuthoredTangents)
		{
			return false;
		}

		FMorphedMeshBundle MorphedBundle;
		MorphMeshData(MeshDataBundle, BlendShape.Vertices, MorphedBundle);

		IMeshUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshUtilities>("MeshUtilities");
		ETangentOptions::Type TangentOptions = (ETangentOptions::Type)(ETangentOptions::BlendOverlappingNormals | ETangentOptions::UseMikkTSpace);
		MeshUtilities.CalculateNormals(
			MorphedBundle.Vertices,
			MorphedBundle.Indices,
			MorphedBundle.UVs,
			MorphedBundle.SmoothingGroups,
			TangentOptions,
			MorphedBundle.NormalsPerIndex
		);

		TMap<uint32, FMorphTargetDelta*> SourceIndexToMorphDelta;
		for (FMorphTargetDelta& Delta : BlendShape.Vertices)
		{
			SourceIndexToMorphDelta.Add(Delta.SourceIdx, &Delta);
		}

		uint32 NumMorphedIndices = static_cast<uint32>(MorphedBundle.Indices.Num());
		for (uint32 MorphedIndexIndex = 0; MorphedIndexIndex < NumMorphedIndices; ++MorphedIndexIndex)
		{
			const uint32 MorphedIndex = MorphedBundle.Indices[MorphedIndexIndex];
			const uint32 SourceIndex = MorphedBundle.MorphedIndexToSourceIndex[MorphedIndex];

			// Note that we store the source normals as one per vertex, but we don't need to do that conversion for the
			// morphed normals, as we're iterating directly over the indices anyway
			const FVector& SourceNormal = (FVector)MeshDataBundle.NormalsPerVertex[SourceIndex];
			const FVector& MorphedNormal = (FVector)MorphedBundle.NormalsPerIndex[MorphedIndexIndex];

			if (FMorphTargetDelta** FoundDelta = SourceIndexToMorphDelta.Find(SourceIndex))
			{
				(*FoundDelta)->TangentZDelta = FVector3f(MorphedNormal - SourceNormal);

				// We will visit each delta multiple times because we're iterating indices and these are per-vertex,
				// so this prevents us from recalculating the delta many times
				SourceIndexToMorphDelta.Remove(SourceIndex);
			}
		}

		return true;
	}

	/** Converts the given offsets into UnrealEditor space and fills in an FUsdBlendShape object with all the data that will become a morph target */
	bool CreateUsdBlendShape(
		const FString& Name,
		const pxr::VtArray<pxr::GfVec3f>& PointOffsets,
		const pxr::VtArray<pxr::GfVec3f>& NormalOffsets,
		const pxr::VtArray<int>& PointIndices,
		const FUsdStageInfo& StageInfo,
		const pxr::GfMatrix4d* InGeomBindTransform,
		uint32 PointIndexOffset,
		int32 LODIndex,
		UsdUtils::FUsdBlendShape& OutBlendShape,
		const UsdToUnreal::FUsdMeshConversionOptions& Options
	)
	{
		uint32 NumOffsets = PointOffsets.size();
		uint32 NumIndices = PointIndices.size();
		uint32 NumNormals = NormalOffsets.size();

		if (NumNormals > 0 && NumOffsets != NumNormals)
		{
			USD_LOG_USERWARNING(FText::Format(
				LOCTEXT(
					"OffsetNormalMismach",
					"BlendShape '{0}' has mismatching numbers of offsets ({1}) and normalOffsets ({2}) and will be ignored"
				),
				FText::FromString(Name),
				NumOffsets,
				NumNormals
			));
			return false;
		}

		if (NumIndices > 0 && NumOffsets != NumIndices)
		{
			USD_LOG_USERWARNING(FText::Format(
				LOCTEXT(
					"OffsetPointMismatch",
					"BlendShape '{0}' has mismatching numbers of offsets ({1}) and point indices ({2}) and will be ignored"
				),
				FText::FromString(Name),
				NumOffsets,
				NumIndices
			));
			return false;
		}

		if (NumOffsets + NumNormals == 0)
		{
			USD_LOG_USERWARNING(
				FText::Format(LOCTEXT("NoOffsets", "BlendShape '{0}' zero offsets and normalOffsets and will be ignored"), FText::FromString(Name))
			);
			return false;
		}

		if (NumNormals > 0)
		{
			OutBlendShape.bHasAuthoredTangents = true;
		}

		OutBlendShape.Name = Name;
		OutBlendShape.LODIndicesThatUseThis.Add(LODIndex);

		// Prepare the indices of the corresponding base points/normals for every local point/normal we have
		TArray<int32> BaseIndices;
		BaseIndices.Reserve(NumOffsets);
		if (NumIndices == 0)
		{
			// If we have no indices it means we have information for all of our local points/normals
			for (uint32 BaseIndex = PointIndexOffset; BaseIndex < PointIndexOffset + NumOffsets; ++BaseIndex)
			{
				BaseIndices.Add(static_cast<int32>(BaseIndex));
			}
		}
		else
		{
			// If we have indices it means our morph target only affects a subset of the base vertices
			for (uint32 LocalIndex = 0; LocalIndex < NumOffsets; ++LocalIndex)
			{
				int32 BaseIndex = PointIndices[LocalIndex] + static_cast<int32>(PointIndexOffset);

				BaseIndices.Add(BaseIndex);
			}
		}

		// Setup GeomBindTransform and invtranspose
		pxr::GfMatrix4d GeomBindTransform = InGeomBindTransform ? *InGeomBindTransform : pxr::GfMatrix4d{1.0f};
		pxr::GfMatrix4d InvTransposeGeomBindTransform = GeomBindTransform;
		if (OutBlendShape.bHasAuthoredTangents && GeomBindTransform != pxr::GfMatrix4d(1.0))	// This is USD's identity
		{
			if (GeomBindTransform.GetDeterminant() == 0.0)
			{
				// Can't invert, just use as-is
				USD_LOG_WARNING(TEXT("Failed to invert geomBindTransform when parsing blend shape '%s'"), *Name);
			}
			else
			{
				InvTransposeGeomBindTransform = GeomBindTransform.GetInverse().GetTranspose();
			}
		}

		FMatrix TotalMatrix = Options.AdditionalTransform.ToMatrixWithScale();
		FMatrix TotalMatrixForNormal = TotalMatrix.Inverse().GetTransposed();

		OutBlendShape.Vertices.SetNumUninitialized(NumOffsets);
		for (uint32 OffsetIndex = 0; OffsetIndex < NumOffsets; ++OffsetIndex)
		{
			FMorphTargetDelta& ModifiedVertex = OutBlendShape.Vertices[OffsetIndex];
			ModifiedVertex.SourceIdx = BaseIndices[OffsetIndex];

			// Position offset
			// Note: TransformDir here because even the position deltas are still *deltas* (i.e. vector offsets)
			pxr::GfVec3d USDPoint = GeomBindTransform.TransformDir(PointOffsets[OffsetIndex]);
			ModifiedVertex.PositionDelta = (FVector3f)(FVector4f)TotalMatrix.TransformVector(UsdToUnreal::ConvertVector(StageInfo, USDPoint));

			// Normal offset
			if (OutBlendShape.bHasAuthoredTangents)
			{
				pxr::GfVec3d USDNormal = InvTransposeGeomBindTransform.TransformDir(NormalOffsets[OffsetIndex]);
				const FVector UENormal = FVector(
					TotalMatrixForNormal.TransformVector(UsdToUnreal::ConvertVector(StageInfo, USDNormal)).GetSafeNormal()
				);
				ModifiedVertex.TangentZDelta = (FVector3f)UENormal;
			}
			else
			{
				// Don't leave it uninitialized
				ModifiedVertex.TangentZDelta = FVector3f::ZeroVector;
			}
		}

		return true;
	}

	/**
	 * Updates MorphTargetDeltas, remapping/adding/removing deltas according to the index remapping in OrigIndexToBuiltIndices.
	 * This is required because the SkeletalMesh build process may create/optimize/destroy vertices, and the indices through
	 * which our deltas refer to these vertices come directly from USD. Example: If a vertex affected by the blend shape is split, we need
	 * to duplicate the delta to all the split versions.
	 */
	void UpdatesDeltasToMeshBuild(TArray<FMorphTargetDelta>& MorphTargetDeltas, const TMap<int32, TArray<int32>>& OrigIndexToBuiltIndices)
	{
		TSet<int32> DeltasToDelete;
		TArray<FMorphTargetDelta> NewDeltas;
		for (int32 DeltaIndex = 0; DeltaIndex < MorphTargetDeltas.Num(); ++DeltaIndex)
		{
			FMorphTargetDelta& ModifiedVertex = MorphTargetDeltas[DeltaIndex];

			if (const TArray<int32>* BuiltIndices = OrigIndexToBuiltIndices.Find(ModifiedVertex.SourceIdx))
			{
				// Our index just got remapped somewhere else: Update it
				if (BuiltIndices->Num() >= 1)
				{
					ModifiedVertex.SourceIdx = static_cast<uint32>((*BuiltIndices)[0]);
				}

				// The vertex we were pointing at got split into multiple vertices: Add a matching delta for each
				for (int32 NewDeltaIndex = 1; NewDeltaIndex < BuiltIndices->Num(); ++NewDeltaIndex)
				{
					FMorphTargetDelta& NewDelta = NewDeltas.Add_GetRef(ModifiedVertex);
					NewDelta.SourceIdx = static_cast<uint32>((*BuiltIndices)[NewDeltaIndex]);
				}
			}
			// The vertex we were pointing at got deleted: Remove the delta
			else
			{
				DeltasToDelete.Add(DeltaIndex);
			}
		}
		if (DeltasToDelete.Num() > 0)
		{
			for (int32 DeltaIndex = MorphTargetDeltas.Num() - 1; DeltaIndex >= 0; --DeltaIndex)
			{
				if (DeltasToDelete.Contains(DeltaIndex))
				{
					MorphTargetDeltas.RemoveAt(DeltaIndex, EAllowShrinking::No);
				}
			}
		}
		MorphTargetDeltas.Append(MoveTemp(NewDeltas));
	}

	/**
	 * Will find or create a AACF_DefaultCurve float curve with CurveName, and set its data to a copy of SourceData.
	 * Adapted from UnFbx::FFbxImporter::ImportCurveToAnimSequence
	 */
	void SetFloatCurveData(UAnimSequence* Sequence, FName CurveName, const FRichCurve& SourceData)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SkelDataConversionImpl::SetFloatCurveData);

		if (!Sequence)
		{
			return;
		}

		USkeleton* Skeleton = Sequence->GetSkeleton();
		if (!Skeleton)
		{
			return;
		}

		// Ignore curves that don't contribute to the animation
		bool bHasNonZeroKey = false;
		for (const FRichCurveKey& Key : SourceData.Keys)
		{
			if (!FMath::IsNearlyEqual(Key.Value, 0.0f))
			{
				bHasNonZeroKey = true;
				break;
			}
		}
		if (!bHasNonZeroKey)
		{
			return;
		}

		const bool bShouldTransact = false;
		const IAnimationDataModel* DataModel = Sequence->GetDataModel();
		IAnimationDataController& Controller = Sequence->GetController();

		FAnimationCurveIdentifier CurveId(CurveName, ERawCurveTrackTypes::RCT_Float);
		const FFloatCurve* Curve = DataModel->FindFloatCurve(CurveId);
		if (!Curve)
		{
			// If curve doesn't exist, add one
			Controller.AddCurve(CurveId, AACF_DefaultCurve, bShouldTransact);
			Curve = DataModel->FindFloatCurve(CurveId);
		}
		else
		{
			if (!(Curve->FloatCurve == SourceData))
			{
				USD_LOG_USERWARNING(FText::Format(
					LOCTEXT(
						"OverwritingMorphTargetCurves",
						"Overwriting animation curve for morph target '{0}' with different data! If the Skeletal Mesh has multiple LODs, make "
						"sure each LOD mesh that wants to animate a certain blend shape does so with the same blend shape curve."
					),
					FText::FromName(CurveName)
				));
			}

			Controller.SetCurveFlags(CurveId, Curve->GetCurveTypeFlags() | AACF_DefaultCurve, bShouldTransact);
		}

		if (Curve)
		{
			Controller.SetCurveKeys(CurveId, SourceData.GetConstRefOfKeys(), bShouldTransact);
		}
		else
		{
			USD_LOG_ERROR(TEXT("Failed to create float curve with name '%s' for UAnimSequence '%s'"), *CurveName.ToString(), *Sequence->GetName());
		}
	}

	/**
	 * If ChannelWeightCurve is the SkelAnim channel intended to affect a USD blend shape and its inbetweens,
	 * this function will remap it into multiple FRichCurve that can be apply to all the independent morph
	 * targets that were generated from the blend shape and its inbetweens, if any.
	 * Index 0 of the returned array always contains the remapped primary morph target weight, and the rest match the inbetween order
	 */
	TArray<FRichCurve> ResolveWeightsForBlendShapeCurve(const UsdUtils::FUsdBlendShape& PrimaryBlendShape, const FRichCurve& ChannelWeightCurve)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SkelDataConversionImpl::ResolveWeightsForBlendShapeCurve);

		int32 NumInbetweens = PrimaryBlendShape.Inbetweens.Num();
		if (NumInbetweens == 0)
		{
			return {ChannelWeightCurve};
		}

		TArray<FRichCurve> Result;
		Result.SetNum(NumInbetweens + 1);	 // One for each inbetween and an additional one for the morph target generated from the primary blend
											 // shape

		TArray<float> ResolvedInbetweenWeightsSample;
		ResolvedInbetweenWeightsSample.SetNum(NumInbetweens);

		for (const FRichCurveKey& SourceKey : ChannelWeightCurve.Keys)
		{
			const float SourceTime = SourceKey.Time;
			const float SourceValue = SourceKey.Value;

			float ResolvedPrimarySample;
			UsdUtils::ResolveWeightsForBlendShape(PrimaryBlendShape, SourceValue, ResolvedPrimarySample, ResolvedInbetweenWeightsSample);

			FRichCurve& PrimaryCurve = Result[0];
			FKeyHandle PrimaryHandle = PrimaryCurve.AddKey(SourceTime, ResolvedPrimarySample);
			PrimaryCurve.SetKeyInterpMode(PrimaryHandle, SourceKey.InterpMode);

			for (int32 InbetweenIndex = 0; InbetweenIndex < NumInbetweens; ++InbetweenIndex)
			{
				FRichCurve& InbetweenCurve = Result[InbetweenIndex + 1];
				FKeyHandle InbetweenHandle = InbetweenCurve.AddKey(SourceTime, ResolvedInbetweenWeightsSample[InbetweenIndex]);
				InbetweenCurve.SetKeyInterpMode(InbetweenHandle, SourceKey.InterpMode);
			}
		}

		return Result;
	}
}	 // namespace SkelDataConversionImpl

namespace UsdToUnrealImpl
{
	int32 GetPrimValueIndex(
		const EUsdInterpolationMethod& InterpMethod,
		const int32 VertexIndex,
		const int32 VertexInstanceIndex,
		const int32 PolygonIndex
	)
	{
		switch (InterpMethod)
		{
			case EUsdInterpolationMethod::Vertex:
				return VertexIndex;
			case EUsdInterpolationMethod::FaceVarying:
				return VertexInstanceIndex;
			case EUsdInterpolationMethod::Uniform:
				return PolygonIndex;
			case EUsdInterpolationMethod::Constant:
			default:
				return 0;
		}
	}

	void ComputeSourceNormals(SkelDataConversionImpl::FMeshDataBundle& UnmorphedShape)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(USDSkeletalDataConversion::ComputeSourceNormals);

		IMeshUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshUtilities>(TEXT("MeshUtilities"));

		// Calculate base normals for the mesh so that we can compute tangent deltas if we need to
		ETangentOptions::Type TangentOptions = (ETangentOptions::Type)(ETangentOptions::BlendOverlappingNormals | ETangentOptions::UseMikkTSpace);
		TArray<FVector3f> NormalsPerIndex;
		MeshUtilities.CalculateNormals(
			UnmorphedShape.Vertices,
			UnmorphedShape.Indices,
			UnmorphedShape.UVs,
			UnmorphedShape.SmoothingGroups,
			TangentOptions,
			NormalsPerIndex
		);	  // LWC_TODO: Perf pessimization (ConvertArray)

		// Convert our normals to one normal per vertex, making it faster to unpack the normals we compute in ComputeTangentDeltas
		// This is possible because we compute them with ETangentOptions::BlendOverlappingNormals, so they are identical for all instances of the
		// vertex
		UnmorphedShape.NormalsPerVertex.SetNumZeroed(UnmorphedShape.Vertices.Num());
		for (int32 IndexIndex = 0; IndexIndex < UnmorphedShape.Indices.Num(); ++IndexIndex)
		{
			uint32 VertexIndex = UnmorphedShape.Indices[IndexIndex];
			UnmorphedShape.NormalsPerVertex[VertexIndex] = NormalsPerIndex[IndexIndex];
		}
	}
}	 // namespace UsdToUnrealImpl

namespace UnrealToUsdImpl
{
	void ConvertSkeletalMeshLOD(
		const USkeletalMesh* SkeletalMesh,
		const FSkeletalMeshLODModel& LODModel,
		pxr::UsdGeomMesh& UsdLODPrimGeomMesh,
		bool bHasVertexColors,
		const TArray<FString>& MaterialAssignments,
		const TArray<int32>& LODMaterialMap,
		const pxr::UsdTimeCode TimeCode,
		pxr::UsdPrim PrimToReceiveMaterialAssignments,
		TArray<int32>& OutSourceToPackedVertexIndex
	)
	{
		FScopedUsdAllocs UsdAllocs;

		pxr::UsdPrim MeshPrim = UsdLODPrimGeomMesh.GetPrim();
		pxr::UsdStageRefPtr Stage = MeshPrim.GetStage();

		pxr::UsdSkelBindingAPI SkelBindingAPI = pxr::UsdSkelBindingAPI::Apply(MeshPrim);
		if (!SkelBindingAPI)
		{
			return;
		}

		if (!Stage)
		{
			return;
		}

		const FUsdStageInfo StageInfo(Stage);

		// FSkelMeshSection can be "disabled", at which point they don't show up in the engine. We'll skip those
		// sections, and will use this array to help remap from a source vertex index to the vertex's corresponding
		// index in a "packed" array of vertices that we'll push to USD
		int32 PackedVertexIndex = 0;

		// Vertices
		{
			if (LODModel.NumVertices == 0)
			{
				return;
			}

			// We manually collect vertices here instead of calling LODModel.GetVertices as we need to skip
			// vertices from disabled sections, which that function won't do
			TArray<FSoftSkinVertex> Vertices;
			Vertices.Reserve(LODModel.NumVertices);
			OutSourceToPackedVertexIndex.SetNum(LODModel.NumVertices);
			for (int32 SectionIndex = 0; SectionIndex < LODModel.Sections.Num(); ++SectionIndex)
			{
				const FSkelMeshSection& Section = LODModel.Sections[SectionIndex];
				if (Section.bDisabled)
				{
					// Mark that these indices are skipped
					for (int32 Index = 0; Index < Section.NumVertices; ++Index)
					{
						OutSourceToPackedVertexIndex[Section.BaseVertexIndex + Index] = INDEX_NONE;
					}
					continue;
				}

				for (int32 Index = 0; Index < Section.NumVertices; ++Index)
				{
					OutSourceToPackedVertexIndex[Section.BaseVertexIndex + Index] = PackedVertexIndex++;
				}

				Vertices.Append(Section.SoftVertices);
			}
			const int32 VertexCount = Vertices.Num();

			// Points
			{
				pxr::UsdAttribute Points = UsdLODPrimGeomMesh.CreatePointsAttr();
				if (Points)
				{
					pxr::VtArray<pxr::GfVec3f> PointsArray;
					PointsArray.reserve(VertexCount);

					for (int32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
					{
						PointsArray.push_back(UnrealToUsd::ConvertVectorFloat(StageInfo, (FVector)Vertices[VertexIndex].Position));
					}

					Points.Set(PointsArray, TimeCode);
				}
			}

			// Normals
			{
				// We need to emit this if we're writing normals (which we always are) because any DCC that can
				// actually subdivide (like usdview) will just discard authored normals and fully recompute them
				// on-demand in case they have a valid subdivision scheme (which is the default state).
				// Reference: https://graphics.pixar.com/usd/release/api/class_usd_geom_mesh.html#UsdGeom_Mesh_Normals
				if (pxr::UsdAttribute SubdivisionAttr = UsdLODPrimGeomMesh.CreateSubdivisionSchemeAttr())
				{
					ensure(SubdivisionAttr.Set(pxr::UsdGeomTokens->none));
				}

				pxr::UsdAttribute NormalsAttribute = UsdLODPrimGeomMesh.CreateNormalsAttr();
				if (NormalsAttribute)
				{
					pxr::VtArray<pxr::GfVec3f> Normals;
					Normals.reserve(VertexCount);

					for (int32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
					{
						Normals.push_back(UnrealToUsd::ConvertVectorFloat(StageInfo, FVector4(Vertices[VertexIndex].TangentZ)));
					}

					NormalsAttribute.Set(Normals, TimeCode);
				}
			}

			// UVs
			{
				for (uint32 TexCoordSourceIndex = 0; TexCoordSourceIndex < LODModel.NumTexCoords; ++TexCoordSourceIndex)
				{
					pxr::TfToken UsdUVSetName = UsdUtils::GetUVSetName(TexCoordSourceIndex).Get();

					pxr::UsdGeomPrimvar PrimvarST = pxr::UsdGeomPrimvarsAPI(MeshPrim).CreatePrimvar(
						UsdUVSetName,
						pxr::SdfValueTypeNames->TexCoord2fArray,
						pxr::UsdGeomTokens->vertex
					);

					if (PrimvarST)
					{
						pxr::VtVec2fArray UVs;

						for (int32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
						{
							FVector2D TexCoord = FVector2D(Vertices[VertexIndex].UVs[TexCoordSourceIndex]);
							TexCoord[1] = 1.f - TexCoord[1];

							UVs.push_back(UnrealToUsd::ConvertVectorFloat(TexCoord));
						}

						PrimvarST.Set(UVs, TimeCode);
					}
				}
			}

			// Vertex colors
			if (bHasVertexColors)
			{
				pxr::UsdGeomPrimvar DisplayColorPrimvar = UsdLODPrimGeomMesh.CreateDisplayColorPrimvar(pxr::UsdGeomTokens->vertex);
				pxr::UsdGeomPrimvar DisplayOpacityPrimvar = UsdLODPrimGeomMesh.CreateDisplayOpacityPrimvar(pxr::UsdGeomTokens->vertex);

				if (DisplayColorPrimvar && DisplayOpacityPrimvar)
				{
					pxr::VtArray<pxr::GfVec3f> DisplayColors;
					DisplayColors.reserve(VertexCount);

					pxr::VtArray<float> DisplayOpacities;
					DisplayOpacities.reserve(VertexCount);

					for (int32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
					{
						const FColor& VertexColor = Vertices[VertexIndex].Color;

						pxr::GfVec4f Color = UnrealToUsd::ConvertColor(VertexColor);
						DisplayColors.push_back(pxr::GfVec3f(Color[0], Color[1], Color[2]));
						DisplayOpacities.push_back(Color[3]);
					}

					DisplayColorPrimvar.Set(DisplayColors, TimeCode);
					DisplayOpacityPrimvar.Set(DisplayOpacities, TimeCode);
				}
			}

			// Joint indices & weights
			{
				const int32 NumInfluencesPerVertex = LODModel.GetMaxBoneInfluences();

				const bool bConstantPrimvar = false;
				pxr::UsdGeomPrimvar JointIndicesPrimvar = SkelBindingAPI.CreateJointIndicesPrimvar(bConstantPrimvar, NumInfluencesPerVertex);
				pxr::UsdGeomPrimvar JointWeightsPrimvar = SkelBindingAPI.CreateJointWeightsPrimvar(bConstantPrimvar, NumInfluencesPerVertex);

				if (JointIndicesPrimvar && JointWeightsPrimvar)
				{
					pxr::VtArray<int> JointIndices;
					JointIndices.reserve(VertexCount * NumInfluencesPerVertex);

					pxr::VtArray<float> JointWeights;
					JointWeights.reserve(VertexCount * NumInfluencesPerVertex);

					for (const FSkelMeshSection& Section : LODModel.Sections)
					{
						if (Section.bDisabled)
						{
							continue;
						}

						for (const FSoftSkinVertex& Vertex : Section.SoftVertices)
						{
							for (int32 InfluenceIndex = 0; InfluenceIndex < NumInfluencesPerVertex; ++InfluenceIndex)
							{
								int32 BoneIndex = Section.BoneMap[Vertex.InfluenceBones[InfluenceIndex]];

								JointIndices.push_back(BoneIndex);
								JointWeights.push_back(Vertex.InfluenceWeights[InfluenceIndex] / UE::AnimationCore::MaxRawBoneWeightFloat);
							}
						}
					}

					JointIndicesPrimvar.Set(JointIndices, TimeCode);
					JointWeightsPrimvar.Set(JointWeights, TimeCode);
				}
			}
		}

		// Faces
		{
			int32 TotalNumTriangles = 0;

			// Face Vertex Counts
			{
				for (const FSkelMeshSection& Section : LODModel.Sections)
				{
					if (Section.bDisabled)
					{
						continue;
					}

					TotalNumTriangles += Section.NumTriangles;
				}

				pxr::UsdAttribute FaceCountsAttribute = UsdLODPrimGeomMesh.CreateFaceVertexCountsAttr();
				if (FaceCountsAttribute)
				{
					pxr::VtArray<int> FaceVertexCounts;
					FaceVertexCounts.reserve(TotalNumTriangles);

					for (int32 FaceIndex = 0; FaceIndex < TotalNumTriangles; ++FaceIndex)
					{
						FaceVertexCounts.push_back(3);
					}

					FaceCountsAttribute.Set(FaceVertexCounts, TimeCode);
				}
			}

			// Face Vertex Indices
			{
				pxr::UsdAttribute FaceVertexIndicesAttribute = UsdLODPrimGeomMesh.GetFaceVertexIndicesAttr();

				if (FaceVertexIndicesAttribute)
				{
					pxr::VtArray<int> FaceVertexIndices;
					FaceVertexIndices.reserve(TotalNumTriangles * 3);

					for (const FSkelMeshSection& Section : LODModel.Sections)
					{
						if (Section.bDisabled)
						{
							continue;
						}

						int32 TriangleCount = Section.NumTriangles;
						for (uint32 TriangleIndex = 0; TriangleIndex < Section.NumTriangles; ++TriangleIndex)
						{
							for (uint32 PointIndex = 0; PointIndex < 3; PointIndex++)
							{
								const int32 SourceVertexIndex = LODModel.IndexBuffer[Section.BaseIndex + ((TriangleIndex * 3) + PointIndex)];

								int32 PackedVertexPositionIndex = OutSourceToPackedVertexIndex[SourceVertexIndex];

								check(PackedVertexPositionIndex >= 0 && PackedVertexPositionIndex != INDEX_NONE);

								FaceVertexIndices.push_back(PackedVertexPositionIndex);
							}
						}
					}

					FaceVertexIndicesAttribute.Set(FaceVertexIndices, TimeCode);
				}
			}
		}

		// Material assignments
		{
			bool bHasUEMaterialAssignements = false;

			TArray<FString> UnrealMaterialsForLOD;
			for (const FSkelMeshSection& Section : LODModel.Sections)
			{
				int32 SkeletalMaterialIndex = INDEX_NONE;
				if (LODMaterialMap.IsValidIndex(Section.MaterialIndex))
				{
					SkeletalMaterialIndex = LODMaterialMap[Section.MaterialIndex];
				}
				// Note that the LODMaterialMap can contain INDEX_NONE to signify no remapping
				if (SkeletalMaterialIndex == INDEX_NONE)
				{
					SkeletalMaterialIndex = Section.MaterialIndex;
				}

				if (MaterialAssignments.IsValidIndex(SkeletalMaterialIndex))
				{
					UnrealMaterialsForLOD.Add(MaterialAssignments[SkeletalMaterialIndex]);
					bHasUEMaterialAssignements = true;
				}
				else
				{
					// Keep unrealMaterials with the same number of elements as our MaterialIndices expect
					UnrealMaterialsForLOD.Add("");
				}
			}

			// This LOD has a single material assignment, just add an unrealMaterials attribute to the mesh prim
			if (bHasUEMaterialAssignements && UnrealMaterialsForLOD.Num() == 1)
			{
				UsdUtils::AuthorUnrealMaterialBinding(PrimToReceiveMaterialAssignments, UnrealMaterialsForLOD[0]);
			}
			// Multiple material assignments to the same LOD (and so the same mesh prim). Need to create a GeomSubset for each UE mesh section
			else if (UnrealMaterialsForLOD.Num() > 1)
			{
				// Need to fetch all triangles of a section, and add their indices to the GeomSubset
				for (int32 SectionIndex = 0; SectionIndex < LODModel.Sections.Num(); ++SectionIndex)
				{
					const FSkelMeshSection& Section = LODModel.Sections[SectionIndex];
					if (Section.bDisabled)
					{
						continue;
					}

					// Note that we will continue authoring the GeomSubsets on even if we later find out we have no material assignment (just
					// "") for this section, so as to satisfy the "partition" family condition (below)
					pxr::UsdPrim GeomSubsetPrim = Stage->DefinePrim(
						UsdLODPrimGeomMesh.GetPath().AppendPath(pxr::SdfPath("Section" + std::to_string(SectionIndex))),
						UnrealToUsd::ConvertToken(TEXT("GeomSubset")).Get()
					);

					pxr::UsdPrim MaterialGeomSubsetPrim = GeomSubsetPrim;
					if (PrimToReceiveMaterialAssignments.GetStage() != MeshPrim.GetStage())
					{
						MaterialGeomSubsetPrim = PrimToReceiveMaterialAssignments.GetStage()->OverridePrim(
							PrimToReceiveMaterialAssignments.GetPath().AppendPath(pxr::SdfPath("Section" + std::to_string(SectionIndex)))
						);
					}

					pxr::UsdGeomSubset GeomSubsetSchema{GeomSubsetPrim};

					// Element type attribute
					pxr::UsdAttribute ElementTypeAttr = GeomSubsetSchema.CreateElementTypeAttr();
					ElementTypeAttr.Set(pxr::UsdGeomTokens->face, TimeCode);

					// Indices attribute
					{
						const uint32 TriangleCount = Section.NumTriangles;
						uint32 FirstTriangleIndex = Section.BaseIndex / 3;	  // BaseIndex is the first *vertex* instance index
						pxr::VtArray<int> IndicesAttrValue;

						// We may have some disabled sections (that wouldn't have emitted triangles). If so, we need to
						// adjust our FirstTriangleIndex.
						// This could be optimized in case vertex instances show up in LODModel.IndexBuffer according
						// to the section order, but so far we haven't found such guarantee, so just check them all.
						for (int32 OtherSectionIndex = 0; OtherSectionIndex < LODModel.Sections.Num(); ++OtherSectionIndex)
						{
							const FSkelMeshSection& OtherSection = LODModel.Sections[OtherSectionIndex];
							if (OtherSection.bDisabled && OtherSection.BaseIndex < Section.BaseIndex)
							{
								FirstTriangleIndex -= OtherSection.NumTriangles;
							}
						}

						for (uint32 TriangleIndex = FirstTriangleIndex; TriangleIndex - FirstTriangleIndex < TriangleCount; ++TriangleIndex)
						{
							// Note that we add VertexInstances in sequence to the usda file for the faceVertexInstances attribute, which
							// also constitutes our triangle order
							IndicesAttrValue.push_back(static_cast<int>(TriangleIndex));
						}

						pxr::UsdAttribute IndicesAttr = GeomSubsetSchema.CreateIndicesAttr();
						IndicesAttr.Set(IndicesAttrValue, TimeCode);
					}

					// Family name attribute
					pxr::UsdAttribute FamilyNameAttr = GeomSubsetSchema.CreateFamilyNameAttr();
					FamilyNameAttr.Set(pxr::UsdShadeTokens->materialBind, TimeCode);

					// Family type
					pxr::UsdGeomSubset::SetFamilyType(UsdLODPrimGeomMesh, pxr::UsdShadeTokens->materialBind, pxr::UsdGeomTokens->partition);

					// material:binding relationship
					UsdUtils::AuthorUnrealMaterialBinding(MaterialGeomSubsetPrim, UnrealMaterialsForLOD[SectionIndex]);
				}
			}
		}
	}

	// Converts UE morph target deltas from DeltaArray into offsets, pointIndices and normalOffsets attributes of BlendShape
	bool ConvertMorphTargetDeltas(
		TConstArrayView<FMorphTargetDelta> DeltaArray,
		TArray<int32> SourceToPackedVertexIndex,
		pxr::UsdSkelBlendShape& BlendShape,
		pxr::UsdTimeCode TimeCode
	)
	{
		if (DeltaArray.IsEmpty() || !BlendShape)
		{
			return false;
		}

		FUsdStageInfo StageInfo{BlendShape.GetPrim().GetStage()};
		FScopedUsdAllocs Allocs;

		pxr::VtArray<pxr::GfVec3f> Offsets;
		pxr::VtArray<int> PointIndices;
		pxr::VtArray<pxr::GfVec3f> Normals;

		Offsets.reserve(DeltaArray.Num());
		PointIndices.reserve(DeltaArray.Num());
		Normals.reserve(DeltaArray.Num());

		for (const FMorphTargetDelta& Delta: DeltaArray)
		{
			int32 PackedIndex = SourceToPackedVertexIndex[Delta.SourceIdx];
			if (PackedIndex == INDEX_NONE)
			{
				continue;
			}

			PointIndices.push_back(PackedIndex);
			Offsets.push_back(UnrealToUsd::ConvertVectorFloat(StageInfo, (FVector)Delta.PositionDelta));
			Normals.push_back(UnrealToUsd::ConvertVectorFloat(StageInfo, (FVector)Delta.TangentZDelta));
		}

		BlendShape.CreateOffsetsAttr().Set(Offsets, TimeCode);
		BlendShape.CreatePointIndicesAttr().Set(PointIndices, TimeCode);
		BlendShape.CreateNormalOffsetsAttr().Set(Normals, TimeCode);

		return true;
	}

	// BoneNamesInOrder represents a hierarchy of bones. OutFullPaths will be the full path to each bone, in the same order
	// e.g. 'Root/Arm/Foot'
	void CreateFullBonePaths(const TArray<FMeshBoneInfo>& BoneNamesInOrder, TArray<FString>& OutFullPaths)
	{
		int32 NumBones = BoneNamesInOrder.Num();
		if (NumBones < 1)
		{
			return;
		}

		OutFullPaths.SetNum(NumBones);

		// The first bone is the root, and has ParentIndex == -1, so do it separately here to void checking the indices for all bones
		// Sanitize because ExportName can have spaces, which USD doesn't like
		OutFullPaths[0] = UsdUnreal::ObjectUtils::SanitizeObjectName(BoneNamesInOrder[0].ExportName);

		// Bones are always stored in an increasing order, so we can do all paths in a single pass
		for (int32 BoneIndex = 1; BoneIndex < NumBones; ++BoneIndex)
		{
			const FMeshBoneInfo& BoneInfo = BoneNamesInOrder[BoneIndex];
			FString SanitizedBoneName = UsdUnreal::ObjectUtils::SanitizeObjectName(BoneInfo.ExportName);

			OutFullPaths[BoneIndex] = FString::Printf(TEXT("%s/%s"), *OutFullPaths[BoneInfo.ParentIndex], *SanitizedBoneName);
		}
	}
}	 // namespace UnrealToUsdImpl

bool UsdToUnreal::ConvertSkeleton(
	const pxr::UsdSkelSkeletonQuery& InSkeletonQuery,
	FUsdSkeletonData& OutConvertedData,
	bool bEnsureAtLeastOneBone,
	bool bEnsureSingleRootBone
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UsdToUnreal::ConvertSkeletonToTempData);

	// Note: In here "joint" and "bone" are used interchangeably

	if (!InSkeletonQuery)
	{
		return false;
	}

	// Retrieve the joint names and parent indices from the skeleton topology
	// GetJointOrder already orders them from parent-to-child
	pxr::VtArray<pxr::TfToken> JointOrder = InSkeletonQuery.GetJointOrder();
	const pxr::UsdSkelTopology& SkelTopology = InSkeletonQuery.GetTopology();
	const int32 NumBones = SkelTopology.GetNumJoints();
	if (NumBones > MAX_BONES)
	{
		return false;
	}

	// Fill in everything but transforms
	uint32 RootBoneCount = 0;
	OutConvertedData.Bones.Reset();
	OutConvertedData.Bones.SetNum(NumBones);
	for (uint32 Index = 0; Index < SkelTopology.GetNumJoints(); ++Index)
	{
		pxr::SdfPath JointPath{JointOrder[Index]};
		FString JointName = UsdToUnreal::ConvertString(JointPath.GetName());
		int32 ParentIndex = SkelTopology.GetParent(Index);

		FUsdSkeletonData::FBone& Bone = OutConvertedData.Bones[Index];
		Bone.Name = JointName;
		Bone.ParentIndex = ParentIndex;

		if (ParentIndex == -1)
		{
			RootBoneCount++;
		}
	}

	// Skeleton has no joints: Generate a dummy single "Root" bone skeleton
	if (NumBones == 0)
	{
		FString SkeletonPrimPath = UsdToUnreal::ConvertPath(InSkeletonQuery.GetPrim().GetPath());

		USD_LOG_USERWARNING(FText::Format(
			LOCTEXT(
				"NoBonesInSkeleton",
				"Skeleton prim '{0}' has no joints! "
				"A new skeleton with a single root joint will be generated as USkeletalMeshes require valid skeletons. "
				"Note that this new skeleton may be written back to the USD stage when exporting the corresponding asset."
			),
			FText::FromString(SkeletonPrimPath)
		));

		if (bEnsureAtLeastOneBone)
		{
			FUsdSkeletonData::FBone& Bone = OutConvertedData.Bones.Emplace_GetRef();
			Bone.Name = TEXT("Root");
			Bone.ParentIndex = INDEX_NONE;
			return true;
		}
	}
	
	pxr::VtArray<pxr::GfMatrix4d> JointWorldBindTransforms;
	bool bTransformsComputed = InSkeletonQuery.GetJointWorldBindTransforms(&JointWorldBindTransforms);
	if (NumBones != JointWorldBindTransforms.size())
	{
		return false;
	}
	
	pxr::VtArray<pxr::GfMatrix4d> JointLocalBindTransforms;
	bTransformsComputed &= pxr::UsdSkelComputeJointLocalTransforms(SkelTopology, JointWorldBindTransforms, &JointLocalBindTransforms);

	if (bTransformsComputed)
	{
		pxr::UsdStageWeakPtr Stage = InSkeletonQuery.GetSkeleton().GetPrim().GetStage();
		const FUsdStageInfo StageInfo(Stage);

		for (uint32 Index = 0; Index < JointWorldBindTransforms.size(); ++Index)
		{
			FUsdSkeletonData::FBone& Bone = OutConvertedData.Bones[Index];

			// Here we use DecomposeWithUniformReflection instead of the previous UsdToUnreal::ConvertMatrix(StageInfo, UsdMatrix)
			// call, because internally that would have done the matrix decomposition via FTransform::SetFromMatrix.
			//
			// The only difference between the two being that if we detect any negative scaling, DecomposeWithUniformReflection will
			// flip *all* axes instead of only one, which will keep the scaling uniform. Otherwise, we may get weird joint flipping
			// effects and the joint rotation axes being inverted (see UE-193643). Those are likely consequences of decomposed
			// transforms not being easily invertible (some code at some point will silently assume uniform scaling, and things would
			// break).
			//
			// Note that FBX secretly does this as well, because the FBX SDK's Matrix.GetT(), Matrix.GetQ() and Matrix.GetS()
			// (used within UnFbx::FFbxImporter::ImportBones) seem to behave the same way and flip all axes when a reflection is
			// detected.
			{
				const pxr::GfMatrix4d& UsdBindTransform = JointLocalBindTransforms[Index];
				FMatrix Matrix = UsdToUnreal::ConvertMatrix(UsdBindTransform);
				FTransform BindTransform = UsdUtils::DecomposeWithUniformReflection(Matrix);
				BindTransform = UsdUtils::ConvertTransformToUESpace(StageInfo, BindTransform);
				Bone.LocalBindTransform = BindTransform;
			}
		}
	}

	// If we have more than one root bone, let's create a new "true root bone" and add the
	// previously root bones as children of it
	if (bEnsureSingleRootBone && RootBoneCount > 1)
	{
		TSet<FString> BoneNames;
		for (FUsdSkeletonData::FBone& Bone : OutConvertedData.Bones)
		{
			BoneNames.Add(Bone.Name);

			// Have previously root bones point at the new bone we'll add soon
			if (Bone.ParentIndex == INDEX_NONE)
			{
				Bone.ParentIndex = 0;
			}
			// All other index references have to move one over since we'll push
			// a new root bone into the start of the array
			else
			{
				Bone.ParentIndex += 1;
			}
		}

		FUsdSkeletonData::FBone TrueRoot;
		TrueRoot.Name = UsdUnreal::ObjectUtils::GetUniqueName(TEXT("Root"), BoneNames);
		TrueRoot.ParentIndex = INDEX_NONE;
		OutConvertedData.Bones.Insert(TrueRoot, 0);
	}

	// Fill in child indices (easier now so we don't have to remap them for multiple root bones)
	for (int32 Index = 0; Index < NumBones; ++Index)
	{
		FUsdSkeletonData::FBone& Bone = OutConvertedData.Bones[Index];
		if (Bone.ParentIndex >= 0)
		{
			FUsdSkeletonData::FBone& ParentBone = OutConvertedData.Bones[Bone.ParentIndex];
			ParentBone.ChildIndices.Add(Index);
		}
	}

	return true;
}

bool UsdToUnreal::ConvertSkeleton(const pxr::UsdSkelSkeletonQuery& SkeletonQuery, FSkeletalMeshImportData& SkelMeshImportData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UsdToUnreal::ConvertSkeleton);

	FUsdSkeletonData TempData;
	const bool bEnsureAtLeastOneBone = true;
	const bool bEnsureSingleRootBone = true;
	const bool bSuccess = ConvertSkeleton(SkeletonQuery, TempData, bEnsureAtLeastOneBone, bEnsureSingleRootBone);
	if (!bSuccess)
	{
		return false;
	}

	// Store the retrieved data as bones into the SkeletalMeshImportData
	const int32 NumBones = TempData.Bones.Num();
	SkelMeshImportData.RefBonesBinary.AddZeroed(NumBones);
	for (int32 Index = 0; Index < NumBones; ++Index)
	{
		FUsdSkeletonData::FBone& InBone = TempData.Bones[Index];
		SkeletalMeshImportData::FBone& OutBone = SkelMeshImportData.RefBonesBinary[Index];

		OutBone.Name = InBone.Name;
		OutBone.ParentIndex = InBone.ParentIndex;
		OutBone.NumChildren = InBone.ChildIndices.Num();

		// Not sure if Length and X/Y/Z Size need to be set, there are no equivalents in USD
		SkeletalMeshImportData::FJointPos& JointMatrix = OutBone.BonePos;
		JointMatrix.Length = 1.f;
		JointMatrix.XSize = 100.f;
		JointMatrix.YSize = 100.f;
		JointMatrix.ZSize = 100.f;
		JointMatrix.Transform = FTransform3f(InBone.LocalBindTransform);
	}

	return true;
}

namespace UE::USDSkeletalDataConversion::Private
{
	bool HasMultipleRootBones(const pxr::UsdSkelSkeletonQuery& SkeletonQuery)
	{
		const pxr::UsdSkelTopology& SkelTopology = SkeletonQuery.GetTopology();
		const pxr::VtArray<int>& JointParentIndices = SkelTopology.GetParentIndices();
		bool bFoundRoot = false;
		for (int ParentIndex : JointParentIndices)
		{
			if (ParentIndex == INDEX_NONE)
			{
				if (bFoundRoot)
				{
					return true;
				}
				bFoundRoot = true;
			}
		}

		return false;
	}
};

bool UsdToUnreal::ConvertSkinnedMesh(
	const pxr::UsdSkelSkinningQuery& SkinningQuery,
	const pxr::UsdSkelSkeletonQuery& SkeletonQuery,
	FSkeletalMeshImportData& SkelMeshImportData,
	UsdUtils::FUsdPrimMaterialAssignmentInfo& MaterialAssignments,
	const FUsdMeshConversionOptions& CommonOptions
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UsdToUnreal::ConvertSkinnedMesh);

	using namespace pxr;

	const UsdPrim& SkinningPrim = SkinningQuery.GetPrim();
	UsdSkelBindingAPI SkelBindingAPI(SkinningPrim);
	if (!SkelBindingAPI)
	{
		return false;
	}

	// Ref. FFbxImporter::FillSkelMeshImporterFromFbx
	UsdGeomMesh UsdMesh = UsdGeomMesh(SkinningPrim);
	if (!UsdMesh)
	{
		return false;
	}

	const FUsdStageInfo StageInfo(SkinningPrim.GetStage());

	// Collect GeomBindTransform if we have one
	pxr::GfMatrix4d GeomBindTransform{1.0};
	pxr::GfMatrix4d InvTransposeGeomBindTransform{1.0};
	{
		GeomBindTransform = SkinningQuery.GetGeomBindTransform(CommonOptions.TimeCode);
		if (GeomBindTransform != pxr::GfMatrix4d(1.0))	  // This is USD's identity
		{
			if (GeomBindTransform.GetDeterminant() == 0.0)
			{
				// Can't invert, just use as-is
				USD_LOG_WARNING(TEXT("Failed to invert geomBindTransform for prim '%s'"), *UsdToUnreal::ConvertPath(SkinningPrim.GetPrimPath()));
				InvTransposeGeomBindTransform = GeomBindTransform;
			}
			else
			{
				InvTransposeGeomBindTransform = GeomBindTransform.GetInverse().GetTranspose();
			}
		}
	}

	// Retrieve the mesh points (vertices) from USD and append it to the SkeletalMeshImportData Points
	uint32 NumPoints = 0;
	uint32 NumExistingPoints = SkelMeshImportData.Points.Num();

	UsdAttribute PointsAttr = UsdMesh.GetPointsAttr();
	if (PointsAttr)
	{
		VtArray<GfVec3f> UsdPoints;
		PointsAttr.Get(&UsdPoints, UsdTimeCode::Default());

		NumPoints = UsdPoints.size();
		SkelMeshImportData.Points.AddUninitialized(NumPoints);

		for (uint32 PointIndex = 0; PointIndex < NumPoints; ++PointIndex)
		{
			const GfVec3d Point = GeomBindTransform.Transform(UsdPoints[PointIndex]);
			SkelMeshImportData.Points[PointIndex + NumExistingPoints] = (FVector3f)CommonOptions.AdditionalTransform.TransformPosition(
				UsdToUnreal::ConvertVector(StageInfo, Point)
			);
		}
	}

	if (NumPoints == 0)
	{
		return false;
	}

	// Convert the face data into SkeletalMeshImportData

	// Face counts
	VtArray<int> FaceCounts;
	UsdAttribute FaceCountsAttribute = UsdMesh.GetFaceVertexCountsAttr();
	if (FaceCountsAttribute)
	{
		FaceCountsAttribute.Get(&FaceCounts, UsdTimeCode::Default());
	}

	// Face indices
	VtArray<int> OriginalFaceIndices;
	UsdAttribute FaceIndicesAttribute = UsdMesh.GetFaceVertexIndicesAttr();
	if (FaceIndicesAttribute)
	{
		FaceIndicesAttribute.Get(&OriginalFaceIndices, UsdTimeCode::Default());
	}

	uint32 NumVertexInstances = static_cast<uint32>(OriginalFaceIndices.size());

	// Normals
	TArray<FVector3f> Normals;//Transformed
	VtArray<GfVec3f> USDNormals;
	UsdAttribute NormalsAttribute = UsdMesh.GetNormalsAttr();
	bool bUsingVertexInstanceNormals = false;
	if (NormalsAttribute)
	{
		FMatrix TotalMatrix = CommonOptions.AdditionalTransform.ToMatrixWithScale();
		FMatrix TotalMatrixForNormal = TotalMatrix.Inverse().GetTransposed();

		if (NormalsAttribute.Get(&USDNormals, UsdTimeCode::Default()) && USDNormals.size() > 0)
		{
			if (USDNormals.size() == NumPoints || USDNormals.size() == NumVertexInstances)
			{
				bUsingVertexInstanceNormals = USDNormals.size() == NumVertexInstances;
				Normals.Reserve(bUsingVertexInstanceNormals ? NumVertexInstances : NumPoints);
				SkelMeshImportData.bHasNormals = true;

				for (GfVec3f& USDNormal : USDNormals)
				{
					USDNormal = static_cast<pxr::GfVec3f>(InvTransposeGeomBindTransform.TransformDir(USDNormal));
					FVector UENormal = TotalMatrixForNormal.TransformVector(UsdToUnreal::ConvertVector(StageInfo, USDNormal)).GetSafeNormal();
					Normals.Add((FVector3f)UENormal);
				}
			}
		}
	}

	uint32 NumExistingFaces = SkelMeshImportData.Faces.Num();
	uint32 NumExistingWedges = SkelMeshImportData.Wedges.Num();

	uint32 NumFaces = FaceCounts.size();
	SkelMeshImportData.Faces.Reserve(NumFaces * 2);

	// Material assignments
	const bool bProvideMaterialIndices = true;
	UsdUtils::FUsdPrimMaterialAssignmentInfo LocalInfo = UsdUtils::GetPrimMaterialAssignments(
		SkinningPrim,
		pxr::UsdTimeCode::EarliestTime(),
		bProvideMaterialIndices,
		CommonOptions.RenderContext,
		CommonOptions.MaterialPurpose
	);
	TArray<UsdUtils::FUsdPrimMaterialSlot>& LocalMaterialSlots = LocalInfo.Slots;
	TArray<int32>& FaceMaterialIndices = LocalInfo.MaterialIndices;

	// We want to combine identical slots for skeletal meshes, which is different to static meshes, where each section gets a slot
	// Note: This is a different index remapping to the one that happens for LODs, using LODMaterialMap! Here we're combining meshes of the same LOD
	TMap<UsdUtils::FUsdPrimMaterialSlot, int32> SlotToCombinedMaterialIndex;

	// Position 3 in this has the value 6 --> Local material slot #3 is actually the combined material slot #6
	TArray<int32> LocalToCombinedMaterialIndex;
	LocalToCombinedMaterialIndex.SetNumZeroed(LocalInfo.Slots.Num());

	for (int32 Index = 0; Index < MaterialAssignments.Slots.Num(); ++Index)
	{
		const UsdUtils::FUsdPrimMaterialSlot& Slot = MaterialAssignments.Slots[Index];

		// Combine entries in this way so that we can append PrimPaths
		TMap<UsdUtils::FUsdPrimMaterialSlot, int32>::TKeyIterator KeyIt = SlotToCombinedMaterialIndex.CreateKeyIterator(Slot);
		if (KeyIt)
		{
			KeyIt.Key().PrimPaths.Append(Slot.PrimPaths);
			KeyIt.Value() = Index;
		}
		else
		{
			SlotToCombinedMaterialIndex.Add(Slot, Index);
		}
	}
	for (int32 LocalIndex = 0; LocalIndex < LocalInfo.Slots.Num(); ++LocalIndex)
	{
		const UsdUtils::FUsdPrimMaterialSlot& LocalSlot = LocalInfo.Slots[LocalIndex];

		// Combine entries in this way so that we can append PrimPaths
		TMap<UsdUtils::FUsdPrimMaterialSlot, int32>::TKeyIterator KeyIt = SlotToCombinedMaterialIndex.CreateKeyIterator(LocalSlot);
		if (KeyIt)
		{
			KeyIt.Key().PrimPaths.Append(LocalSlot.PrimPaths);

			const int32 ExistingCombinedIndex = KeyIt.Value();
			LocalToCombinedMaterialIndex[LocalIndex] = ExistingCombinedIndex;
		}
		else
		{
			int32 NewIndex = MaterialAssignments.Slots.Add(LocalSlot);
			SlotToCombinedMaterialIndex.Add(LocalSlot, NewIndex);
			LocalToCombinedMaterialIndex[LocalIndex] = NewIndex;
		}
	}
	// Now that we merged all prim paths into they keys of SlotToCombinedMaterialIndex, let's copy them back into
	// our output
	for (UsdUtils::FUsdPrimMaterialSlot& Slot : MaterialAssignments.Slots)
	{
		TMap<UsdUtils::FUsdPrimMaterialSlot, int32>::TKeyIterator KeyIt = SlotToCombinedMaterialIndex.CreateKeyIterator(Slot);
		ensure(KeyIt);
		Slot.PrimPaths = KeyIt.Key().PrimPaths;
	}

	// Retrieve vertex colors
	UsdGeomPrimvar ColorPrimvar = UsdMesh.GetDisplayColorPrimvar();
	TArray<FColor> Colors;
	EUsdInterpolationMethod DisplayColorInterp = EUsdInterpolationMethod::Constant;
	if (ColorPrimvar)
	{
		pxr::VtArray<pxr::GfVec3f> UsdColors;
		if (ColorPrimvar.ComputeFlattened(&UsdColors))
		{
			uint32 NumExpectedColors = 0;
			uint32 NumColors = UsdColors.size();
			pxr::TfToken USDInterpType = ColorPrimvar.GetInterpolation();

			if (USDInterpType == pxr::UsdGeomTokens->uniform)
			{
				NumExpectedColors = NumFaces;
				DisplayColorInterp = EUsdInterpolationMethod::Uniform;
			}
			else if (USDInterpType == pxr::UsdGeomTokens->vertex || USDInterpType == pxr::UsdGeomTokens->varying)
			{
				NumExpectedColors = NumPoints;
				DisplayColorInterp = EUsdInterpolationMethod::Vertex;
			}
			else if (USDInterpType == pxr::UsdGeomTokens->faceVarying)
			{
				NumExpectedColors = NumVertexInstances;
				DisplayColorInterp = EUsdInterpolationMethod::FaceVarying;
			}
			else if (USDInterpType == pxr::UsdGeomTokens->constant)
			{
				NumExpectedColors = 1;
				DisplayColorInterp = EUsdInterpolationMethod::Constant;
			}

			if (NumExpectedColors == NumColors)
			{
				Colors.Reserve(NumColors);
				for (uint32 Index = 0; Index < NumColors; ++Index)
				{
					const bool bSRGB = true;
					Colors.Add(UsdToUnreal::ConvertColor(UsdColors[Index]).ToFColor(bSRGB));
				}

				SkelMeshImportData.bHasVertexColors = true;
			}
			else
			{
				USD_LOG_USERWARNING(FText::Format(
					LOCTEXT(
						"InvalidDisplayColorInterpolation",
						"Prim '{0}' has invalid number of displayColor values for primvar interpolation type '{1}'! (expected {2}, found {3})"
					),
					FText::FromString(UsdToUnreal::ConvertPath(SkinningPrim.GetPath())),
					FText::FromString(UsdToUnreal::ConvertToken(USDInterpType)),
					NumExpectedColors,
					NumColors
				));
			}
		}
	}

	// Retrieve vertex opacity
	UsdGeomPrimvar OpacityPrimvar = UsdMesh.GetDisplayOpacityPrimvar();
	TArray<float> Opacities;
	EUsdInterpolationMethod DisplayOpacityInterp = EUsdInterpolationMethod::Constant;
	if (OpacityPrimvar)
	{
		pxr::VtArray<float> UsdOpacities;
		if (OpacityPrimvar.ComputeFlattened(&UsdOpacities))
		{
			uint32 NumExpectedOpacities = 0;
			const uint32 NumOpacities = UsdOpacities.size();
			pxr::TfToken USDInterpType = OpacityPrimvar.GetInterpolation();

			if (USDInterpType == pxr::UsdGeomTokens->uniform)
			{
				NumExpectedOpacities = NumFaces;
				DisplayOpacityInterp = EUsdInterpolationMethod::Uniform;
			}
			else if (USDInterpType == pxr::UsdGeomTokens->vertex || USDInterpType == pxr::UsdGeomTokens->varying)
			{
				NumExpectedOpacities = NumPoints;
				DisplayOpacityInterp = EUsdInterpolationMethod::Vertex;
			}
			else if (USDInterpType == pxr::UsdGeomTokens->faceVarying)
			{
				NumExpectedOpacities = NumVertexInstances;
				DisplayOpacityInterp = EUsdInterpolationMethod::FaceVarying;
			}
			else if (USDInterpType == pxr::UsdGeomTokens->constant)
			{
				NumExpectedOpacities = 1;
				DisplayOpacityInterp = EUsdInterpolationMethod::Constant;
			}

			if (NumExpectedOpacities == NumOpacities)
			{
				Opacities.Reserve(NumOpacities);
				for (uint32 Index = 0; Index < NumOpacities; ++Index)
				{
					Opacities.Add(UsdOpacities[Index]);
				}

				SkelMeshImportData.bHasVertexColors = true;	   // We'll need to store these in the vertex colors
			}
			else
			{
				USD_LOG_USERWARNING(FText::Format(
					LOCTEXT(
						"InvalidDisplayOpacityInterpolation",
						"Prim '{0}' has invalid number of displayOpacity values for primvar interpolation type '{1}'! (expected {2}, found {3})"
					),
					FText::FromString(UsdToUnreal::ConvertPath(SkinningPrim.GetPath())),
					FText::FromString(UsdToUnreal::ConvertToken(USDInterpType)),
					NumExpectedOpacities,
					NumOpacities
				));
			}
		}
	}

	// Make sure these have at least one valid entry, as we'll default to Constant and we may have either valid opacities or colors
	if (Colors.Num() < 1)
	{
		Colors.Add(FColor::White);
	}
	if (Opacities.Num() < 1)
	{
		Opacities.Add(1.0f);
	}

	bool bReverseOrder = IUsdPrim::GetGeometryOrientation(UsdMesh) == EUsdGeomOrientation::LeftHanded;

	struct FUVSet
	{
		TOptional<VtIntArray> UVIndices;	// UVs might be indexed or they might be flat (one per vertex)
		VtVec2fArray UVs;

		EUsdInterpolationMethod InterpolationMethod = EUsdInterpolationMethod::FaceVarying;
	};

	TArray<FUVSet> UVSets;

	// If we already have a primvar to UV index assignment, let's just use that.
	// When collapsing, we'll do a pre-pass on all meshes to translate and determine this beforehand.
	TArray<TUsdStore<pxr::UsdGeomPrimvar>> PrimvarsByUVIndex;
	if (MaterialAssignments.PrimvarToUVIndex.Num() > 0)
	{
		TArray<TUsdStore<pxr::UsdGeomPrimvar>> AllMeshUVPrimvars = UsdUtils::GetUVSetPrimvars(SkinningPrim, TNumericLimits<int32>::Max());

		PrimvarsByUVIndex = UsdUtils::AssemblePrimvarsIntoUVSets(AllMeshUVPrimvars, MaterialAssignments.PrimvarToUVIndex);
	}
	// Let's use the best primvar assignment for this particular mesh instead
	else
	{
		PrimvarsByUVIndex = UsdUtils::GetUVSetPrimvars(SkinningPrim);

		MaterialAssignments.PrimvarToUVIndex = UsdUtils::AssemblePrimvarsIntoPrimvarToUVIndexMap(PrimvarsByUVIndex);
	}

	int32 UVChannelIndex = 0;
	while (true)
	{
		if (!PrimvarsByUVIndex.IsValidIndex(UVChannelIndex))
		{
			break;
		}

		UsdGeomPrimvar& PrimvarST = PrimvarsByUVIndex[UVChannelIndex].Get();
		if (!PrimvarST)
		{
			break;
		}

		if (PrimvarST)
		{
			FUVSet UVSet;

			if (PrimvarST.GetInterpolation() == UsdGeomTokens->vertex)
			{
				UVSet.InterpolationMethod = EUsdInterpolationMethod::Vertex;
			}
			else if (PrimvarST.GetInterpolation() == UsdGeomTokens->faceVarying)
			{
				UVSet.InterpolationMethod = EUsdInterpolationMethod::FaceVarying;
			}
			else if (PrimvarST.GetInterpolation() == UsdGeomTokens->uniform)
			{
				UVSet.InterpolationMethod = EUsdInterpolationMethod::Uniform;
			}
			else if (PrimvarST.GetInterpolation() == UsdGeomTokens->constant)
			{
				UVSet.InterpolationMethod = EUsdInterpolationMethod::Constant;
			}

			if (PrimvarST.IsIndexed())
			{
				UVSet.UVIndices.Emplace();

				if (PrimvarST.GetIndices(&UVSet.UVIndices.GetValue()) && PrimvarST.Get(&UVSet.UVs))
				{
					if (UVSet.UVs.size() > 0)
					{
						UVSets.Add(MoveTemp(UVSet));

						if (UVSets.Num() == MAX_TEXCOORDS)
						{
							break;
						}
					}
				}
			}
			else
			{
				if (PrimvarST.Get(&UVSet.UVs))
				{
					if (UVSet.UVs.size() > 0)
					{
						UVSets.Add(MoveTemp(UVSet));

						if (UVSets.Num() == MAX_TEXCOORDS)
						{
							break;
						}
					}
				}
			}
		}
		else
		{
			break;
		}

		++UVChannelIndex;
	}

	// Force our mesh data to always have at least one UV set.
	// This so that we can have consistency across all our translated meshes, but it should only be needed in case
	// Interchange wants to use the FMeshDescription produced from this SkelMeshImportData for a StaticMesh build,
	// which assumes that at least one UV set is always available (Check the GetRawArray(0) within FStaticMeshOperations::ComputeMikktTangents,
	// which was causing a crash for UE-224831)
	if (UVSets.Num() == 0)
	{
		FUVSet& EmptySet = UVSets.Emplace_GetRef();
		EmptySet.UVs.push_back(pxr::GfVec2f{0.0f, 0.0f});
		EmptySet.InterpolationMethod = EUsdInterpolationMethod::Constant;
	}

	SkelMeshImportData.NumTexCoords = FMath::Clamp(FMath::Max(SkelMeshImportData.NumTexCoords, (uint32)UVSets.Num()), 0, MAX_TEXCOORDS);

	SkelMeshImportData.Wedges.Reserve((NumExistingFaces + NumFaces) * 6);

	uint32 NumSkippedPolygons = 0;

	uint32 NumProcessedFaceVertexIndices = 0;
	for (uint32 PolygonIndex = NumExistingFaces, LocalIndex = 0; PolygonIndex < NumExistingFaces + NumFaces; ++PolygonIndex, ++LocalIndex)
	{
		const uint32 NumOriginalFaceVertices = FaceCounts[LocalIndex];
		const uint32 NumFinalFaceVertices = 3;

		// Skip "polygon" if it has less than 3 vertices
		if (NumOriginalFaceVertices < 3)
		{
			++NumSkippedPolygons;
			NumProcessedFaceVertexIndices += NumOriginalFaceVertices;
			continue;
		}

		// Manage materials
		int32 LocalMaterialIndex = 0;
		if (FaceMaterialIndices.IsValidIndex(LocalIndex))
		{
			LocalMaterialIndex = FaceMaterialIndices[LocalIndex];
			if (!LocalMaterialSlots.IsValidIndex(LocalMaterialIndex))
			{
				LocalMaterialIndex = 0;
			}
		}

		int32 RealMaterialIndex = LocalToCombinedMaterialIndex[LocalMaterialIndex];
		SkelMeshImportData.MaxMaterialIndex = FMath::Max<uint32>(SkelMeshImportData.MaxMaterialIndex, RealMaterialIndex);

		// The SkelMeshImportData now requires that the Materials array has number of entries that matches
		// the max material index
		// TODO: This really doesn't need to be done *per polygon*... It could be done outside the current for loop
		SkelMeshImportData.Materials.SetNum(SkelMeshImportData.MaxMaterialIndex + 1);
		for (int32 Index = 0; Index < SkelMeshImportData.Materials.Num(); ++Index)
		{
			SkeletalMeshImportData::FMaterial& Material = SkelMeshImportData.Materials[Index];
			Material.MaterialImportName = LexToString(Index);
			Material.Material = UMaterial::GetDefaultMaterial(MD_Surface);
		}

		// SkeletalMeshImportData uses triangle faces so quads will have to be split into triangles
		const bool bIsQuad = (NumOriginalFaceVertices == 4);
		const uint32 NumTriangles = bIsQuad ? 2 : 1;

		for (uint32 TriangleIndex = 0; TriangleIndex < NumTriangles; ++TriangleIndex)
		{
			// This needs to be zeroed as we'll hash these faces later
			int32 TriangleFaceIndex = SkelMeshImportData.Faces.AddZeroed();

			SkeletalMeshImportData::FTriangle& Triangle = SkelMeshImportData.Faces[TriangleFaceIndex];

			// Set the face smoothing by default. It could be any number, but not zero
			Triangle.SmoothingGroups = 255;

			Triangle.MatIndex = RealMaterialIndex;
			Triangle.AuxMatIndex = 0;

			// Fill the wedge data and complete the triangle setup with the wedge indices
			for (uint32 CornerIndex = 0; CornerIndex < NumFinalFaceVertices; ++CornerIndex)
			{
				uint32 OriginalCornerIndex = ((TriangleIndex * (NumOriginalFaceVertices - 2)) + CornerIndex) % NumOriginalFaceVertices;
				uint32 OriginalVertexInstanceIndex = NumProcessedFaceVertexIndices + OriginalCornerIndex;
				int32 OriginalVertexIndex = OriginalFaceIndices[OriginalVertexInstanceIndex];

				int32 FinalCornerIndex = bReverseOrder ? NumFinalFaceVertices - 1 - CornerIndex : CornerIndex;

				// Its important to make sure the UVs aren't just uninitialized memory because BuildSkeletalMesh will read them
				// when trying to merge vertices. Uninitialized memory would lead to inconsistent, non-deterministic meshes
				const uint32 WedgeIndex = SkelMeshImportData.Wedges.AddZeroed();
				SkeletalMeshImportData::FVertex& SkelMeshWedge = SkelMeshImportData.Wedges[WedgeIndex];

				if (SkelMeshImportData.bHasVertexColors)
				{
					uint32 DisplayColorIndex = UsdToUnrealImpl::GetPrimValueIndex(
						DisplayColorInterp,
						OriginalVertexIndex,
						OriginalVertexInstanceIndex,
						LocalIndex
					);
					uint32 DisplayOpacityIndex = UsdToUnrealImpl::GetPrimValueIndex(
						DisplayOpacityInterp,
						OriginalVertexIndex,
						OriginalVertexInstanceIndex,
						LocalIndex
					);

					const FColor& DisplayColor = Colors[DisplayColorIndex];

					SkelMeshWedge.Color.R = DisplayColor.R;
					SkelMeshWedge.Color.G = DisplayColor.G;
					SkelMeshWedge.Color.B = DisplayColor.B;
					SkelMeshWedge.Color.A = static_cast<uint8>(FMath::Clamp(Opacities[DisplayOpacityIndex], 0.0f, 1.0f) * 255.0f + 0.5f);
				}

				SkelMeshWedge.MatIndex = Triangle.MatIndex;
				SkelMeshWedge.VertexIndex = NumExistingPoints + OriginalVertexIndex;
				SkelMeshWedge.Reserved = 0;

				int32 UVLayerIndex = 0;
				for (const FUVSet& UVSet : UVSets)
				{
					int32 ValueIndex = 0;

					if (UVSet.InterpolationMethod == EUsdInterpolationMethod::Vertex)
					{
						ValueIndex = OriginalVertexIndex;
					}
					else if (UVSet.InterpolationMethod == EUsdInterpolationMethod::FaceVarying)
					{
						ValueIndex = OriginalVertexInstanceIndex;
					}
					else if (UVSet.InterpolationMethod == EUsdInterpolationMethod::Uniform)
					{
						ValueIndex = PolygonIndex;
					}
					else if (UVSet.InterpolationMethod == EUsdInterpolationMethod::Constant)
					{
						ValueIndex = 0;
					}

					GfVec2f UV(0.f, 0.f);

					if (UVSet.UVIndices.IsSet())
					{
						if (ensure(UVSet.UVIndices.GetValue().size() > ValueIndex))
						{
							UV = UVSet.UVs[UVSet.UVIndices.GetValue()[ValueIndex]];
						}
					}
					else if (ensure(UVSet.UVs.size() > ValueIndex))
					{
						UV = UVSet.UVs[ValueIndex];
					}

					// Flip V for Unreal uv's which match directx
					FVector2f FinalUVVector(UV[0], 1.f - UV[1]);
					SkelMeshWedge.UVs[UVLayerIndex] = FinalUVVector;

					++UVLayerIndex;
				}

				Triangle.TangentX[FinalCornerIndex] = FVector3f::ZeroVector;
				Triangle.TangentY[FinalCornerIndex] = FVector3f::ZeroVector;
				Triangle.TangentZ[FinalCornerIndex] = FVector3f::ZeroVector;

				Triangle.WedgeIndex[FinalCornerIndex] = WedgeIndex;

				if (SkelMeshImportData.bHasNormals && Normals.Num())
				{
					if (bUsingVertexInstanceNormals && Normals.IsValidIndex(OriginalVertexInstanceIndex))
					{
						Triangle.TangentZ[FinalCornerIndex] = Normals[OriginalVertexInstanceIndex];
					}
					else if (Normals.IsValidIndex(OriginalVertexIndex))
					{
						Triangle.TangentZ[FinalCornerIndex] = Normals[OriginalVertexIndex];
					}
				}
			}
		}

		NumProcessedFaceVertexIndices += NumOriginalFaceVertices;
	}

	if (NumSkippedPolygons > 0)
	{
		USD_LOG_WARNING(
			TEXT("Ignoring %d polygons with less than 3 vertices from mesh '%s'"),
			NumSkippedPolygons,
			*UsdToUnreal::ConvertPath(UsdMesh.GetPrim().GetPrimPath())
		);
	}

	// Convert joint influences into the SkeletalMeshImportData

	// ComputeJointInfluences returns the influences per bone that applies to all the points of the mesh
	// ComputeVaryingJointInfluences returns the joint influences for each points, expanding the influences to all points if the mesh is rigidly
	// deformed
	VtArray<int> JointIndices;
	VtArray<float> JointWeights;
	SkinningQuery.ComputeVaryingJointInfluences(NumPoints, &JointIndices, &JointWeights);

	// Keep track of whether we added an additional "true" root bone in the cases the bound skeleton has
	// multiple root bones
	// We'll only ever set NumAdditionalBones to 1 or 0 (as we'll only either need a "true root bone" or
	// not), but naming it this way allows us to use it like an offset, which should make it easier to
	// understand whenever it is used
	uint32 NumAdditionalBones = UE::USDSkeletalDataConversion::Private::HasMultipleRootBones(SkeletonQuery) ? 1 : 0;

	// Recompute the joint influences if we need to
	uint32 NumInfluencesPerComponent = SkinningQuery.GetNumInfluencesPerComponent();
	const uint32 MaxAllowedInfluences = EXTRA_BONE_INFLUENCES;
	const bool bUseUnlimitedBoneInfluences = FGPUBaseSkinVertexFactory::UseUnlimitedBoneInfluences(NumInfluencesPerComponent);
	if (NumInfluencesPerComponent > MaxAllowedInfluences && !bUseUnlimitedBoneInfluences)
	{
		UsdSkelResizeInfluences(&JointIndices, NumInfluencesPerComponent, MaxAllowedInfluences);
		UsdSkelResizeInfluences(&JointWeights, NumInfluencesPerComponent, MaxAllowedInfluences);
		NumInfluencesPerComponent = MaxAllowedInfluences;
	}

	// We keep track of which influences we added because we combine many Mesh prim (each with potentially a different
	// explicit joint order) into the same skeletal mesh asset
	const int32 NumInfluencesBefore = SkelMeshImportData.Influences.Num();
	if (JointWeights.size() > (NumPoints - 1) * (NumInfluencesPerComponent - 1))
	{
		uint32 JointIndex = 0;
		SkelMeshImportData.Influences.Reserve(NumPoints);
		for (uint32 PointIndex = 0; PointIndex < NumPoints; ++PointIndex)
		{
			// The JointIndices/JointWeights contain the influences data for NumPoints * NumInfluencesPerComponent
			for (uint32 InfluenceIndex = 0; InfluenceIndex < NumInfluencesPerComponent; ++InfluenceIndex, ++JointIndex)
			{
				// BoneWeight could be 0 if the actual number of influences were less than NumInfluencesPerComponent for a given point so just ignore
				// it
				float BoneWeight = JointWeights[JointIndex];
				if (BoneWeight != 0.f)
				{
					SkelMeshImportData.Influences.AddUninitialized();
					SkelMeshImportData.Influences.Last().BoneIndex = NumAdditionalBones + JointIndices[JointIndex];
					SkelMeshImportData.Influences.Last().Weight = BoneWeight;
					SkelMeshImportData.Influences.Last().VertexIndex = NumExistingPoints + PointIndex;
				}
			}
		}
	}
	const int32 NumInfluencesAfter = SkelMeshImportData.Influences.Num();

	// If we have a joint mapper this Mesh has an explicit joint ordering, so we need to map joint indices to the skeleton's bone indices
	if (pxr::UsdSkelAnimMapperRefPtr AnimMapper = SkinningQuery.GetJointMapper())
	{
		VtArray<int> SkeletonBoneIndices;
		if (pxr::UsdSkelSkeleton BoundSkeleton = SkelBindingAPI.GetInheritedSkeleton())
		{
			if (pxr::UsdAttribute SkeletonJointsAttr = BoundSkeleton.GetJointsAttr())
			{
				VtArray<TfToken> SkeletonJoints;
				if (SkeletonJointsAttr.Get(&SkeletonJoints))
				{
					// If the skeleton has N bones, this will just contain { 0, 1, 2, ..., N-1 }
					int NumUsdSkeletonBones = static_cast<int>(SkeletonJoints.size());
					for (int SkeletonBoneIndex = 0; SkeletonBoneIndex < NumUsdSkeletonBones; ++SkeletonBoneIndex)
					{
						SkeletonBoneIndices.push_back(SkeletonBoneIndex);
					}

					// Use the AnimMapper to produce the indices of the Mesh's joints within the Skeleton's list of joints.
					// Example: Imagine skeleton had { "Root", "Root/Hip", "Root/Hip/Shoulder", "Root/Hip/Shoulder/Arm", "Root/Hip/Shoulder/Arm/Elbow"
					// }, and so BoneIndexRemapping was { 0, 1, 2, 3, 4 }. Consider a Mesh that specifies the explicit joints { "Root/Hip/Shoulder",
					// "Root/Hip/Shoulder/Arm" }, and so uses the indices 0 and 1 to refer to Shoulder and Arm. After the Remap call
					// SkeletonBoneIndices will hold { 2, 3 }, as those are the indices of Shoulder and Arm within the skeleton's bones
					VtArray<int> BoneIndexRemapping;
					if (AnimMapper->Remap(SkeletonBoneIndices, &BoneIndexRemapping))
					{
						for (int32 AddedInfluenceIndex = NumInfluencesBefore; AddedInfluenceIndex < NumInfluencesAfter; ++AddedInfluenceIndex)
						{
							SkeletalMeshImportData::FRawBoneInfluence& Influence = SkelMeshImportData.Influences[AddedInfluenceIndex];

							// We have to remove our "NumAdditionalBones" offset from the influence's bone index because that's a UE concept that
							// the BoneIndexRemapping array doesn't really know about. After that, we have a bone index that matches the USD Skeleton
							// joint order, then we can remap with BoneIndexRemapping and add our NumAdditionalBones back in so that it matches our
							// USkeleton
							Influence.BoneIndex = NumAdditionalBones + BoneIndexRemapping[Influence.BoneIndex - NumAdditionalBones];
						}
					}
				}
			}
		}
	}

	return true;
}

// Using UsdSkelSkeletonQuery instead of UsdSkelAnimQuery as it automatically does the joint remapping when we ask it to compute joint transforms.
// It also initializes the joint transforms with the rest pose, if available, in case the animation doesn't provide data for all joints.
bool UsdToUnreal::ConvertSkelAnim(
	const pxr::UsdSkelSkeletonQuery& InUsdSkeletonQuery,
	const pxr::VtArray<pxr::UsdSkelSkinningQuery>* InSkinningTargets,
	const UsdUtils::FBlendShapeMap* InBlendShapes,
	bool bInInterpretLODs,
	const pxr::UsdPrim& RootMotionPrim,
	UAnimSequence* OutSkeletalAnimationAsset,
	float* OutStartOffsetSeconds,
	float* OutScaleSeconds
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UsdToUnreal::ConvertSkelAnim);

	FScopedUnrealAllocs UEAllocs;

	if (!InUsdSkeletonQuery || !OutSkeletalAnimationAsset)
	{
		return false;
	}

	// If we have no skeleton we can't add animation data to the AnimSequence, so we may as well just return
	USkeleton* Skeleton = OutSkeletalAnimationAsset->GetSkeleton();
	if (!Skeleton)
	{
		return false;
	}

	TUsdStore<pxr::UsdSkelAnimQuery> AnimQuery = InUsdSkeletonQuery.GetAnimQuery();
	if (!AnimQuery.Get())
	{
		return false;
	}

	pxr::UsdPrim SkelAnimPrim = AnimQuery.Get().GetPrim();
	UE::FSdfLayerOffset Offset = UsdUtils::GetPrimToStageOffset(UE::FUsdPrim{SkelAnimPrim});

	UE::FSdfLayer SkelAnimPrimLayer = UsdUtils::FindLayerForPrim(SkelAnimPrim);
	double LayerTimeCodesPerSecond = SkelAnimPrimLayer.GetTimeCodesPerSecond();

	TUsdStore<pxr::UsdStageWeakPtr> Stage(InUsdSkeletonQuery.GetPrim().GetStage());
	FUsdStageInfo StageInfo{Stage.Get()};
	double StageTimeCodesPerSecond = Stage.Get()->GetTimeCodesPerSecond();
	if (FMath::IsNearlyZero(StageTimeCodesPerSecond))
	{
		USD_LOG_USERWARNING(LOCTEXT("TimeCodesPerSecondIsZero", "Cannot bake skeletal animations as the stage has timeCodesPerSecond set to zero!"));
		return false;
	}

	const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();
	const TArray<FMeshBoneInfo>& BoneInfo = RefSkeleton.GetRawRefBoneInfo();
	int32 NumBonesInUE = BoneInfo.Num();	// This will already contain any new "true root bone" we may have created
	int32 NumBonesInUsd = static_cast<int32>(InUsdSkeletonQuery.GetJointOrder().size());

	// Keep track of whether we added an additional "true" root bone in the cases the bound skeleton has
	// multiple root bones
	// We'll only ever set NumAdditionalBones to 1 or 0 (as we'll only either need a "true root bone" or
	// not), but naming it this way allows us to use it like an offset, which should make it easier to
	// understand whenever it is used
	uint32 NumAdditionalBones = UE::USDSkeletalDataConversion::Private::HasMultipleRootBones(InUsdSkeletonQuery) ? 1 : 0;

	// If we have zero bones on our skeleton we'll generate a dummy "Root" bone just so that Unreal can have a USkeleton asset,
	// so we have to check for that case
	if (((NumBonesInUsd + NumAdditionalBones) != (uint32)NumBonesInUE)
		&& !(NumBonesInUsd == 0 && NumBonesInUE == 1 && BoneInfo[0].Name == TEXT("Root")))
	{
		return false;
	}

	TUsdStore<std::vector<double>> UsdJointTransformTimeSamples;
	AnimQuery.Get().GetJointTransformTimeSamples(&(UsdJointTransformTimeSamples.Get()));
	int32 NumJointTransformSamples = UsdJointTransformTimeSamples.Get().size();
	TOptional<double> FirstJointSampleTimeCode;
	TOptional<double> LastJointSampleTimeCode;
	if (UsdJointTransformTimeSamples.Get().size() > 0)
	{
		const std::vector<double>& JointTransformTimeSamples = UsdJointTransformTimeSamples.Get();
		FirstJointSampleTimeCode = JointTransformTimeSamples[0];
		LastJointSampleTimeCode = JointTransformTimeSamples[JointTransformTimeSamples.size() - 1];
	}

	TUsdStore<std::vector<double>> UsdBlendShapeTimeSamples;
	AnimQuery.Get().GetBlendShapeWeightTimeSamples(&(UsdBlendShapeTimeSamples.Get()));
	int32 NumBlendShapeSamples = UsdBlendShapeTimeSamples.Get().size();
	TOptional<double> FirstBlendShapeSampleTimeCode;
	TOptional<double> LastBlendShapeSampleTimeCode;
	if (UsdBlendShapeTimeSamples.Get().size() > 0)
	{
		const std::vector<double>& BlendShapeTimeSamples = UsdBlendShapeTimeSamples.Get();
		FirstBlendShapeSampleTimeCode = BlendShapeTimeSamples[0];
		LastBlendShapeSampleTimeCode = BlendShapeTimeSamples[BlendShapeTimeSamples.size() - 1];
	}

	TUsdStore<std::vector<double>> UsdRootMotionPrimTimeSamples;
	TOptional<double> FirstRootMotionTimeCode;
	TOptional<double> LastRootMotionTimeCode;
	TUsdStore<pxr::UsdGeomXformable> RootMotionXformable;
	{
		FScopedUsdAllocs UsdAllocs;

		// Note how we don't care whether the root motion is animated or not and will use RootMotionXformable
		// regardless, to have a similar effect in case its just a single non-animated transform
		RootMotionXformable = pxr::UsdGeomXformable{RootMotionPrim};
		if (RootMotionXformable.Get())
		{
			std::vector<double> UsdTimeSamples;
			if (RootMotionXformable.Get().GetTimeSamples(&UsdTimeSamples))
			{
				if (UsdTimeSamples.size() > 0)
				{
					FirstRootMotionTimeCode = UsdTimeSamples[0];
					LastRootMotionTimeCode = UsdTimeSamples[UsdTimeSamples.size() - 1];
				}
			}
		}
	}

	// Nothing to do: we don't actually have joints or blend shape time samples
	if (!FirstJointSampleTimeCode.IsSet() && !FirstBlendShapeSampleTimeCode.IsSet() && !FirstRootMotionTimeCode.IsSet())
	{
		return true;
	}

	// The animation should have a length in seconds according exclusively to its layer's timeCodesPerSecond, and that's it.
	// Here we intentionally scrape away any scalings due to the layer's offset and scale when referenced, and also reverse
	// the effect of the stage's timeCodesPerSecond.
	// USD's intent is for a layer's animation to have the same length in seconds when referenced by another layer, regardless
	// of it's timeCodesPerSeconds. To do that the SDK will intentionally compensate any difference in timeCodesPerSecond whenever
	// we query time samples, which we must compensate for here.
	// We do all of this because we want to bake this UAnimSequence without any offset/scaling effects, as if it was a standalone layer,
	// which is important because later our composition of tracks and subsections within a LevelSequence will reapply analogous
	// offsets and scalings anyway

	const double StartTimeCodeInStage = FMath::Min(
		FirstJointSampleTimeCode.Get(TNumericLimits<double>::Max()),
		FMath::Min(FirstBlendShapeSampleTimeCode.Get(TNumericLimits<double>::Max()), FirstRootMotionTimeCode.Get(TNumericLimits<double>::Max()))
	);
	const double EndTimeCodeInStage = FMath::Max(
		LastJointSampleTimeCode.Get(TNumericLimits<double>::Lowest()),
		FMath::Max(LastBlendShapeSampleTimeCode.Get(TNumericLimits<double>::Lowest()), LastRootMotionTimeCode.Get(TNumericLimits<double>::Lowest()))
	);
	const double StartSecondsInStage = StartTimeCodeInStage / StageTimeCodesPerSecond;
	const double SequenceLengthTimeCodesInStage = EndTimeCodeInStage - StartTimeCodeInStage;
	const double SequenceLengthTimeCodesInLayer = SequenceLengthTimeCodesInStage / Offset.Scale;
	const double SequenceLengthSecondsInLayer = FMath::Max<double>(SequenceLengthTimeCodesInLayer / LayerTimeCodesPerSecond, MINIMUM_ANIMATION_LENGTH);
	const double StartTimeCodeInLayer = (StartTimeCodeInStage - Offset.Offset) / Offset.Scale;
	const double StartSecondsInLayer = StartTimeCodeInLayer / LayerTimeCodesPerSecond;

	// Just bake each time code in the source layer as a frame
	const int32 NumBakedFrames = FMath::RoundToInt(FMath::Max(SequenceLengthSecondsInLayer * LayerTimeCodesPerSecond + 1.0, 1.0));
	const double StageBakeIntervalTimeCodes = 1.0 * Offset.Scale;

	IAnimationDataController& Controller = OutSkeletalAnimationAsset->GetController();

	// If we should transact, we'll already have a transaction from somewhere else. We should suppress this because
	// it will also create a transaction when importing into UE assets, and the level sequence assets can emit some warnings about it
	const bool bShouldTransact = false;
	Controller.OpenBracket(LOCTEXT("ImportUSDAnimData_Bracket", "Importing USD Animation Data"), bShouldTransact);
	Controller.InitializeModel();
	Controller.ResetModel(bShouldTransact);

	// Bake the animation for each frame.
	// An alternative route would be to convert the time samples into TransformCurves, add them to UAnimSequence::RawCurveData,
	// and then call UAnimSequence::BakeTrackCurvesToRawAnimation. Doing it this way provides a few benefits though: The main one is that the way with
	// which UAnimSequence bakes can lead to artifacts on problematic joints (e.g. 90 degree rotation joints children of -1 scale joints, etc.) as it
	// compounds the transformation with the rest pose. Another benefit is that that doing it this way lets us offload the interpolation to USD, so
	// that it can do it however it likes, and we can just sample the joints at the target framerate
	if (NumJointTransformSamples >= 2)
	{
		FScopedUsdAllocs Allocs;

		TArray<FRawAnimSequenceTrack> JointTracks;
		JointTracks.SetNum(NumBonesInUE);

		for (int32 BoneIndex = 0; BoneIndex < NumBonesInUE; ++BoneIndex)
		{
			FRawAnimSequenceTrack& JointTrack = JointTracks[BoneIndex];
			JointTrack.PosKeys.Reserve(NumBakedFrames);
			JointTrack.RotKeys.Reserve(NumBakedFrames);
			JointTrack.ScaleKeys.Reserve(NumBakedFrames);
		}

		FTransform RootMotionTransform;

		pxr::VtArray<pxr::GfMatrix4d> UsdJointTransforms;
		for (int32 FrameIndex = 0; FrameIndex < NumBakedFrames; ++FrameIndex)
		{
			const double StageFrameTimeCodes = StartTimeCodeInStage + FrameIndex * StageBakeIntervalTimeCodes;

			InUsdSkeletonQuery.ComputeJointLocalTransforms(&UsdJointTransforms, StageFrameTimeCodes);
			for (int32 BoneIndex = 0; BoneIndex < NumBonesInUE; ++BoneIndex)
			{
				// UsdJointTransforms will never have a transform value for our AdditionalBones that we manually added
				// (inserted "true root" bone), so we have to have this annoying check here to redirect the bone indices
				// properly when querying USD with them
				FTransform UEJointTransform = FTransform::Identity;
				if (NumAdditionalBones == 0 || BoneIndex != 0)
				{
					pxr::GfMatrix4d& UsdJointTransform = UsdJointTransforms[BoneIndex - NumAdditionalBones];
					UEJointTransform = UsdToUnreal::ConvertMatrix(StageInfo, UsdJointTransform);
				}

				// Concatenate the root bone transform with the transform track actually present on the skel root as a
				// whole
				if (BoneIndex == 0)
				{
					// We don't care about resetXformStack here: We'll always use the root motion prim's transform as
					// a local transformation anyway
					bool* OutResetTransformStack = nullptr;
					const bool bSuccess = UsdToUnreal::ConvertXformable(
						Stage.Get(),
						RootMotionXformable.Get(),
						RootMotionTransform,
						StageFrameTimeCodes,
						OutResetTransformStack
					);

					if (bSuccess)
					{
						UEJointTransform = UEJointTransform * RootMotionTransform;
					}
				}

				FRawAnimSequenceTrack& JointTrack = JointTracks[BoneIndex];
				JointTrack.PosKeys.Add(FVector3f(UEJointTransform.GetTranslation()));
				JointTrack.RotKeys.Add(FQuat4f(UEJointTransform.GetRotation()));
				JointTrack.ScaleKeys.Add(FVector3f(UEJointTransform.GetScale3D()));
			}
		}

		for (int32 BoneIndex = 0; BoneIndex < NumBonesInUE; ++BoneIndex)
		{
			Controller.AddBoneCurve(BoneInfo[BoneIndex].Name, bShouldTransact);
			Controller.SetBoneTrackKeys(
				BoneInfo[BoneIndex].Name,
				JointTracks[BoneIndex].PosKeys,
				JointTracks[BoneIndex].RotKeys,
				JointTracks[BoneIndex].ScaleKeys,
				bShouldTransact
			);
		}
	}

	// Add float tracks to animate morph target weights
	if (InBlendShapes && InSkinningTargets)
	{
		FScopedUsdAllocs Allocs;

		pxr::UsdSkelAnimQuery UsdAnimQuery = AnimQuery.Get();

		pxr::VtTokenArray SkelAnimChannelOrder = UsdAnimQuery.GetBlendShapeOrder();
		int32 NumSkelAnimChannels = SkelAnimChannelOrder.size();

		if (NumSkelAnimChannels > 0)
		{
			// Create a float curve for each blend shape channel. These will be copied for each blend shape that uses it
			// Don't remove redundant keys because if there are blendshapes with inbetweens that use this channel,
			// we want to make sure that we don't miss the frames where the curve would have reached the exact weight of a blend shape
			ERichCurveInterpMode CurveInterpMode = Stage.Get()->GetInterpolationType() == pxr::UsdInterpolationTypeHeld
													   ? ERichCurveInterpMode::RCIM_Constant
													   : ERichCurveInterpMode::RCIM_Linear;
			TArray<FRichCurve> SkelAnimChannelCurves;
			SkelAnimChannelCurves.SetNum(NumSkelAnimChannels);
			pxr::VtArray<float> WeightsForFrame;
			for (int32 FrameIndex = 0; FrameIndex < NumBakedFrames; ++FrameIndex)
			{
				const double StageFrameTimeCodes = StartTimeCodeInStage + FrameIndex * StageBakeIntervalTimeCodes;
				const double LayerFrameTimeCodes = (StageFrameTimeCodes - Offset.Offset) / Offset.Scale;
				const double LayerFrameSeconds = LayerFrameTimeCodes / LayerTimeCodesPerSecond - StartSecondsInLayer;

				UsdAnimQuery.ComputeBlendShapeWeights(&WeightsForFrame, pxr::UsdTimeCode(StageFrameTimeCodes));

				for (int32 SkelAnimChannelIndex = 0; SkelAnimChannelIndex < NumSkelAnimChannels; ++SkelAnimChannelIndex)
				{
					FRichCurve& Curve = SkelAnimChannelCurves[SkelAnimChannelIndex];

					FKeyHandle NewKeyHandle = Curve.AddKey(LayerFrameSeconds, WeightsForFrame[SkelAnimChannelIndex]);
					Curve.SetKeyInterpMode(NewKeyHandle, CurveInterpMode);
				}
			}

			UsdUtils::FCaseSensitiveStringSet ProcessedLODParentPaths;

			// Since we may need to switch variants to parse LODs, we could invalidate references to SkinningQuery objects, so we need
			// to keep track of these by path and construct one whenever we need them
			TArray<pxr::SdfPath> PathsToSkinnedPrims;
			for (const pxr::UsdSkelSkinningQuery& SkinningQuery : *InSkinningTargets)
			{
				// In USD, the skinning target need not be a mesh, but for Unreal we are only interested in skinning meshes
				if (pxr::UsdGeomMesh SkinningMesh = pxr::UsdGeomMesh(SkinningQuery.GetPrim()))
				{
					PathsToSkinnedPrims.Add(SkinningMesh.GetPrim().GetPath());
				}
			}

			TFunction<bool(const pxr::UsdGeomMesh&, int32)> CreateCurvesForLOD =
				[&InUsdSkeletonQuery, InBlendShapes, NumSkelAnimChannels, &SkelAnimChannelOrder, &SkelAnimChannelCurves, OutSkeletalAnimationAsset](
					const pxr::UsdGeomMesh& LODMesh,
					int32 LODIndex
				)
			{
				pxr::UsdSkelSkinningQuery SkinningQuery = UsdUtils::CreateSkinningQuery(LODMesh.GetPrim(), InUsdSkeletonQuery);
				if (!SkinningQuery)
				{
					return true;	// Continue trying other LODs
				}

				pxr::VtTokenArray MeshChannelOrder;
				if (!SkinningQuery.GetBlendShapeOrder(&MeshChannelOrder))
				{
					return true;
				}

				pxr::SdfPathVector BlendShapeTargets;
				const pxr::UsdRelationship& BlendShapeTargetsRel = SkinningQuery.GetBlendShapeTargetsRel();
				BlendShapeTargetsRel.GetTargets(&BlendShapeTargets);

				// USD will already show a warning if this happens, so let's just continue
				int32 NumMeshChannels = static_cast<int32>(MeshChannelOrder.size());
				if (NumMeshChannels != static_cast<int32>(BlendShapeTargets.size()))
				{
					return true;
				}

				pxr::SdfPath MeshPath = SkinningQuery.GetPrim().GetPath();
				for (int32 MeshChannelIndex = 0; MeshChannelIndex < NumMeshChannels; ++MeshChannelIndex)
				{
					FString PrimaryBlendShapePath = UsdToUnreal::ConvertPath(BlendShapeTargets[MeshChannelIndex].MakeAbsolutePath(MeshPath));

					if (const UsdUtils::FUsdBlendShape* FoundPrimaryBlendShape = InBlendShapes->Find(PrimaryBlendShapePath))
					{
						// Find a float curve for the primary blend shape
						FRichCurve* PrimaryBlendShapeCurve = nullptr;
						pxr::TfToken& MeshChannel = MeshChannelOrder[MeshChannelIndex];
						for (int32 SkelAnimChannelIndex = 0; SkelAnimChannelIndex < NumSkelAnimChannels; ++SkelAnimChannelIndex)
						{
							const pxr::TfToken& SkelAnimChannel = SkelAnimChannelOrder[SkelAnimChannelIndex];
							if (SkelAnimChannel == MeshChannel)
							{
								PrimaryBlendShapeCurve = &SkelAnimChannelCurves[SkelAnimChannelIndex];
								break;
							}
						}

						if (!PrimaryBlendShapeCurve)
						{
							USD_LOG_USERWARNING(FText::Format(
								LOCTEXT("NoChannelForPrimary", "Could not find a float channel to apply to primary blend shape '{0}'"),
								FText::FromString(PrimaryBlendShapePath)
							));
							continue;
						}

						// Primary blend shape has no inbetweens, so we can just use the skel anim channel curve directly
						if (FoundPrimaryBlendShape->Inbetweens.Num() == 0)
						{
							SkelDataConversionImpl::SetFloatCurveData(
								OutSkeletalAnimationAsset,
								*FoundPrimaryBlendShape->Name,
								*PrimaryBlendShapeCurve
							);
						}
						// Blend shape has inbetweens --> Need to map these to multiple float curves. This can be different for each mesh, so we need
						// to do it for each
						else
						{
							TArray<FRichCurve> RemappedBlendShapeCurves = SkelDataConversionImpl::ResolveWeightsForBlendShapeCurve(
								*FoundPrimaryBlendShape,
								*PrimaryBlendShapeCurve
							);
							if (RemappedBlendShapeCurves.Num() != FoundPrimaryBlendShape->Inbetweens.Num() + 1)
							{
								USD_LOG_USERWARNING(FText::Format(
									LOCTEXT("FailedToRemapInbetweens", "Failed to remap inbetween float curves for blend shape '{0}'"),
									FText::FromString(PrimaryBlendShapePath)
								));
								continue;
							}

							SkelDataConversionImpl::SetFloatCurveData(
								OutSkeletalAnimationAsset,
								*FoundPrimaryBlendShape->Name,
								RemappedBlendShapeCurves[0]
							);

							for (int32 InbetweenIndex = 0; InbetweenIndex < FoundPrimaryBlendShape->Inbetweens.Num(); ++InbetweenIndex)
							{
								const UsdUtils::FUsdBlendShapeInbetween& Inbetween = FoundPrimaryBlendShape->Inbetweens[InbetweenIndex];
								const FRichCurve& InbetweenCurve = RemappedBlendShapeCurves[InbetweenIndex + 1];	// Index 0 is the primary

								SkelDataConversionImpl::SetFloatCurveData(OutSkeletalAnimationAsset, *Inbetween.Name, InbetweenCurve);
							}
						}
					}
				}

				return true;
			};

			for (const pxr::SdfPath& SkinnedPrimPath : PathsToSkinnedPrims)
			{
				pxr::UsdPrim SkinnedPrim = Stage.Get()->GetPrimAtPath(SkinnedPrimPath);
				if (!SkinnedPrim)
				{
					continue;
				}

				pxr::UsdGeomMesh SkinnedMesh{SkinnedPrim};
				if (!SkinnedMesh)
				{
					continue;
				}

				pxr::UsdPrim ParentPrim = SkinnedMesh.GetPrim().GetParent();
				FString ParentPrimPath = UsdToUnreal::ConvertPath(ParentPrim.GetPath());

				bool bInterpretedLODs = false;
				if (bInInterpretLODs && ParentPrim && !ProcessedLODParentPaths.Contains(ParentPrimPath))
				{
					// At the moment we only consider a single mesh per variant, so if multiple meshes tell us to process the same parent prim, we
					// skip. This check would also prevent us from getting in here in case we just have many meshes children of a same prim, outside
					// of a variant. In this case they don't fit the "one mesh per variant" pattern anyway, and we want to fallback to ignoring LODs
					ProcessedLODParentPaths.Add(ParentPrimPath);

					// WARNING: After this is called, references to objects that were inside any of the LOD Meshes will be invalidated!
					bInterpretedLODs = UsdUtils::IterateLODMeshes(ParentPrim, CreateCurvesForLOD);
				}

				if (!bInterpretedLODs)
				{
					// Refresh reference to this prim as it could have been inside a variant that was temporarily switched by IterateLODMeshes
					CreateCurvesForLOD(SkinnedMesh, 0);
				}
			}
		}
	}

	OutSkeletalAnimationAsset->Interpolation = Stage.Get()->GetInterpolationType() == pxr::UsdInterpolationTypeHeld ? EAnimInterpolationType::Step
																													: EAnimInterpolationType::Linear;
	OutSkeletalAnimationAsset->ImportFileFramerate = LayerTimeCodesPerSecond;
	OutSkeletalAnimationAsset->ImportResampleFramerate = LayerTimeCodesPerSecond;

	const FFrameRate FrameRate(LayerTimeCodesPerSecond, 1);
	Controller.SetFrameRate(FrameRate, bShouldTransact);
	const FFrameNumber FrameNumber = FrameRate.AsFrameNumber(SequenceLengthSecondsInLayer);
	Controller.SetNumberOfFrames(FrameNumber, bShouldTransact);
	Controller.NotifyPopulated();	 // This call is important to get the controller to not use the sampling frequency as framerate
	Controller.CloseBracket(bShouldTransact);

	OutSkeletalAnimationAsset->PostEditChange();
	OutSkeletalAnimationAsset->MarkPackageDirty();

	if (OutStartOffsetSeconds)
	{
		*OutStartOffsetSeconds = StartSecondsInStage;
	}

	if (OutScaleSeconds)
	{
		*OutScaleSeconds = Offset.Scale;
	}

	return true;
}

bool UsdToUnreal::ConvertBlendShape(
	const pxr::UsdSkelBlendShape& UsdBlendShape,
	const FUsdStageInfo& StageInfo,
	uint32 PointIndexOffset,
	TSet<FString>& UsedMorphTargetNames,
	UsdUtils::FBlendShapeMap& OutBlendShapes,
	const FUsdMeshConversionOptions& Options
)
{
	return ConvertBlendShape(UsdBlendShape, StageInfo, 0, PointIndexOffset, UsedMorphTargetNames, OutBlendShapes, Options);
}

bool UsdToUnreal::ConvertBlendShape(
	const pxr::UsdSkelBlendShape& UsdBlendShape,
	const FUsdStageInfo& StageInfo,
	int32 LODIndex,
	uint32 PointIndexOffset,
	TSet<FString>& UsedMorphTargetNames,
	UsdUtils::FBlendShapeMap& OutBlendShapes,
	const UsdToUnreal::FUsdMeshConversionOptions& Options,
	const pxr::GfMatrix4d* GeomBindTransform
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UsdToUnreal::ConvertBlendShape);

	if (!UsdBlendShape)
	{
		return false;
	}

	FScopedUsdAllocs Allocs;

	pxr::UsdAttribute OffsetsAttr = UsdBlendShape.GetOffsetsAttr();
	pxr::VtArray<pxr::GfVec3f> Offsets;
	OffsetsAttr.Get(&Offsets);

	pxr::UsdAttribute IndicesAttr = UsdBlendShape.GetPointIndicesAttr();
	pxr::VtArray<int> PointIndices;
	IndicesAttr.Get(&PointIndices);

	pxr::UsdAttribute NormalsAttr = UsdBlendShape.GetNormalOffsetsAttr();
	pxr::VtArray<pxr::GfVec3f> Normals;
	NormalsAttr.Get(&Normals);

	// We need to guarantee blend shapes have unique names because these will be used as UMorphTarget names
	// Note that we can't just use the prim path here and need an index to guarantee uniqueness,
	// because although the path is usually unique, USD has case sensitive paths and the FNames of the
	// UMorphTargets are case insensitive
	FString PrimaryName = UsdUnreal::ObjectUtils::GetUniqueName(
		UsdUnreal::ObjectUtils::SanitizeObjectName(UsdToUnreal::ConvertString(UsdBlendShape.GetPrim().GetName())),
		UsedMorphTargetNames
	);
	FString PrimaryPath = UsdToUnreal::ConvertPath(UsdBlendShape.GetPrim().GetPath());
	if (UsdUtils::FUsdBlendShape* ExistingBlendShape = OutBlendShapes.Find(PrimaryPath))
	{
		ExistingBlendShape->LODIndicesThatUseThis.Add(LODIndex);
		return true;
	}

	UsdUtils::FUsdBlendShape PrimaryBlendShape;
	if (!SkelDataConversionImpl::CreateUsdBlendShape(
			PrimaryName,
			Offsets,
			Normals,
			PointIndices,
			StageInfo,
			GeomBindTransform,
			PointIndexOffset,
			LODIndex,
			PrimaryBlendShape,
			Options
		))
	{
		return false;
	}
	UsedMorphTargetNames.Add(PrimaryBlendShape.Name);

	UsdUtils::FBlendShapeMap InbetweenBlendShapes;
	for (const pxr::UsdSkelInbetweenShape& Inbetween : UsdBlendShape.GetInbetweens())
	{
		if (!Inbetween)
		{
			continue;
		}

		float Weight = 0.0f;
		if (!Inbetween.GetWeight(&Weight))
		{
			continue;
		}

		FString OrigInbetweenName = UsdToUnreal::ConvertString(Inbetween.GetAttr().GetName());
		FString InbetweenPath = FString::Printf(TEXT("%s_%s"), *PrimaryPath, *OrigInbetweenName);
		FString InbetweenName = UsdUnreal::ObjectUtils::GetUniqueName(
			UsdUnreal::ObjectUtils::SanitizeObjectName(FPaths::GetCleanFilename(InbetweenPath)),
			UsedMorphTargetNames
		);

		if (Weight > 1.0f || Weight < 0.0f || FMath::IsNearlyZero(Weight) || FMath::IsNearlyEqual(Weight, 1.0f))
		{
			continue;
		}

		pxr::VtArray<pxr::GfVec3f> InbetweenPointsOffsets;
		pxr::VtArray<pxr::GfVec3f> InbetweenNormalOffsets;

		Inbetween.GetOffsets(&InbetweenPointsOffsets);
		Inbetween.GetNormalOffsets(&InbetweenNormalOffsets);

		// Create separate blend shape for the inbetween
		// Now how the inbetween always shares the same point indices as the parent
		UsdUtils::FUsdBlendShape InbetweenShape;
		if (!SkelDataConversionImpl::CreateUsdBlendShape(
				InbetweenName,
				InbetweenPointsOffsets,
				InbetweenNormalOffsets,
				PointIndices,
				StageInfo,
				GeomBindTransform,
				PointIndexOffset,
				LODIndex,
				InbetweenShape,
				Options
			))
		{
			continue;
		}
		UsedMorphTargetNames.Add(InbetweenShape.Name);
		InbetweenBlendShapes.Add(InbetweenPath, InbetweenShape);

		// Keep track of it in the PrimaryBlendShape so we can resolve weights later
		UsdUtils::FUsdBlendShapeInbetween& ConvertedInbetween = PrimaryBlendShape.Inbetweens.Emplace_GetRef();
		ConvertedInbetween.Name = InbetweenShape.Name;
		ConvertedInbetween.InbetweenWeight = Weight;
	}

	// Sort according to weight so they're easier to resolve later
	PrimaryBlendShape.Inbetweens.Sort(
		[](const UsdUtils::FUsdBlendShapeInbetween& Lhs, const UsdUtils::FUsdBlendShapeInbetween& Rhs)
		{
			return Lhs.InbetweenWeight < Rhs.InbetweenWeight;
		}
	);

	OutBlendShapes.Add(PrimaryPath, PrimaryBlendShape);
	OutBlendShapes.Append(MoveTemp(InbetweenBlendShapes));

	return true;
}

bool UsdToUnreal::ConvertSkeletalImportData(
	TArray<FSkeletalMeshImportData>& InLODIndexToSkeletalMeshImportData,
	const TArray<SkeletalMeshImportData::FBone>& InSkeletonBones,
	UsdUtils::FBlendShapeMap& InBlendShapesByPath,
	USkeletalMesh* InOutSkeletalMesh
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UsdToUnreal::ConvertSkeletalImportData);

	if (!InOutSkeletalMesh || InLODIndexToSkeletalMeshImportData.Num() == 0)
	{
		return false;
	}

	USkeleton* Skeleton = InOutSkeletalMesh->GetSkeleton();
	if (!Skeleton)
	{
		return false;
	}

	// Process reference skeleton from import data
	int32 SkeletalDepth = 0;
	FSkeletalMeshImportData DummyData;
	DummyData.RefBonesBinary = InSkeletonBones;
	if (!SkeletalMeshImportUtils::ProcessImportMeshSkeleton(
			InOutSkeletalMesh->GetSkeleton(),
			InOutSkeletalMesh->GetRefSkeleton(),
			SkeletalDepth,
			DummyData
		))
	{
		return false;
	}

	// This prevents PostEditChange calls when it is alive, also ensuring it is called once when we return from this function.
	// This is required because we must ensure the morphtargets are in the SkeletalMesh before the first call to PostEditChange(),
	// or else they will be effectively discarded
	FScopedSkeletalMeshPostEditChange ScopedPostEditChange(InOutSkeletalMesh);
	InOutSkeletalMesh->PreEditChange(nullptr);

	// Create initial bounding box based on expanded version of reference pose for meshes without physics assets
	const FSkeletalMeshImportData& LowestLOD = InLODIndexToSkeletalMeshImportData[0];
	FBox3f BoundingBox(LowestLOD.Points.GetData(), LowestLOD.Points.Num());
	FBox3f Temp = BoundingBox;
	FVector3f MidMesh = 0.5f * (Temp.Min + Temp.Max);
	BoundingBox.Min = Temp.Min + 1.0f * (Temp.Min - MidMesh);
	BoundingBox.Max = Temp.Max + 1.0f * (Temp.Max - MidMesh);
	BoundingBox.Min[2] = Temp.Min[2] + 0.1f * (Temp.Min[2] - MidMesh[2]);
	const FVector3f BoundingBoxSize = BoundingBox.GetSize();
	if (LowestLOD.Points.Num() > 2 && BoundingBoxSize.X < THRESH_POINTS_ARE_SAME && BoundingBoxSize.Y < THRESH_POINTS_ARE_SAME
		&& BoundingBoxSize.Z < THRESH_POINTS_ARE_SAME)
	{
		return false;
	}

#if WITH_EDITOR
	IMeshUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshUtilities>("MeshUtilities");
#endif	  // WITH_EDITOR

	FSkeletalMeshModel* ImportedResource = InOutSkeletalMesh->GetImportedModel();
	ImportedResource->LODModels.Empty();
	InOutSkeletalMesh->SetNumSourceModels(0);
	bool bHasVertexColors = false;
	for (int32 LODIndex = 0; LODIndex < InLODIndexToSkeletalMeshImportData.Num(); ++LODIndex)
	{
		FSkeletalMeshImportData& LODImportData = InLODIndexToSkeletalMeshImportData[LODIndex];

		// In the future it will be expected for bone data to be inside FSkeletalMeshImportData as well so we should
		// probably do this
		LODImportData.RefBonesBinary = InSkeletonBones;

		ImportedResource->LODModels.Add(new FSkeletalMeshLODModel());
		FSkeletalMeshLODModel& LODModel = ImportedResource->LODModels.Last();

		// Process bones influence (normalization and optimization) (optional)
		SkeletalMeshImportUtils::ProcessImportMeshInfluences(LODImportData, InOutSkeletalMesh->GetPathName());

		FSkeletalMeshLODInfo& NewLODInfo = InOutSkeletalMesh->AddLODInfo();
		NewLODInfo.ReductionSettings.NumOfTrianglesPercentage = 1.0f;
		NewLODInfo.ReductionSettings.NumOfVertPercentage = 1.0f;
		NewLODInfo.ReductionSettings.MaxDeviationPercentage = 0.0f;
		NewLODInfo.LODHysteresis = 0.02f;

		bHasVertexColors |= LODImportData.bHasVertexColors;

		LODModel.NumTexCoords = FMath::Max<uint32>(1, LODImportData.NumTexCoords);

		// Data needed by BuildSkeletalMesh
		LODImportData.PointToRawMap.AddUninitialized(LODImportData.Points.Num());
		for (int32 PointIndex = 0; PointIndex < LODImportData.Points.Num(); ++PointIndex)
		{
			LODImportData.PointToRawMap[PointIndex] = PointIndex;
		}

		TArray<FVector3f> LODPoints;
		TArray<SkeletalMeshImportData::FMeshWedge> LODWedges;
		TArray<SkeletalMeshImportData::FMeshFace> LODFaces;
		TArray<SkeletalMeshImportData::FVertInfluence> LODInfluences;
		TArray<int32> LODPointToRawMap;
		LODImportData.CopyLODImportData(LODPoints, LODWedges, LODFaces, LODInfluences, LODPointToRawMap);

#if WITH_EDITOR
		IMeshUtilities::MeshBuildOptions BuildOptions;
		BuildOptions.TargetPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
		// #ueent_todo: Normals and tangents shouldn't need to be recomputed when they are retrieved from USD
		// BuildOptions.bComputeNormals = !LODImportData.bHasNormals;
		// BuildOptions.bComputeTangents = !LODImportData.bHasTangents;
		BuildOptions.bUseMikkTSpace = true;

		TArray<FText> WarningMessages;
		TArray<FName> WarningNames;

		bool bBuildSuccess = MeshUtilities.BuildSkeletalMesh(
			LODModel,
			InOutSkeletalMesh->GetPathName(),
			InOutSkeletalMesh->GetRefSkeleton(),
			LODInfluences,
			LODWedges,
			LODFaces,
			LODPoints,
			LODPointToRawMap,
			BuildOptions,
			&WarningMessages,
			&WarningNames
		);

		for (int32 WarningIndex = 0; WarningIndex < FMath::Max(WarningMessages.Num(), WarningNames.Num()); ++WarningIndex)
		{
			const FText& Text = WarningMessages.IsValidIndex(WarningIndex) ? WarningMessages[WarningIndex] : FText::GetEmpty();
			const FName& Name = WarningNames.IsValidIndex(WarningIndex) ? WarningNames[WarningIndex] : NAME_None;

			if (bBuildSuccess)
			{
				USD_LOG_WARNING(TEXT("Warning when trying to build skeletal mesh from USD: '%s': '%s'"), *Name.ToString(), *Text.ToString());
			}
			else
			{
				USD_LOG_ERROR(TEXT("Error when trying to build skeletal mesh from USD: '%s': '%s'"), *Name.ToString(), *Text.ToString());
			}
		}

		if (!bBuildSuccess)
		{
			return false;
		}

		// UMorphTarget::PopulateDeltas called by BuildMorphTargetsInternal will ignore deltas below
		// MorphThresholdPosition, so we must have something larger than that
		const FVector3f SmallMorphDelta{0.0f, 0.0f, FMath::Max(BuildOptions.OverlappingThresholds.MorphThresholdPosition * 1.1f, 1e-4f)};

		// Morph target data is now primarily provided via the MeshDescription. For now we still don't convert
		// skeletal data directly to the MeshDescription, so we must feed it into FSkeletalMeshImportData, so that
		// SaveLODImportedData converts it into the skeletal MeshDescription for us
		//
		// Reference: FSkeletalMeshImportData::AddMorphTarget, but we don't use it directly as matching the interface
		// would involve copying our BlendShape.Vertices into a new FMorphTargetLODModel.
		//
		// TODO: Add in the morph target normal data (from FMorphTargetDelta::TangentZDelta) to the import data
		// at the right location when FSkeletalMeshImportData::GetMeshDescription starts reading normal data, or whenever
		// we start converting skeletal data into MeshDescriptions
		LODImportData.MorphTargets.Reserve(InBlendShapesByPath.Num());
		LODImportData.MorphTargetNames.Reserve(InBlendShapesByPath.Num());
		LODImportData.MorphTargetModifiedPoints.Reserve(InBlendShapesByPath.Num());
		for (const TPair<FString, UsdUtils::FUsdBlendShape>& Pair : InBlendShapesByPath)
		{
			const UsdUtils::FUsdBlendShape& BlendShape = Pair.Value;

			// The morph targets used for higher LOD levels must be a subset of the morph targets used for
			// lower LODS, or else the skeletal mesh build will just discard them. So even if this blend
			// shape doesn't affect this LOD, just add it anyway
			LODImportData.MorphTargetNames.Add(BlendShape.Name);
			FSkeletalMeshImportData& MorphTarget = LODImportData.MorphTargets.Emplace_GetRef();
			TSet<uint32>& NewModifiedPoints = LODImportData.MorphTargetModifiedPoints.Emplace_GetRef();

			if (!BlendShape.LODIndicesThatUseThis.Contains(LODIndex))
			{
				if (LODImportData.Points.Num() > 0)
				{
					// Additionally, in order to keep a morph target that in USD is only defined for higher LODS, we must
					// add a tiny valid delta when handling it for lower LODs, or else the skeletal mesh build and
					// UMorphTarget::PopulateDeltas will get rid of the morph target during the processing of the lower LODS,
					// and then ignore it when later processing the higher LODs, as the morph target was removed...
					MorphTarget.Points.Add(LODImportData.Points[0] + SmallMorphDelta);
					NewModifiedPoints.Add(0);
				}

				continue;
			}

			MorphTarget.Points.Reserve(LODImportData.Points.Num());
			NewModifiedPoints.Reserve(BlendShape.Vertices.Num());

			for (const FMorphTargetDelta& Delta : BlendShape.Vertices)
			{
				if (!LODImportData.Points.IsValidIndex(Delta.SourceIdx))
				{
					continue;
				}

				NewModifiedPoints.Add(Delta.SourceIdx);
				MorphTarget.Points.Add(Delta.PositionDelta + LODImportData.Points[Delta.SourceIdx]);
			}
		}

		// This is important because it will fill in the LODModel's RawSkeletalMeshBulkDataID,
		// which is the part of the skeletal mesh's DDC key that is affected by the actual mesh data
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		InOutSkeletalMesh->SaveLODImportedData(LODIndex, LODImportData);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif	  // WITH_EDITOR
	}

	InOutSkeletalMesh->SetImportedBounds(FBoxSphereBounds((FBox)BoundingBox));
	InOutSkeletalMesh->SetHasVertexColors(bHasVertexColors);
	InOutSkeletalMesh->SetVertexColorGuid(InOutSkeletalMesh->GetHasVertexColors() ? FGuid::NewGuid() : FGuid());
	InOutSkeletalMesh->CalculateInvRefMatrices();

	Skeleton->MergeAllBonesToBoneTree(InOutSkeletalMesh);
	if (InOutSkeletalMesh->GetRefSkeleton().GetRawBoneNum() == 0)
	{
		return false;
	}

	// "Declare" the morph target curves on the skeleton or skeletal mesh according to bAddCurveMetadataToSkeleton.
	// This is important otherwise the ControlRig will not hoist these curves as controls when using e.g. FKControlRig.
	for (const TPair<FString, UsdUtils::FUsdBlendShape>& BlendShapeByPath : InBlendShapesByPath)
	{
		const UsdUtils::FUsdBlendShape& BlendShape = BlendShapeByPath.Value;
		const FName CurveName = *BlendShape.Name;

		if (bAddCurveMetadataToSkeleton)
		{
			const bool bMaterialCurve = false;
			const bool bMorphTargetCurve = true;
			Skeleton->AccumulateCurveMetaData(CurveName, bMaterialCurve, bMorphTargetCurve);

			// Ensure we have a morph flag set
			FCurveMetaData* CurveMetaData = Skeleton->GetCurveMetaData(CurveName);
			CurveMetaData->Type.bMorphtarget = true;
		}
		else
		{
			UAnimCurveMetaData* AnimCurveMetaData = InOutSkeletalMesh->GetAssetUserData<UAnimCurveMetaData>();
			if (AnimCurveMetaData == nullptr)
			{
				AnimCurveMetaData = NewObject<UAnimCurveMetaData>(InOutSkeletalMesh, NAME_None, RF_Transactional);
				InOutSkeletalMesh->AddAssetUserData(AnimCurveMetaData);
			}

			AnimCurveMetaData->AddCurveMetaData(CurveName);

			// Ensure we have a morph flag set
			if (FCurveMetaData* CurveMetaData = AnimCurveMetaData->GetCurveMetaData(CurveName))
			{
				CurveMetaData->Type.bMorphtarget = true;
			}
		}
	}

	return true;
}

#endif	  // #if USE_USD_SDK && WITH_EDITOR

void UsdUtils::ResolveWeightsForBlendShape(
	const UsdUtils::FUsdBlendShape& InBlendShape,
	float InWeight,
	float& OutMainWeight,
	TArray<float>& OutInbetweenWeights
)
{
	OutMainWeight = 0.0f;
	int32 NumInbetweens = InBlendShape.Inbetweens.Num();
	if (NumInbetweens == 0)
	{
		OutMainWeight = InWeight;
		return;
	}

	OutInbetweenWeights.SetNumUninitialized(NumInbetweens);
	for (float& OutInbetweenWeight : OutInbetweenWeights)
	{
		OutInbetweenWeight = 0.0f;
	}

	if (FMath::IsNearlyEqual(InWeight, 0.0f))
	{
		OutMainWeight = 0.0f;
		return;
	}
	else if (FMath::IsNearlyEqual(InWeight, 1.0f))
	{
		OutMainWeight = 1.0f;
		return;
	}

	// Note how we don't care if UpperIndex/LowerIndex are beyond the bounds of the array here,
	// as that signals when we're above/below all inbetweens
	int32 UpperIndex = Algo::UpperBoundBy(
		InBlendShape.Inbetweens,
		InWeight,
		[](const UsdUtils::FUsdBlendShapeInbetween& Inbetween)
		{
			return Inbetween.InbetweenWeight;
		}
	);
	int32 LowerIndex = UpperIndex - 1;

	float UpperWeight = 1.0f;
	if (UpperIndex <= NumInbetweens - 1)
	{
		UpperWeight = InBlendShape.Inbetweens[UpperIndex].InbetweenWeight;
	}

	float LowerWeight = 0.0f;
	if (LowerIndex >= 0)
	{
		LowerWeight = InBlendShape.Inbetweens[LowerIndex].InbetweenWeight;
	}

	UpperWeight = (InWeight - LowerWeight) / (UpperWeight - LowerWeight);
	LowerWeight = (1.0f - UpperWeight);

	// We're between upper inbetween and the 1.0 weight
	if (UpperIndex > NumInbetweens - 1)
	{
		OutMainWeight = UpperWeight;
		OutInbetweenWeights[NumInbetweens - 1] = LowerWeight;
	}
	// We're between 0.0 and the first inbetween weight
	else if (LowerIndex < 0)
	{
		OutMainWeight = 0;
		OutInbetweenWeights[0] = UpperWeight;
	}
	// We're between two inbetweens
	else
	{
		OutInbetweenWeights[UpperIndex] = UpperWeight;
		OutInbetweenWeights[LowerIndex] = LowerWeight;
	}
}

// Returns bone-space joint transforms from the SkeletalMeshComponent while paying attention to whether
// it has a LeaderPoseComponent or not.
//
// References:
// - FMLDeformerEditorToolkit::GetDebugActorComponentSpaceTransforms
// - FAnimationRecorder::GetBoneTransforms
void UsdUtils::GetBoneTransforms(USkeletalMeshComponent* Component, TArray<FTransform>& BoneTransforms)
{
	if (!Component)
	{
		return;
	}

	int32 NumBones = INDEX_NONE;
	if (USkeletalMesh* Mesh = Component->GetSkeletalMeshAsset())
	{
		const FReferenceSkeleton& RefSkel = Mesh->GetRefSkeleton();
		NumBones = RefSkel.GetNum();
	}
	if (NumBones == INDEX_NONE)
	{
		return;
	}

	if (USkeletalMeshComponent* Leader = Cast<USkeletalMeshComponent>(Component->LeaderPoseComponent.Get()))
	{
		const TArray<FTransform>& LeaderTransforms = Leader->GetBoneSpaceTransforms();
		const TArray<FTransform>& FollowerTransforms = Component->GetBoneSpaceTransforms();

		const TArray<int32>& BoneMap = Component->GetLeaderBoneMap();

		BoneTransforms.SetNumUninitialized(NumBones);
		for (int32 BoneIndex = 0; BoneIndex < NumBones; BoneIndex++)
		{
			if (BoneMap.IsValidIndex(BoneIndex) && LeaderTransforms.IsValidIndex(BoneMap[BoneIndex]))
			{
				BoneTransforms[BoneIndex] = LeaderTransforms[BoneMap[BoneIndex]];
			}
			else if (FollowerTransforms.IsValidIndex(BoneIndex))
			{
				BoneTransforms[BoneIndex] = FollowerTransforms[BoneIndex];
			}
		}
	}
	else
	{
		BoneTransforms = Component->GetBoneSpaceTransforms();
	}
}

void UsdUtils::RefreshSkeletalMeshComponent(USkeletalMeshComponent& Component)
{
	// This whole incantation is required or else the component will really not update until the next frame.
	// Note: This will also cause the update of morph target weights.
	Component.TickAnimation(0.f, false);
	Component.UpdateLODStatus();
	Component.RefreshBoneTransforms();
	Component.RefreshFollowerComponents();
	Component.UpdateComponentToWorld();
	Component.FinalizeBoneTransform();
	Component.MarkRenderTransformDirty();
	Component.MarkRenderDynamicDataDirty();
}

#if USE_USD_SDK

// Adapted from UsdSkel_CacheImpl::ReadScope::_FindOrCreateSkinningQuery because we need to manually create these on UsdGeomMeshes we already have
UE::FUsdSkelSkinningQuery UsdUtils::CreateSkinningQuery(const pxr::UsdPrim& SkinnedMeshPrim, const pxr::UsdSkelSkeletonQuery& SkeletonQuery)
{
	pxr::UsdSkelBindingAPI SkelBindingAPI{SkinnedMeshPrim};
	const pxr::UsdSkelAnimQuery& AnimQuery = SkeletonQuery.GetAnimQuery();
	if (!SkelBindingAPI)
	{
		return {};
	}

	return UE::FUsdSkelSkinningQuery{pxr::UsdSkelSkinningQuery(
		SkinnedMeshPrim,
		SkeletonQuery ? SkeletonQuery.GetJointOrder() : pxr::VtTokenArray(),
		AnimQuery ? AnimQuery.GetBlendShapeOrder() : pxr::VtTokenArray(),
		SkelBindingAPI.GetJointIndicesAttr(),
		SkelBindingAPI.GetJointWeightsAttr(),
		SkelBindingAPI.GetSkinningMethodAttr(),
		SkelBindingAPI.GetGeomBindTransformAttr(),
		SkelBindingAPI.GetJointsAttr(),
		SkelBindingAPI.GetBlendShapesAttr(),
		SkelBindingAPI.GetBlendShapeTargetsRel()
	)};
}

void UsdUtils::BindAnimationSource(pxr::UsdPrim& Prim, const pxr::UsdPrim& AnimationSource)
{
	FScopedUsdAllocs UsdAllocs;

	pxr::UsdSkelBindingAPI SkelBindingAPI = pxr::UsdSkelBindingAPI::Apply(Prim);
	pxr::UsdRelationship AnimSourceRel = SkelBindingAPI.CreateAnimationSourceRel();
	if (AnimationSource)
	{
		AnimSourceRel.SetTargets(pxr::SdfPathVector({AnimationSource.GetPath()}));
	}
	else
	{
		const bool bRemoveSpec = false;
		AnimSourceRel.ClearTargets(bRemoveSpec);
	}
}

UE::FUsdPrim UsdUtils::FindFirstAnimationSource(const UE::FUsdPrim& SkelRootPrim)
{
	if (!SkelRootPrim)
	{
		return {};
	}

	FScopedUsdAllocs UsdAllocs;

	// For now we really only parse the first skeletal binding of a SkelRoot (check USDSkelSkeletonTranslator.cpp,
	// LoadAllSkeletalData) and its SkelAnimation, if any.
	// Note that we don't check the SkelRoot prim directly for the SkelAnimation binding: If it has a valid one
	// it will propagate down to child namespaces and affect our first skeletal binding anyway

	if (pxr::UsdSkelRoot SkeletonRoot{pxr::UsdPrim{SkelRootPrim}})
	{
		std::vector<pxr::UsdSkelBinding> SkeletonBindings;

		pxr::UsdSkelCache SkeletonCache;
		SkeletonCache.Populate(SkeletonRoot, pxr::UsdTraverseInstanceProxies());
		SkeletonCache.ComputeSkelBindings(SkeletonRoot, &SkeletonBindings, pxr::UsdTraverseInstanceProxies());

		for (const pxr::UsdSkelBinding& Binding : SkeletonBindings)
		{
			const pxr::UsdSkelSkeleton& Skeleton = Binding.GetSkeleton();
			pxr::UsdSkelSkeletonQuery SkelQuery = SkeletonCache.GetSkelQuery(Skeleton);
			pxr::UsdSkelAnimQuery AnimQuery = SkelQuery.GetAnimQuery();
			if (!AnimQuery)
			{
				continue;
			}

			return UE::FUsdPrim{AnimQuery.GetPrim()};
		}
	}

	return {};
}

UE::FUsdPrim UsdUtils::FindAnimationSource(const pxr::UsdPrim& SkelRootPrim, const pxr::UsdPrim& SkeletonPrim)
{
	if (!SkeletonPrim)
	{
		return {};
	}

	FScopedUsdAllocs UsdAllocs;

	pxr::UsdSkelSkeleton Skeleton{SkeletonPrim};
	pxr::UsdSkelRoot ClosestParentSkelRoot{SkelRootPrim};
	if (Skeleton && ClosestParentSkelRoot)
	{
		pxr::UsdSkelCache SkeletonCache;
		SkeletonCache.Populate(ClosestParentSkelRoot, pxr::UsdTraverseInstanceProxies());

		pxr::UsdSkelSkeletonQuery SkelQuery = SkeletonCache.GetSkelQuery(Skeleton);
		if (pxr::UsdSkelAnimQuery AnimQuery = SkelQuery.GetAnimQuery())
		{
			return UE::FUsdPrim{AnimQuery.GetPrim()};
		}
	}

	return {};
}

UE::FUsdPrim UsdUtils::GetClosestParentSkelRoot(const pxr::UsdPrim& SomePrim)
{
	FScopedUsdAllocs Allocs;

	pxr::UsdPrim Parent = SomePrim;
	while (Parent && !Parent.IsPseudoRoot())
	{
		if (Parent.IsA<pxr::UsdSkelRoot>())
		{
			return UE::FUsdPrim{Parent};
		}

		Parent = Parent.GetParent();
	}

	return {};
}

bool UsdUtils::GetSkelQueries(
	const pxr::UsdSkelRoot& InSkelRootPrim,
	const pxr::UsdSkelSkeleton& InSkeletonPrim,
	pxr::UsdSkelBinding& OutSkelBinding,
	pxr::UsdSkelSkeletonQuery& OutSkeletonQuery,
	pxr::UsdSkelCache* InOutSkelCache
)
{
	if (!InSkelRootPrim || !InSkeletonPrim)
	{
		return false;
	}

	FScopedUsdAllocs Allocs;

	TOptional<pxr::UsdSkelCache> TempCache;
	if (!InOutSkelCache)
	{
		TempCache.Emplace();
		InOutSkelCache = &TempCache.GetValue();
		InOutSkelCache->Populate(InSkelRootPrim, pxr::UsdTraverseInstanceProxies());
	}

	OutSkeletonQuery = InOutSkelCache->GetSkelQuery(InSkeletonPrim);

	return InOutSkelCache->ComputeSkelBinding(InSkelRootPrim, InSkeletonPrim, &OutSkelBinding, pxr::UsdTraverseInstanceProxies());
}

bool UsdUtils::ApplyBlendShape(
	FMeshDescription& InOutMeshDescription,
	const pxr::UsdPrim& InBlendShapePrim,
	const FTransform& AdditionalTransform,
	float Weight,
	const FString InInbetweenName
)
{
	const static FMatrix& GeomBindTransform = FMatrix::Identity;

	return ApplyBlendShape(
		InOutMeshDescription,
		InBlendShapePrim,
		GeomBindTransform,
		AdditionalTransform,
		Weight,
		InInbetweenName
	);
}

bool UsdUtils::ApplyBlendShape(
	FMeshDescription& InOutMeshDescription,
	const pxr::UsdPrim& InBlendShapePrim,
	const FMatrix& GeomBindTransform,
	const FTransform& AdditionalTransform,
	float Weight,
	const FString InInbetweenName
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UsdToUnreal::ConvertBlendShape);

	FScopedUsdAllocs Allocs;

	pxr::UsdSkelBlendShape UsdBlendShape{InBlendShapePrim};
	if (!UsdBlendShape)
	{
		return false;
	}

	pxr::UsdStageRefPtr Stage = InBlendShapePrim.GetStage();
	if (!Stage)
	{
		return false;
	}

	FUsdStageInfo StageInfo{Stage};

	// Collect blend shape deltas
	pxr::VtArray<pxr::GfVec3f> PositionOffsets;
	pxr::VtArray<pxr::GfVec3f> NormalOffsets;
	pxr::VtArray<int> PointIndices;
	{
		pxr::UsdAttribute IndicesAttr = UsdBlendShape.GetPointIndicesAttr();
		IndicesAttr.Get(&PointIndices);

		if (!InInbetweenName.IsEmpty())
		{
			if (pxr::UsdSkelInbetweenShape Inbetween = UsdBlendShape.GetInbetween(UnrealToUsd::ConvertToken(*InInbetweenName).Get()))
			{
				Inbetween.GetOffsets(&PositionOffsets);
				Inbetween.GetNormalOffsets(&NormalOffsets);
			}
			else
			{
				USD_LOG_WARNING(
					TEXT("Failed to find inbetween '%s' when applying blend shape prim '%s' to a mesh description"),
					*InInbetweenName,
					*UsdToUnreal::ConvertPath(InBlendShapePrim.GetPrimPath())
				);
			}
		}
		else
		{
			if (pxr::UsdAttribute OffsetsAttr = UsdBlendShape.GetOffsetsAttr())
			{
				OffsetsAttr.Get(&PositionOffsets);
			}

			if (pxr::UsdAttribute NormalsAttr = UsdBlendShape.GetNormalOffsetsAttr())
			{
				NormalsAttr.Get(&NormalOffsets);
			}
		}
	}

	// Apply GeomBindTransform if we have one
	if (GeomBindTransform != FMatrix::Identity)
	{
		pxr::GfMatrix4d UsdGeomBindTransform = UnrealToUsd::ConvertMatrix(GeomBindTransform);
		pxr::GfMatrix4d InvTransposeGeomBindTransform;
		if (UsdGeomBindTransform.GetDeterminant() == 0.0)
		{
			// Can't invert, just use as-is
			USD_LOG_WARNING(
				TEXT("Failed to invert geomBindTransform for blend shape prim '%s'"),
				*UsdToUnreal::ConvertPath(InBlendShapePrim.GetPrimPath())
			);
			InvTransposeGeomBindTransform = UsdGeomBindTransform;
		}
		else
		{
			InvTransposeGeomBindTransform = UsdGeomBindTransform.GetInverse().GetTranspose();
		}

		for (pxr::GfVec3f& Position : PositionOffsets)
		{
			// Note: TransformDir here because even the position deltas are still *deltas* (i.e. vector offsets)
			Position = static_cast<pxr::GfVec3f>(UsdGeomBindTransform.TransformDir(Position));
		}
		for (pxr::GfVec3f& Normal : NormalOffsets)
		{
			Normal = static_cast<pxr::GfVec3f>(InvTransposeGeomBindTransform.TransformDir(Normal));
		}
	}

	FStaticMeshAttributes Attributes{InOutMeshDescription};
	TVertexAttributesRef<FVector3f> MeshPositions = Attributes.GetVertexPositions();
	TVertexInstanceAttributesRef<FVector3f> MeshInstanceNormals = Attributes.GetVertexInstanceNormals();

	bool bApplied = false;

	FMatrix TotalMatrix = AdditionalTransform.ToMatrixWithScale();
	FMatrix TotalMatrixForNormal = TotalMatrix.Inverse().GetTransposed();

	// We have one value for each vertex of the mesh description
	// TODO: Deduplicate this code, since only the indexing differs?
	if (PointIndices.size() == 0)
	{
		// Position offsets
		if (PositionOffsets.size() == (size_t)MeshPositions.GetNumElements())
		{
			for (uint32 OffsetIndex = 0; OffsetIndex < PositionOffsets.size(); ++OffsetIndex)
			{
				const FVector UEOffset = TotalMatrix.TransformVector(UsdToUnreal::ConvertVector(StageInfo, PositionOffsets[OffsetIndex]));
				MeshPositions[OffsetIndex] += FVector3f{UEOffset * Weight};
			}

			bApplied = true;
		}
		else
		{
			FString InbetweenText = InInbetweenName.IsEmpty() ? FString::Printf(TEXT(" (inbetween '%s')"), *InInbetweenName) : TEXT("");
			USD_LOG_WARNING(
				TEXT("Failed to apply position offsets from BlendShape '%s'%s: Expected MeshDescription to have %zu vertex positions, but it has %u!"),
				*UsdToUnreal::ConvertPath(InBlendShapePrim.GetPrimPath()),
				*InbetweenText,
				PositionOffsets.size(),
				MeshPositions.GetNumElements()
			);
		}

		// Normal offsets
		if (NormalOffsets.size() == (size_t)MeshPositions.GetNumElements())
		{
			for (uint32 VertexIndex = 0; VertexIndex < NormalOffsets.size(); ++VertexIndex)
			{
				const FVector UENormal = TotalMatrixForNormal.TransformVector(UsdToUnreal::ConvertVector(StageInfo, NormalOffsets[VertexIndex])).GetSafeNormal();

				TArrayView<const FVertexInstanceID> Instances = InOutMeshDescription.GetVertexVertexInstanceIDs(VertexIndex);
				for (FVertexInstanceID InstanceID : Instances)
				{
					MeshInstanceNormals[InstanceID] = (MeshInstanceNormals[InstanceID] + Weight * FVector3f{UENormal}).GetSafeNormal();
				}
			}

			bApplied = true;
		}
		else if (NormalOffsets.size() != 0)
		{
			FString InbetweenText = InInbetweenName.IsEmpty() ? FString::Printf(TEXT(" (inbetween '%s')"), *InInbetweenName) : TEXT("");
			USD_LOG_WARNING(
				TEXT("Failed to apply normal offsets from BlendShape '%s'%s: Expected MeshDescription to have %zu vertices, but it has %u!"),
				*UsdToUnreal::ConvertPath(InBlendShapePrim.GetPrimPath()),
				*InbetweenText,
				NormalOffsets.size(),
				MeshPositions.GetNumElements()
			);
		}
	}
	// We have values for only a few vertices of the mesh description
	else if (PointIndices.size() > 0)
	{
		// Position offsets
		if (PointIndices.size() == PositionOffsets.size())
		{
			for (uint32 OffsetIndex = 0; OffsetIndex < PositionOffsets.size(); ++OffsetIndex)
			{
				int TargetPointIndex = PointIndices[OffsetIndex];
				if (TargetPointIndex >= 0 && TargetPointIndex < MeshPositions.GetNumElements())
				{
					const FVector UEOffset = TotalMatrix.TransformVector(UsdToUnreal::ConvertVector(StageInfo, PositionOffsets[OffsetIndex]));
					MeshPositions[TargetPointIndex] += FVector3f{UEOffset * Weight};
				}
			}

			bApplied = true;
		}
		else
		{
			FString InbetweenText = InInbetweenName.IsEmpty() ? FString::Printf(TEXT(" (inbetween '%s')"), *InInbetweenName) : TEXT("");
			USD_LOG_WARNING(
				TEXT(
					"Failed to apply indexed position offsets from BlendShape '%s'%s: The blend shape has %zu offsets, but %zu indices! (those should match)"
				),
				*UsdToUnreal::ConvertPath(InBlendShapePrim.GetPrimPath()),
				*InbetweenText,
				PositionOffsets.size(),
				PointIndices.size()
			);
		}

		// Normal offsets
		if (PointIndices.size() == NormalOffsets.size())
		{
			for (uint32 NormalIndex = 0; NormalIndex < NormalOffsets.size(); ++NormalIndex)
			{
				int TargetPointIndex = PointIndices[NormalIndex];
				if (TargetPointIndex >= 0 && TargetPointIndex < MeshPositions.GetNumElements())
				{
					const FVector UENormal = TotalMatrixForNormal.TransformVector(UsdToUnreal::ConvertVector(StageInfo, NormalOffsets[NormalIndex])).GetSafeNormal();

					TArrayView<const FVertexInstanceID> Instances = InOutMeshDescription.GetVertexVertexInstanceIDs(TargetPointIndex);
					for (FVertexInstanceID InstanceID : Instances)
					{
						MeshInstanceNormals[InstanceID] = (MeshInstanceNormals[InstanceID] + Weight * FVector3f{UENormal}).GetSafeNormal();
					}
				}
			}

			bApplied = true;
		}
		else if (NormalOffsets.size() != 0)
		{
			FString InbetweenText = InInbetweenName.IsEmpty() ? FString::Printf(TEXT(" (inbetween '%s')"), *InInbetweenName) : TEXT("");
			USD_LOG_WARNING(
				TEXT(
					"Failed to apply indexed normal offsets from BlendShape '%s'%s: The blend shape has %zu offsets, but %zu indices! (those should match)"
				),
				*UsdToUnreal::ConvertPath(InBlendShapePrim.GetPrimPath()),
				*InbetweenText,
				NormalOffsets.size(),
				PointIndices.size()
			);
		}
	}

	return bApplied;
}

#endif	  // USE_USD_SDK

#if USE_USD_SDK && WITH_EDITOR

bool UnrealToUsd::ConvertSkeleton(const FReferenceSkeleton& ReferenceSkeleton, pxr::UsdSkelSkeleton& UsdSkeleton)
{
	FScopedUsdAllocs Allocs;

	pxr::UsdStageRefPtr Stage = UsdSkeleton.GetPrim().GetStage();
	if (!Stage)
	{
		return false;
	}

	FUsdStageInfo StageInfo{Stage};

	// Joints
	{
		pxr::UsdAttribute JointsAttr = UsdSkeleton.CreateJointsAttr();
		UnrealToUsd::ConvertJointsAttribute(ReferenceSkeleton, JointsAttr);
	}

	pxr::VtArray<pxr::GfMatrix4d> LocalSpaceJointTransforms;
	LocalSpaceJointTransforms.reserve(ReferenceSkeleton.GetRefBonePose().Num());
	for (const FTransform& BonePose : ReferenceSkeleton.GetRefBonePose())
	{
		LocalSpaceJointTransforms.push_back(UnrealToUsd::ConvertTransform(StageInfo, BonePose));
	}

	TArray<FTransform> WorldSpaceUEJointTransforms;
	FAnimationRuntime::FillUpComponentSpaceTransforms(ReferenceSkeleton, ReferenceSkeleton.GetRefBonePose(), WorldSpaceUEJointTransforms);

	pxr::VtArray<pxr::GfMatrix4d> WorldSpaceJointTransforms;
	WorldSpaceJointTransforms.reserve(WorldSpaceUEJointTransforms.Num());
	for (const FTransform& WorldSpaceUETransform : WorldSpaceUEJointTransforms)
	{
		WorldSpaceJointTransforms.push_back(UnrealToUsd::ConvertTransform(StageInfo, WorldSpaceUETransform));
	}

	// Rest transforms
	{
		pxr::UsdAttribute RestTransformsAttr = UsdSkeleton.CreateRestTransformsAttr();
		RestTransformsAttr.Set(LocalSpaceJointTransforms);
	}

	// Bind transforms
	{
		pxr::UsdAttribute BindTransformsAttr = UsdSkeleton.CreateBindTransformsAttr();
		BindTransformsAttr.Set(WorldSpaceJointTransforms);
	}

	// Use Guide purpose on skeletons by default, unless it has some specific purpose set already
	if (pxr::UsdAttribute PurposeAttr = UsdSkeleton.GetPurposeAttr())
	{
		if (!PurposeAttr.HasAuthoredValue())
		{
			PurposeAttr.Set(pxr::UsdGeomTokens->guide);
		}
	}

	return true;
}

bool UnrealToUsd::ConvertJointsAttribute(const FReferenceSkeleton& ReferenceSkeleton, pxr::UsdAttribute& JointsAttribute)
{
	if (!JointsAttribute)
	{
		return false;
	}

	TArray<FString> FullBonePaths;
	UnrealToUsdImpl::CreateFullBonePaths(ReferenceSkeleton.GetRefBoneInfo(), FullBonePaths);

	pxr::VtArray<pxr::TfToken> Joints;
	Joints.reserve(FullBonePaths.Num());
	for (const FString& BonePath : FullBonePaths)
	{
		Joints.push_back(UnrealToUsd::ConvertToken(*BonePath).Get());
	}

	JointsAttribute.Set(Joints);
	return true;
}

bool UnrealToUsd::ConvertSkeleton(const USkeleton* Skeleton, pxr::UsdSkelSkeleton& UsdSkeleton)
{
	if (!Skeleton)
	{
		return false;
	}

	return UnrealToUsd::ConvertSkeleton(Skeleton->GetReferenceSkeleton(), UsdSkeleton);
}

bool UnrealToUsd::ConvertSkeletalMesh(
	const USkeletalMesh* SkeletalMesh,
	pxr::UsdPrim& SkelRootPrim,
	const pxr::UsdTimeCode TimeCode,
	UE::FUsdStage* StageForMaterialAssignments,
	int32 LowestMeshLOD,
	int32 HighestMeshLOD
)
{
	pxr::UsdSkelRoot SkelRoot{SkelRootPrim};
	if (!SkeletalMesh || !SkeletalMesh->GetSkeleton() || !SkelRoot)
	{
		return false;
	}

	FScopedUsdAllocs UsdAllocs;

	pxr::UsdStageRefPtr Stage = SkelRootPrim.GetStage();
	if (!Stage)
	{
		return false;
	}
	const FUsdStageInfo StageInfo(Stage);

	const FSkeletalMeshModel* SkelMeshResource = SkeletalMesh->GetImportedModel();
	int32 NumLODs = SkelMeshResource->LODModels.Num();
	if (NumLODs < 1)
	{
		return false;
	}

	// Make sure they're both >= 0 (the options dialog slider is clamped, but this may be called directly)
	LowestMeshLOD = FMath::Clamp(LowestMeshLOD, 0, NumLODs - 1);
	HighestMeshLOD = FMath::Clamp(HighestMeshLOD, 0, NumLODs - 1);

	// Make sure Lowest <= Highest
	int32 Temp = FMath::Min(LowestMeshLOD, HighestMeshLOD);
	HighestMeshLOD = FMath::Max(LowestMeshLOD, HighestMeshLOD);
	LowestMeshLOD = Temp;

	// Make sure it's at least 1 LOD level
	NumLODs = FMath::Max(HighestMeshLOD - LowestMeshLOD + 1, 1);

	pxr::UsdVariantSets VariantSets = SkelRootPrim.GetVariantSets();
	if (NumLODs > 1 && VariantSets.HasVariantSet(UnrealIdentifiers::LOD))
	{
		USD_LOG_ERROR(
			TEXT("Failed to export higher LODs for skeletal mesh '%s', as the target prim already has a variant set named '%s'!"),
			*SkeletalMesh->GetName(),
			*UsdToUnreal::ConvertToken(UnrealIdentifiers::LOD)
		);
		NumLODs = 1;
	}

	bool bExportMultipleLODs = NumLODs > 1;

	pxr::SdfPath ParentPrimPath = SkelRootPrim.GetPath();
	std::string LowestLODAdded = "";

	// Collect all material assignments, referenced by the sections' material indices
	bool bHasMaterialAssignments = false;
	TArray<FString> MaterialAssignments;
	for (const FSkeletalMaterial& SkeletalMaterial : SkeletalMesh->GetMaterials())
	{
		FString AssignedMaterialPathName;
		if (UMaterialInterface* Material = SkeletalMaterial.MaterialInterface)
		{
			if (Material->GetOutermost() != GetTransientPackage())
			{
				AssignedMaterialPathName = Material->GetPathName();
				bHasMaterialAssignments = true;
			}
		}

		MaterialAssignments.Add(AssignedMaterialPathName);
	}
	if (!bHasMaterialAssignments)
	{
		// Prevent creation of the unrealMaterials attribute in case we don't have any assignments at all
		MaterialAssignments.Reset();
	}

	// Create and fill skeleton
	pxr::UsdSkelBindingAPI SkelBindingAPI = pxr::UsdSkelBindingAPI::Apply(SkelRootPrim);
	{
		pxr::UsdPrim SkeletonPrim = Stage->DefinePrim(
			SkelRootPrim.GetPath().AppendChild(UnrealToUsd::ConvertToken(UnrealIdentifiers::ExportedSkeletonPrimName).Get()),
			UnrealToUsd::ConvertToken(TEXT("Skeleton")).Get()
		);
		pxr::UsdSkelSkeleton SkelSkeleton{SkeletonPrim};

		pxr::UsdRelationship SkelRel = SkelBindingAPI.CreateSkeletonRel();
		SkelRel.SetTargets({SkeletonPrim.GetPath()});

		UnrealToUsd::ConvertSkeleton(SkeletalMesh->GetRefSkeleton(), SkelSkeleton);
	}

	// Export extents onto the SkelRoot
	TUsdStore<pxr::VtArray<pxr::GfVec3f>> USDBounds = UnrealToUsd::ConvertBounds(StageInfo, SkeletalMesh->GetBounds().GetBox());
	if (USDBounds.Get().size() > 0)
	{
		if (pxr::UsdAttribute Attr = SkelRoot.CreateExtentAttr())
		{
			Attr.Set(USDBounds.Get());
		}
	}

	// Actual meshes
	for (int32 LODIndex = LowestMeshLOD; LODIndex <= HighestMeshLOD; ++LODIndex)
	{
		const FSkeletalMeshLODModel& LODModel = SkelMeshResource->LODModels[LODIndex];

		if (LODModel.NumVertices == 0 || LODModel.Sections.Num() == 0)
		{
			continue;
		}

		// LOD0, LOD1, etc.
		std::string VariantName = UnrealIdentifiers::LOD.GetString() + UnrealToUsd::ConvertString(*LexToString(LODIndex)).Get();
		if (LowestLODAdded.size() == 0)
		{
			LowestLODAdded = VariantName;
		}

		// Enable the variant edit context, if we are creating variant LODs
		TOptional<pxr::UsdEditContext> EditContext;
		if (bExportMultipleLODs)
		{
			pxr::UsdVariantSet VariantSet = VariantSets.GetVariantSet(UnrealIdentifiers::LOD);

			if (!VariantSet.AddVariant(VariantName))
			{
				continue;
			}

			VariantSet.SetVariantSelection(VariantName);
			EditContext.Emplace(VariantSet.GetVariantEditContext());
		}

		pxr::SdfPath MeshPrimPath = ParentPrimPath.AppendPath(pxr::SdfPath(
			bExportMultipleLODs ? VariantName : UnrealToUsd::ConvertString(*UsdUtils::SanitizeUsdIdentifier(*SkeletalMesh->GetName())).Get()
		));
		pxr::UsdPrim UsdLODPrim = Stage->DefinePrim(MeshPrimPath, UnrealToUsd::ConvertToken(TEXT("Mesh")).Get());
		pxr::UsdGeomMesh UsdLODPrimGeomMesh{UsdLODPrim};

		// Export extents onto the Mesh itself too (it's the same extent in our case as we always just have one mesh)
		if (USDBounds.Get().size() > 0)
		{
			if (pxr::UsdAttribute Attr = UsdLODPrimGeomMesh.CreateExtentAttr())
			{
				Attr.Set(USDBounds.Get());
			}
		}

		pxr::UsdPrim MaterialPrim = UsdLODPrim;
		if (StageForMaterialAssignments)
		{
			pxr::UsdStageRefPtr MaterialStage{*StageForMaterialAssignments};
			MaterialPrim = MaterialStage->OverridePrim(MeshPrimPath);
		}

		TArray<int32> LODMaterialMap;
		if (const FSkeletalMeshLODInfo* LODInfo = SkeletalMesh->GetLODInfo(LODIndex))
		{
			LODMaterialMap = LODInfo->LODMaterialMap;
		}

		TArray<int32> SourceToPackedVertexIndex;
		UnrealToUsdImpl::ConvertSkeletalMeshLOD(
			SkeletalMesh,
			LODModel,
			UsdLODPrimGeomMesh,
			SkeletalMesh->GetHasVertexColors(),
			MaterialAssignments,
			LODMaterialMap,
			TimeCode,
			MaterialPrim,
			SourceToPackedVertexIndex
		);

		// Relationships can't target prims inside variants, so if we have BlendShapes to export we have to disable the edit target
		// so that the blend shapes end up outside the variants and the Meshes can have their blendShapeTargets relationships pointing at them
		if (bExportMultipleLODs && SkeletalMesh->GetMorphTargets().Num() > 0)
		{
			EditContext.Reset();
		}

		pxr::VtArray<pxr::TfToken> AddedBlendShapes;
		pxr::SdfPathVector AddedBlendShapeTargets;
		for (UMorphTarget* MorphTarget : SkeletalMesh->GetMorphTargets())
		{
			if (!MorphTarget || !MorphTarget->HasDataForLOD(LODIndex))
			{
				continue;
			}

			TConstArrayView<FMorphTargetDelta> DeltaArray = MorphTarget->GetMorphTargetDeltas(LODIndex);
			if (DeltaArray.IsEmpty())
			{
				continue;
			}

			pxr::SdfPath ParentPath = bExportMultipleLODs ? SkelRootPrim.GetPath() : UsdLODPrim.GetPath();

			pxr::SdfPath BlendShapePath = ParentPath.AppendPath(
				UnrealToUsd::ConvertPath(*UsdUtils::SanitizeUsdIdentifier(*MorphTarget->GetName())).Get()
			);
			pxr::UsdPrim BlendShapePrim = UsdLODPrim.GetStage()->DefinePrim(BlendShapePath, UnrealToUsd::ConvertToken(TEXT("BlendShape")).Get());
			pxr::UsdSkelBlendShape BlendShape{BlendShapePrim};

			bool bCreatedBlendShape = UnrealToUsdImpl::ConvertMorphTargetDeltas(
				DeltaArray,
				SourceToPackedVertexIndex,
				BlendShape,
				TimeCode
			);
			if (!bCreatedBlendShape)
			{
				continue;
			}

			AddedBlendShapes.push_back(UnrealToUsd::ConvertToken(*UsdUtils::SanitizeUsdIdentifier(*MorphTarget->GetName())).Get());
			AddedBlendShapeTargets.push_back(BlendShapePath);
		}

		if (AddedBlendShapeTargets.size() > 0)
		{
			// Restore the edit target to the current LOD variant so that the relationship itself ends up inside the mesh, inside the variant
			if (bExportMultipleLODs)
			{
				EditContext.Emplace(VariantSets.GetVariantSet(UnrealIdentifiers::LOD).GetVariantEditContext());
			}

			pxr::UsdSkelBindingAPI LODMeshSkelBindingAPI = pxr::UsdSkelBindingAPI::Apply(UsdLODPrim);
			LODMeshSkelBindingAPI.CreateBlendShapeTargetsRel().SetTargets(AddedBlendShapeTargets);
			LODMeshSkelBindingAPI.CreateBlendShapesAttr().Set(AddedBlendShapes);
		}
	}

	if (bExportMultipleLODs)
	{
		VariantSets.GetVariantSet(UnrealIdentifiers::LOD).SetVariantSelection(LowestLODAdded);
	}

	return true;
}

bool UnrealToUsd::ConvertAnimSequence(UAnimSequence* AnimSequence, pxr::UsdPrim& SkelAnimPrim)
{
	if (!SkelAnimPrim || !AnimSequence || !AnimSequence->GetSkeleton())
	{
		return false;
	}

	pxr::UsdSkelAnimation UsdSkelAnim(SkelAnimPrim);

	if (!UsdSkelAnim)
	{
		return false;
	}

	USkeleton* AnimSkeleton = AnimSequence->GetSkeleton();
	USkeletalMesh* SkeletalMesh = AnimSkeleton->GetAssetPreviewMesh(AnimSequence);

	if (!SkeletalMesh)
	{
		SkeletalMesh = AnimSkeleton->FindCompatibleMesh();
	}

	if (!SkeletalMesh)
	{
		return false;
	}

	const FReferenceSkeleton& RefSkeleton = SkeletalMesh->GetRefSkeleton();
	const int32 NumBones = RefSkeleton.GetRefBoneInfo().Num();
	const double TimeCodesPerSecond = SkelAnimPrim.GetStage()->GetTimeCodesPerSecond();

	// The +1 is because 1s length at 1 tcps implies we want two time codes: One at zero, and one at 1s
	// The ceil to make sure we never clip the animation in case it doesn't end exactly on an exported time code value
	const int32 NumTimeCodes = FMath::CeilToInt(AnimSequence->GetPlayLength() * TimeCodesPerSecond) + 1;

	if (NumBones <= 0)
	{
		return false;
	}

	FScopedUsdAllocs UsdAllocs;
	pxr::SdfChangeBlock ChangeBlock;

	pxr::UsdSkelRoot ParentSkelRoot = pxr::UsdSkelRoot{UsdUtils::GetClosestParentSkelRoot(SkelAnimPrim)};
	pxr::UsdAttribute ExtentsAttr = ParentSkelRoot ? ParentSkelRoot.CreateExtentAttr() : pxr::UsdAttribute{};

	FUsdStageInfo StageInfo(SkelAnimPrim.GetStage());

	// Blend shapes
	{
		pxr::VtArray<pxr::TfToken> BlendShapeNames;
		pxr::VtArray<float> BlendShapeWeights;

		// We need to make sure we have at least one mark on the memstack allocator because FBlendedCurve
		// will allocate using one and will assert if there aren't any marks yet
		FMemMark Mark(FMemStack::Get());

		// Blend shape weights
		for (int32 TimeCode = 0; TimeCode < NumTimeCodes; ++TimeCode)
		{
			const double AnimTime = TimeCode / TimeCodesPerSecond;

			FBlendedCurve BlendedCurve;
			constexpr bool bForceUseRawData = true;
			AnimSequence->EvaluateCurveData(BlendedCurve, FAnimExtractContext(static_cast<double>(AnimTime)), bForceUseRawData);

			BlendShapeNames.clear();
			BlendShapeNames.reserve(BlendedCurve.Num());
			BlendShapeWeights.clear();
			BlendShapeWeights.reserve(BlendedCurve.Num());

			BlendedCurve.ForEachElement(
				[&BlendShapeWeights, &BlendShapeNames](const UE::Anim::FCurveElement& InElement)
				{
					if (EnumHasAnyFlags(InElement.Flags, UE::Anim::ECurveElementFlags::MorphTarget))
					{
						BlendShapeNames.push_back(UnrealToUsd::ConvertToken(*InElement.Name.ToString()).Get());
						BlendShapeWeights.push_back(InElement.Value);
					}
				}
			);

			if (BlendShapeWeights.size() > 0 && BlendShapeNames.size() > 0)
			{
				UsdSkelAnim.CreateBlendShapesAttr().Set(BlendShapeNames, pxr::UsdTimeCode(TimeCode));
				UsdSkelAnim.CreateBlendShapeWeightsAttr().Set(BlendShapeWeights, pxr::UsdTimeCode(TimeCode));
			}
		}
	}

	// Joints
	{
		pxr::UsdAttribute JointsAttr = UsdSkelAnim.CreateJointsAttr();
		UnrealToUsd::ConvertJointsAttribute(RefSkeleton, JointsAttr);
	}

	// Translations, Rotations, Scales & Extents
	{
		pxr::UsdAttribute TranslationsAttr = UsdSkelAnim.CreateTranslationsAttr();
		pxr::UsdAttribute RotationsAttr = UsdSkelAnim.CreateRotationsAttr();
		pxr::UsdAttribute ScalesAttr = UsdSkelAnim.CreateScalesAttr();

		UDebugSkelMeshComponent* DebugSkelMeshComponent = NewObject<UDebugSkelMeshComponent>();
		DebugSkelMeshComponent->RegisterComponentWithWorld(IUsdClassesModule::GetCurrentWorld());
		DebugSkelMeshComponent->EmptyOverrideMaterials();
		DebugSkelMeshComponent->SetSkeletalMesh(SkeletalMesh);

		const bool bEnable = true;
		DebugSkelMeshComponent->EnablePreview(bEnable, AnimSequence);

		for (int32 TimeCode = 0; TimeCode < NumTimeCodes; ++TimeCode)
		{
			const float AnimTime = TimeCode / TimeCodesPerSecond;

			const bool bFireNotifies = false;
			DebugSkelMeshComponent->SetPosition(AnimTime, bFireNotifies);
			DebugSkelMeshComponent->RefreshBoneTransforms();

			pxr::VtVec3fArray Translations;
			Translations.reserve(NumBones);

			pxr::VtQuatfArray Rotations;
			Rotations.reserve(NumBones);

			pxr::VtVec3hArray Scales;
			Scales.reserve(NumBones);

			TArray<FTransform> LocalBoneTransforms = DebugSkelMeshComponent->GetBoneSpaceTransforms();

			for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
			{
				FTransform BoneTransform = LocalBoneTransforms[BoneIndex];
				BoneTransform = UsdUtils::ConvertAxes(StageInfo.UpAxis == EUsdUpAxis::ZAxis, BoneTransform);

				Translations.push_back(UnrealToUsd::ConvertVectorFloat(BoneTransform.GetTranslation()));
				Rotations.push_back(UnrealToUsd::ConvertQuatFloat(BoneTransform.GetRotation()).GetNormalized());
				Scales.push_back(UnrealToUsd::ConvertVectorHalf(BoneTransform.GetScale3D()));
			}

			TranslationsAttr.Set(Translations, pxr::UsdTimeCode(TimeCode));
			RotationsAttr.Set(Rotations, pxr::UsdTimeCode(TimeCode));
			ScalesAttr.Set(Scales, pxr::UsdTimeCode(TimeCode));

			FBox Bounds = DebugSkelMeshComponent->CalcBounds(FTransform::Identity).GetBox();
			if (Bounds.IsValid && ExtentsAttr)
			{
				TUsdStore<pxr::VtArray<pxr::GfVec3f>> USDBounds = UnrealToUsd::ConvertBounds(StageInfo, Bounds);
				ExtentsAttr.Set(USDBounds.Get(), pxr::UsdTimeCode(TimeCode));
			}
		}

		// Actively delete it or else it will remain visible on the viewport
		DebugSkelMeshComponent->DestroyComponent();
	}

	const int32 StageEndTimeCode = SkelAnimPrim.GetStage()->GetEndTimeCode();

	if (NumTimeCodes > StageEndTimeCode)
	{
		SkelAnimPrim.GetStage()->SetEndTimeCode(NumTimeCodes - 1);
	}

	return true;
}

bool UnrealToUsd::ConvertControlRigSection(
	UMovieSceneControlRigParameterSection* InSection,
	const FMovieSceneInverseSequenceTransform& InTransform,
	UMovieScene* InMovieScene,
	IMovieScenePlayer* InPlayer,
	const FReferenceSkeleton& InRefSkeleton,
	pxr::UsdPrim& InSkelRoot,
	pxr::UsdPrim& OutSkelAnimPrim,
	const UsdUtils::FBlendShapeMap* InBlendShapeMap
)
{
	if (!InSection || !InPlayer || !OutSkelAnimPrim)
	{
		return false;
	}

	UControlRig* ControlRig = InSection->GetControlRig();
	if (!ControlRig)
	{
		return false;
	}

	FScopedUsdAllocs UsdAllocs;

	pxr::UsdSkelAnimation SkelAnim{OutSkelAnimPrim};
	pxr::UsdStageRefPtr UsdStage = OutSkelAnimPrim.GetStage();
	if (!UsdStage || !SkelAnim)
	{
		return false;
	}

	if (UsdUtils::NotifyIfInstanceProxy(OutSkelAnimPrim))
	{
		return false;
	}

	FUsdStageInfo StageInfo(UsdStage);

	double StartTime = FPlatformTime::Cycles64();

	ControlRig->Initialize();
	ControlRig->RequestInit();
	ControlRig->Evaluate_AnyThread();	 // Important as it runs the Construction event, which can change topology

	// Record how the topology looks while we setup our arrays and maps. If this changes during
	// baking we'll just drop everything and return
	URigHierarchy* InitialHierarchy = ControlRig->GetHierarchy();
	if (!InitialHierarchy)
	{
		return false;
	}
	uint16 TopologyVersion = InitialHierarchy->GetTopologyVersion();

	// Prepare to remap from Rig joint order to USkeleton/Skeleton prim joint order.
	// This works because the topology won't change in here, and bone names are unique across the entire skeleton
	// Its possible we'll be putting INDEX_NONEs into RigIndexToRefSkeletonIndex, but that's alright.
	TArray<int32> RigJointIndexToRefSkeletonIndex;
	TFunction<void()> RegenerateRigJointIndexToRefSkeletonIndex = [ControlRig, &RigJointIndexToRefSkeletonIndex, &InRefSkeleton]()
	{
		URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
		if (!Hierarchy)
		{
			return;
		}

		RigJointIndexToRefSkeletonIndex.Reset();
		for (FRigBoneElement* RigBone : Hierarchy->GetBones())
		{
			RigJointIndexToRefSkeletonIndex.Add(InRefSkeleton.FindBoneIndex(RigBone->GetFName()));
		}
	};
	RegenerateRigJointIndexToRefSkeletonIndex();

	pxr::UsdAttribute JointsAttr = SkelAnim.CreateJointsAttr();
	UnrealToUsd::ConvertJointsAttribute(InRefSkeleton, JointsAttr);

	TArray<FTransform> GlobalUEJointTransformsForFrame;
	GlobalUEJointTransformsForFrame.SetNum(InRefSkeleton.GetNum());

	pxr::UsdAttribute TranslationsAttr = SkelAnim.CreateTranslationsAttr();
	pxr::UsdAttribute RotationsAttr = SkelAnim.CreateRotationsAttr();
	pxr::UsdAttribute ScalesAttr = SkelAnim.CreateScalesAttr();
	pxr::UsdAttribute BlendShapeWeightsAttr = SkelAnim.CreateBlendShapeWeightsAttr();
	pxr::UsdAttribute BlendShapesAttr = SkelAnim.CreateBlendShapesAttr();

	TranslationsAttr.Clear();
	RotationsAttr.Clear();
	ScalesAttr.Clear();
	pxr::VtVec3fArray Translations;
	pxr::VtQuatfArray Rotations;
	pxr::VtVec3hArray Scales;
	Translations.resize(InRefSkeleton.GetNum());
	Rotations.resize(InRefSkeleton.GetNum());
	Scales.resize(InRefSkeleton.GetNum());

	FFrameRate TickResolution = InMovieScene->GetTickResolution();
	FFrameRate DisplayRate = InMovieScene->GetDisplayRate();

	const double StageTimeCodesPerSecond = UsdStage->GetTimeCodesPerSecond();
	const FFrameRate StageFrameRate(StageTimeCodesPerSecond, 1);

	TRange<FFrameNumber> PlaybackRange = InMovieScene->GetPlaybackRange();
	TRange<FFrameNumber> BakeTickRange = InSection->ComputeEffectiveRange();

	// Try our best to find the section start/end inclusive frames
	FFrameNumber StartInclTickFrame;
	FFrameNumber EndInclTickFrame;
	{
		TOptional<TRangeBound<FFrameNumber>> LowerBoundToUse;
		if (BakeTickRange.HasLowerBound())
		{
			TRangeBound<FFrameNumber> SectionLowerBound = BakeTickRange.GetLowerBound();
			if (!SectionLowerBound.IsOpen())
			{
				LowerBoundToUse = SectionLowerBound;
			}
		}
		if (!LowerBoundToUse.IsSet() && PlaybackRange.HasLowerBound())
		{
			TRangeBound<FFrameNumber> PlaybackLowerBound = PlaybackRange.GetLowerBound();
			if (!PlaybackLowerBound.IsOpen())
			{
				LowerBoundToUse = PlaybackLowerBound;
			}
		}
		if (!LowerBoundToUse.IsSet())
		{
			return false;
		}
		StartInclTickFrame = LowerBoundToUse.GetValue().GetValue() + (LowerBoundToUse.GetValue().IsInclusive() ? 0 : 1);
	}
	{
		TOptional<TRangeBound<FFrameNumber>> UpperBoundToUse;
		if (BakeTickRange.HasUpperBound())
		{
			TRangeBound<FFrameNumber> SectionUpperBound = BakeTickRange.GetUpperBound();
			if (!SectionUpperBound.IsOpen())
			{
				UpperBoundToUse = SectionUpperBound;
			}
		}
		if (!UpperBoundToUse.IsSet() && PlaybackRange.HasUpperBound())
		{
			TRangeBound<FFrameNumber> PlaybackUpperBound = PlaybackRange.GetUpperBound();
			if (!PlaybackUpperBound.IsOpen())
			{
				UpperBoundToUse = PlaybackUpperBound;
			}
		}
		if (!UpperBoundToUse.IsSet())
		{
			return false;
		}
		EndInclTickFrame = UpperBoundToUse.GetValue().GetValue() + (UpperBoundToUse.GetValue().IsInclusive() ? 0 : -1);
	}

	UsdUtils::NotifyIfOverriddenOpinion(BlendShapeWeightsAttr);
	UsdUtils::NotifyIfOverriddenOpinion(TranslationsAttr);
	UsdUtils::NotifyIfOverriddenOpinion(RotationsAttr);
	UsdUtils::NotifyIfOverriddenOpinion(ScalesAttr);

	pxr::VtArray<pxr::TfToken> CurveNames;
	pxr::VtArray<float> CurvesValuesForTime;

	// Prepare blend shape baking
	// So far there doesn't seem to be any good way of handling the baking into blend shapes with inbetweens:
	//	- We can't just pretend the Mesh prims have the flattened inbetween blend shapes (like we'd get if they were
	//    exported) because we'd get warnings by having blend shape targets to blend shape prims that don't exist;
	//  - We could flatten the actual BlendShape on the Mesh prim here, but that may be a bit too bold as the user likely
	//    wants to keep their Mesh asset more or less intact when just baking out an animation section. If users do want
	//    this behaviour we can later add it though;
	//  - An alternative would have been to try to collect all the primary+inbetween weights, combine them back into a
	//    single weight value, and write them back. That would work, but it would be incredibly hard to tell what is
	//    going on from the users' perspective because that weight conversion is lossy and imperfect. Not to mention we'd
	//    have this tricky code to test/maintain that slows down the baking process as a whole, and everything would break
	//    if e.g. the curves were renamed;
	//  - A slightly different approach to above would be to have the Mesh prims listen to the flattened inbetween blend
	//    shape channels, but map them all to the single blend shape: This is not allowed in USD though, and its enough
	//    to crash usdview. Besides, it wouldn't have added a lot of value as it would be impossible to comprehend what
	//    was going on.
	// The best we can do at the moment is to make one channel for each curve on the SkelAnimation prim, but maintain
	// each Mesh prim connected only to the primary blend shape channel, if it was originally. We'll show a warning
	// explaining the situation though.
	if (InBlendShapeMap && InSkelRoot)
	{
		TArray<FRigCurveElement*> CurveElements = InitialHierarchy->GetCurves();

		CurveNames.reserve(CurveElements.Num());
		for (int32 Index = 0; Index < CurveElements.Num(); ++Index)
		{
			FRigCurveElement* Element = CurveElements[Index];
			const FString& CurveNameString = Element->GetName();

			CurveNames.push_back(UnrealToUsd::ConvertToken(*CurveNameString).Get());
		}

		// Check if the blend shape channels on skel animation are the same names as morph target curves.
		// Note that the actual order of the channel names within BlendShapesAttr is not important, as we'll always
		// write out a new order that matches the rig anyway. We just want to know if all consumers of this SkelAnimation
		// already have the processed, "one per morph target" channels
		bool bNeedChannelUpdate = true;
		pxr::VtArray<pxr::TfToken> SkelAnimBlendShapeChannels;
		if (BlendShapesAttr && BlendShapesAttr.Get(&SkelAnimBlendShapeChannels))
		{
			if (SkelAnimBlendShapeChannels.size() == CurveNames.size())
			{
				std::unordered_set<pxr::TfToken, pxr::TfToken::HashFunctor> ExistingCurveNames;
				for (const pxr::TfToken& Channel : SkelAnimBlendShapeChannels)
				{
					ExistingCurveNames.insert(Channel);
				}

				bool bFoundAllCurves = true;
				for (const pxr::TfToken& CurveName : CurveNames)
				{
					if (ExistingCurveNames.count(CurveName) == 0)
					{
						bFoundAllCurves = false;
						break;
					}
				}

				bNeedChannelUpdate = !bFoundAllCurves;
			}
		}

		// We haven't processed this SkelAnimation before, so we need to do it now.
		// The summary is that since each MorphTarget/BlendShape has an independet curve in UE, but can share curves
		// arbitrarily in USD, we need to replace the existing SkelAnimation channels with ones that are unique for
		// each blend shape. This is not ideal, but the alternatives would be to: Not handle blend shape curves via
		// control rigs; Have some morph target curves unintuitively "mirror each other" in UE, if at all possible;
		// Try to keep the channels shared on USD's side, which would desync USD/UE and show a different result when
		// reloading.
		if (bNeedChannelUpdate)
		{
			// We'll change the blend shape channel names, so we need to update all meshes that were using them too.
			// For now we'll assume that they're all inside the same skel root. We could upgrade this for the stage
			// later too, if needed
			// TODO: This could probably be updated to just find the actual skinned meshes, and have some better parameters
			// like skinning/skeleton queries
			for (UE::FUsdPrim& MeshPrim : UsdUtils::GetAllPrimsOfType(UE::FUsdPrim{InSkelRoot}, TEXT("UsdGeomMesh")))
			{
				pxr::UsdSkelBindingAPI SkelBindingAPI{MeshPrim};
				if (!SkelBindingAPI)
				{
					continue;
				}

				pxr::UsdRelationship TargetsRel = SkelBindingAPI.GetBlendShapeTargetsRel();
				pxr::UsdAttribute ChannelsAttr = SkelBindingAPI.GetBlendShapesAttr();

				if (TargetsRel && ChannelsAttr)
				{
					pxr::SdfPathVector BlendShapeTargets;
					if (TargetsRel.GetTargets(&BlendShapeTargets))
					{
						pxr::VtArray<pxr::TfToken> BlendShapeChannels;
						ChannelsAttr.Get(&BlendShapeChannels);

						BlendShapeChannels.resize(BlendShapeTargets.size());

						pxr::SdfPath MeshPath{MeshPrim.GetPrimPath()};

						bool bRenamedAChannel = false;
						for (int32 BlendShapeIndex = 0; BlendShapeIndex < BlendShapeTargets.size(); ++BlendShapeIndex)
						{
							const pxr::SdfPath& BlendShapePath = BlendShapeTargets[BlendShapeIndex];
							FString PrimaryBlendShapePath = UsdToUnreal::ConvertPath(BlendShapePath.MakeAbsolutePath(MeshPath));

							// Mesh had <blendshape1> target on channel "C" -> We have a morph target called "blendshape1" already, and
							// we'll create a new channel on SkelAnimation called "blendshape1" -> Let's replace channel "C" with channel
							// "blendshape1"
							if (const UsdUtils::FUsdBlendShape* FoundBlendShape = InBlendShapeMap->Find(PrimaryBlendShapePath))
							{
								bRenamedAChannel = true;
								BlendShapeChannels[BlendShapeIndex] = UnrealToUsd::ConvertToken(*FoundBlendShape->Name).Get();
								USD_LOG_INFO(
									TEXT("Updating Mesh '%s' to bind BlendShape target '%s' to SkelAnimation curve '%s'"),
									*MeshPrim.GetPrimPath().GetString(),
									*PrimaryBlendShapePath,
									*FoundBlendShape->Name
								);

								if (FoundBlendShape->Inbetweens.Num() > 0)
								{
									USD_LOG_USERWARNING(FText::Format(
										LOCTEXT(
											"UnsupportedInbetweens",
											"Baking Control Rig parameter sections for BlendShapes with inbetweens (like '{0}') is not currently "
											"supported, so animation for mesh '{1}' may look incorrect! Please flatten the inbetweens into separate "
											"BlendShapes beforehand (importing and exporting will do that)."
										),
										FText::FromString(FoundBlendShape->Name),
										FText::FromString(MeshPrim.GetPrimPath().GetString())
									));
								}
							}
						}

						if (bRenamedAChannel)
						{
							ChannelsAttr.Set(BlendShapeChannels);
							UsdUtils::NotifyIfOverriddenOpinion(ChannelsAttr);
						}
					}
				}
			}
		}

		// Now that we updated the channel names we need to make sure we clear the previous weights as they'll
		// make no sense
		BlendShapeWeightsAttr.Clear();
		BlendShapesAttr.Set(CurveNames);
		CurvesValuesForTime.resize(CurveNames.size());

		UsdUtils::NotifyIfOverriddenOpinion(BlendShapesAttr);
	}

	FFrameTime TickIncr = FFrameRate::TransformTime(1, DisplayRate, TickResolution);
	for (FFrameTime FrameTickTime = StartInclTickFrame; FrameTickTime <= EndInclTickFrame; FrameTickTime += TickIncr)
	{
		TOptional<FFrameTime> TransformedFrameTickTime = InTransform.TryTransformTime(FrameTickTime);
		if (!TransformedFrameTickTime)
		{
			continue;
		}

		double UsdTimeCode = FFrameRate::TransformTime(TransformedFrameTickTime.GetValue(), TickResolution, StageFrameRate).AsDecimal();

		FMovieSceneContext Context = FMovieSceneContext(
										 FMovieSceneEvaluationRange(TransformedFrameTickTime.GetValue(), TickResolution),
										 InPlayer->GetPlaybackStatus()
		)
										 .SetHasJumped(true);

		InPlayer->GetEvaluationTemplate().EvaluateSynchronousBlocking(Context);
		ControlRig->Evaluate_AnyThread();

		URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
		if (!Hierarchy)
		{
			USD_LOG_ERROR(TEXT("Baking Control Rig tracks for rig '%s' failed"), *ControlRig->GetPathName());
			return false;
		}

		if (Hierarchy->GetTopologyVersion() != TopologyVersion)
		{
			USD_LOG_INFO(
				TEXT("Regenerating ControlRig to reference skeleton mapping for rig '%s' as its topology changed"),
				*ControlRig->GetPathName()
			);
			RegenerateRigJointIndexToRefSkeletonIndex();
			TopologyVersion = Hierarchy->GetTopologyVersion();
		}

		// Sadly we have to fetch these each frame as these are regenerated on each evaluation of the Sequencer
		// (c.f. FControlRigBindingHelper::BindToSequencerInstance, URigHierarchy::CopyHierarchy)
		TArray<FRigBoneElement*> BoneElements = Hierarchy->GetBones();

		if (CurvesValuesForTime.size() > 0)
		{
			TArray<FRigCurveElement*> CurveElements = Hierarchy->GetCurves();
			for (int32 ElementIndex = 0; ElementIndex < CurveElements.Num(); ++ElementIndex)
			{
				FRigCurveElement* Element = CurveElements[ElementIndex];
				CurvesValuesForTime[ElementIndex] = Hierarchy->GetCurveValue(Element);
			}

			BlendShapeWeightsAttr.Set(CurvesValuesForTime, pxr::UsdTimeCode(UsdTimeCode));
		}

		for (int32 RigBoneIndex = 0; RigBoneIndex < BoneElements.Num(); ++RigBoneIndex)
		{
			FRigBoneElement* El = BoneElements[RigBoneIndex];

			// Our skeleton doesn't have this rig bone
			int32 RefSkeletonBoneIndex = RigJointIndexToRefSkeletonIndex[RigBoneIndex];
			if (RefSkeletonBoneIndex == INDEX_NONE)
			{
				continue;
			}

			GlobalUEJointTransformsForFrame[RefSkeletonBoneIndex] = Hierarchy->GetTransform(El, ERigTransformType::CurrentGlobal);

			FTransform UsdTransform;

			// We have to calculate the local transforms ourselves since the parent element could be a control
			int32 RefSkeletonParentBoneIndex = InRefSkeleton.GetParentIndex(RefSkeletonBoneIndex);
			if (RefSkeletonParentBoneIndex == INDEX_NONE)
			{
				UsdTransform = UsdUtils::ConvertTransformToUsdSpace(StageInfo, GlobalUEJointTransformsForFrame[RefSkeletonBoneIndex]);
			}
			else
			{
				const FTransform& ChildGlobal = GlobalUEJointTransformsForFrame[RefSkeletonBoneIndex];
				const FTransform& ParentGlobal = GlobalUEJointTransformsForFrame[RefSkeletonParentBoneIndex];
				UsdTransform = UsdUtils::ConvertTransformToUsdSpace(StageInfo, ChildGlobal.GetRelativeTransform(ParentGlobal));
			}

			Translations[RefSkeletonBoneIndex] = UnrealToUsd::ConvertVectorFloat(UsdTransform.GetTranslation());
			Rotations[RefSkeletonBoneIndex] = UnrealToUsd::ConvertQuatFloat(UsdTransform.GetRotation()).GetNormalized();
			Scales[RefSkeletonBoneIndex] = UnrealToUsd::ConvertVectorHalf(UsdTransform.GetScale3D());
		}

		TranslationsAttr.Set(Translations, pxr::UsdTimeCode(UsdTimeCode));
		RotationsAttr.Set(Rotations, pxr::UsdTimeCode(UsdTimeCode));
		ScalesAttr.Set(Scales, pxr::UsdTimeCode(UsdTimeCode));
	}

	double ElapsedSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);
	int ElapsedMin = int(ElapsedSeconds / 60.0);
	ElapsedSeconds -= 60.0 * (double)ElapsedMin;
	USD_LOG_INFO(
		TEXT("Baked new animation for prim '%s' in [%d min %.3f s]"),
		*UsdToUnreal::ConvertPath(OutSkelAnimPrim.GetPrimPath()),
		ElapsedMin,
		ElapsedSeconds
	);

	return true;
}
#endif	  // #if USE_USD_SDK && WITH_EDITOR

#undef LOCTEXT_NAMESPACE

