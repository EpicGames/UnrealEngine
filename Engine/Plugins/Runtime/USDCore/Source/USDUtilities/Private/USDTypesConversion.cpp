// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDTypesConversion.h"

#include "USDConversionUtils.h"
#include "USDMemory.h"

#include "Containers/StringConv.h"

#if USE_USD_SDK

#include "USDIncludesStart.h"
#include "pxr/base/gf/matrix2d.h"
#include "pxr/base/gf/matrix3d.h"
#include "pxr/base/gf/matrix4d.h"
#include "pxr/base/gf/quatd.h"
#include "pxr/base/gf/quatf.h"
#include "pxr/base/gf/quath.h"
#include "pxr/base/gf/vec2f.h"
#include "pxr/base/gf/vec3f.h"
#include "pxr/base/gf/vec4f.h"
#include "pxr/base/vt/array.h"
#include "pxr/usd/sdf/path.h"
#include "pxr/usd/usdGeom/tokens.h"
#include "USDIncludesEnd.h"

FUsdStageInfo::FUsdStageInfo(const pxr::UsdStageRefPtr& Stage)
{
	pxr::TfToken UsdStageAxis = UsdUtils::GetUsdStageUpAxis(Stage);

	if (UsdStageAxis == pxr::UsdGeomTokens->y)
	{
		UpAxis = EUsdUpAxis::YAxis;
	}
	else
	{
		UpAxis = EUsdUpAxis::ZAxis;
	}

	MetersPerUnit = UsdUtils::GetUsdStageMetersPerUnit(Stage);
}

namespace UsdUtils
{
	FTransform ConvertAxes(const bool bZUp, const FTransform Transform)
	{
		FVector Translation = Transform.GetTranslation();
		FQuat Rotation = Transform.GetRotation();
		FVector Scale = Transform.GetScale3D();

		if (bZUp)
		{
			Translation.Y = -Translation.Y;
			Rotation.X = -Rotation.X;
			Rotation.Z = -Rotation.Z;
		}
		else
		{
			Swap(Translation.Y, Translation.Z);

			Rotation = Rotation.Inverse();
			Swap(Rotation.Y, Rotation.Z);

			Swap(Scale.Y, Scale.Z);
		}

		return FTransform(Rotation, Translation, Scale);
	}

	FTransform DecomposeWithUniformReflection(const FMatrix& InMatrix)
	{
		// Reference: FTransform::SetFromMatrix

		FTransform Result;

		FMatrix M = InMatrix;

		FVector Scale = M.ExtractScaling();

		// If there is negative scaling going on, we handle that here
		if (InMatrix.Determinant() < 0.f)
		{
			// Note: Here is where we flip on all three axes simultaneously. Flipping on any odd number of
			// axes would have led to the same result (as long as the rotation matrix M is also correspondingly
			// flipped), but by flipping 3 of them we get to keep the scaling uniform.
			Scale *= -1.0f;
			M.SetAxis(0, -M.GetScaledAxis(EAxis::X));
			M.SetAxis(1, -M.GetScaledAxis(EAxis::Y));
			M.SetAxis(2, -M.GetScaledAxis(EAxis::Z));
		}

		Result.SetScale3D(Scale);
		Result.SetRotation(FQuat{M}.GetNormalized());
		Result.SetTranslation(InMatrix.GetOrigin());

		return Result;
	}

	FTransform ConvertTransformToUsdSpace(const FUsdStageInfo& StageInfo, const FTransform& TransformInUESpace)
	{
		FTransform TransformInUsdSpace = UsdUtils::ConvertAxes(StageInfo.UpAxis == EUsdUpAxis::ZAxis, TransformInUESpace);

		const double UEMetersPerUnit = 0.01;
		if (!FMath::IsNearlyEqual(StageInfo.MetersPerUnit, UEMetersPerUnit))
		{
			TransformInUsdSpace.ScaleTranslation(UEMetersPerUnit / StageInfo.MetersPerUnit);
		}

		return TransformInUsdSpace;
	}

	FTransform ConvertTransformToUESpace(const FUsdStageInfo& StageInfo, const FTransform& TransformInUsdSpace)
	{
		FTransform TransformInUESpace = UsdUtils::ConvertAxes(StageInfo.UpAxis == EUsdUpAxis::ZAxis, TransformInUsdSpace);

		const double UEMetersPerUnit = 0.01;
		if (!FMath::IsNearlyEqual(StageInfo.MetersPerUnit, UEMetersPerUnit))
		{
			TransformInUESpace.ScaleTranslation(StageInfo.MetersPerUnit / UEMetersPerUnit);
		}

		return TransformInUESpace;
	}

	FTransform GetUsdToUESpaceTransform(const FUsdStageInfo& StageInfo)
	{
		FTransform MetersPerUnitTransform;
		const double UEMetersPerUnit = 0.01;
		if (!FMath::IsNearlyEqual(StageInfo.MetersPerUnit, UEMetersPerUnit))
		{
			const double ScaleFactor = StageInfo.MetersPerUnit / UEMetersPerUnit;
			MetersPerUnitTransform.SetScale3D(FVector{ScaleFactor, ScaleFactor, ScaleFactor});
		}

		FTransform UpAxisTransform;
		UpAxisTransform.SetScale3D(FVector{1.0, -1.0, 1.0});
		if (StageInfo.UpAxis == EUsdUpAxis::YAxis)
		{
			UpAxisTransform.SetRotation(FQuat::MakeFromEuler(FVector{90, 0, 0}));
		}

		return MetersPerUnitTransform * UpAxisTransform;
	}
}	 // namespace UsdUtils

namespace UsdToUnreal
{
	FString ConvertString(const std::string& InString)
	{
		return FString(UTF8_TO_TCHAR(InString.c_str()));
	}

	FString ConvertString(const char* InString)
	{
		return FString(UTF8_TO_TCHAR(InString));
	}

	FString ConvertPath(const pxr::SdfPath& Path)
	{
		return ConvertString(Path.GetString().c_str());
	}

	FName ConvertName(const char* InString)
	{
		return FName(InString);
	}

	FName ConvertName(const std::string& InString)
	{
		return FName(InString.c_str());
	}

	FString ConvertToken(const pxr::TfToken& Token)
	{
		return UsdToUnreal::ConvertString(Token.GetString());
	}

	FLinearColor ConvertColor(const pxr::GfVec3f& InValue)
	{
		return FLinearColor(InValue[0], InValue[1], InValue[2]);
	}

	FLinearColor ConvertColor(const pxr::GfVec4f& InValue)
	{
		return FLinearColor(InValue[0], InValue[1], InValue[2], InValue[3]);
	}

	FVector2D ConvertVector(const pxr::GfVec2h& InValue)
	{
		return FVector2D(InValue[0], InValue[1]);
	}

	FVector2D ConvertVector(const pxr::GfVec2f& InValue)
	{
		return FVector2D(InValue[0], InValue[1]);
	}

	FVector2D ConvertVector(const pxr::GfVec2d& InValue)
	{
		return FVector2D(InValue[0], InValue[1]);
	}

	FIntPoint ConvertVector(const pxr::GfVec2i& InValue)
	{
		return FIntPoint(InValue[0], InValue[1]);
	}

	FVector ConvertVector(const pxr::GfVec3h& InValue)
	{
		return FVector(InValue[0], InValue[1], InValue[2]);
	}

	FVector ConvertVector(const pxr::GfVec3f& InValue)
	{
		return FVector(InValue[0], InValue[1], InValue[2]);
	}

	FVector ConvertVector(const pxr::GfVec3d& InValue)
	{
		return FVector(InValue[0], InValue[1], InValue[2]);
	}

	FIntVector ConvertVector(const pxr::GfVec3i& InValue)
	{
		return FIntVector(InValue[0], InValue[1], InValue[2]);
	}

	template<typename VecType>
	FVector ConvertVectorInner(const FUsdStageInfo& StageInfo, const VecType& InValue)
	{
		FVector Value = ConvertVector(InValue);

		const double UEMetersPerUnit = 0.01;
		if (!FMath::IsNearlyEqual(StageInfo.MetersPerUnit, UEMetersPerUnit))
		{
			Value *= StageInfo.MetersPerUnit / UEMetersPerUnit;
		}

		const bool bIsZUp = (StageInfo.UpAxis == EUsdUpAxis::ZAxis);

		if (bIsZUp)
		{
			Value.Y = -Value.Y;
		}
		else
		{
			Swap(Value.Y, Value.Z);
		}

		return Value;
	}

	FVector ConvertVector(const FUsdStageInfo& StageInfo, const pxr::GfVec3h& InValue)
	{
		return ConvertVectorInner(StageInfo, InValue);
	}

	FVector ConvertVector(const FUsdStageInfo& StageInfo, const pxr::GfVec3f& InValue)
	{
		return ConvertVectorInner(StageInfo, InValue);
	}

	FVector ConvertVector(const FUsdStageInfo& StageInfo, const pxr::GfVec3d& InValue)
	{
		return ConvertVectorInner(StageInfo, InValue);
	}

	FVector4 ConvertVector(const pxr::GfVec4h& InValue)
	{
		return FVector4(InValue[0], InValue[1], InValue[2], InValue[3]);
	}

	FVector4 ConvertVector(const pxr::GfVec4f& InValue)
	{
		return FVector4(InValue[0], InValue[1], InValue[2], InValue[3]);
	}

	FVector4 ConvertVector(const pxr::GfVec4d& InValue)
	{
		return FVector4(InValue[0], InValue[1], InValue[2], InValue[3]);
	}

	FIntVector4 ConvertVector(const pxr::GfVec4i& InValue)
	{
		return FIntVector4(InValue[0], InValue[1], InValue[2], InValue[3]);
	}

	FMatrix2D ConvertMatrix(const pxr::GfMatrix2d& Matrix)
	{
		FMatrix2D Result;
		Result.Row0 = ConvertVector(Matrix.GetRow(0));
		Result.Row1 = ConvertVector(Matrix.GetRow(1));
		return Result;
	}

	FMatrix3D ConvertMatrix(const pxr::GfMatrix3d& Matrix)
	{
		FMatrix3D Result;
		Result.Row0 = ConvertVector(Matrix.GetRow(0));
		Result.Row1 = ConvertVector(Matrix.GetRow(1));
		Result.Row2 = ConvertVector(Matrix.GetRow(2));
		return Result;
	}

	FMatrix ConvertMatrix(const pxr::GfMatrix4d& Matrix)
	{
		FMatrix UnrealMatrix(
			FPlane(Matrix[0][0], Matrix[0][1], Matrix[0][2], Matrix[0][3]),
			FPlane(Matrix[1][0], Matrix[1][1], Matrix[1][2], Matrix[1][3]),
			FPlane(Matrix[2][0], Matrix[2][1], Matrix[2][2], Matrix[2][3]),
			FPlane(Matrix[3][0], Matrix[3][1], Matrix[3][2], Matrix[3][3])
		);

		return UnrealMatrix;
	}

	FTransform ConvertMatrix(const FUsdStageInfo& StageInfo, const pxr::GfMatrix4d& InMatrix)
	{
		FMatrix Matrix = ConvertMatrix(InMatrix);
		FTransform Transform(Matrix);

		Transform = UsdUtils::ConvertAxes(StageInfo.UpAxis == EUsdUpAxis::ZAxis, Transform);

		const double UEMetersPerUnit = 0.01;
		if (!FMath::IsNearlyEqual(StageInfo.MetersPerUnit, UEMetersPerUnit))
		{
			Transform.ScaleTranslation(StageInfo.MetersPerUnit / UEMetersPerUnit);
		}

		return Transform;
	}

	FQuat ConvertQuat(const pxr::GfQuath& InValue)
	{
		const pxr::GfVec3h& Imaginary = InValue.GetImaginary();
		return FQuat{Imaginary[0], Imaginary[1], Imaginary[2], InValue.GetReal()};
	}

	FQuat ConvertQuat(const pxr::GfQuatf& InValue)
	{
		const pxr::GfVec3f& Imaginary = InValue.GetImaginary();
		return FQuat{Imaginary[0], Imaginary[1], Imaginary[2], InValue.GetReal()};
	}

	FQuat ConvertQuat(const pxr::GfQuatd& InValue)
	{
		const pxr::GfVec3d& Imaginary = InValue.GetImaginary();
		return FQuat{Imaginary[0], Imaginary[1], Imaginary[2], InValue.GetReal()};
	}

	float ConvertDistance(const FUsdStageInfo& StageInfo, float Value)
	{
		const double UEMetersPerUnit = 0.01;
		if (!FMath::IsNearlyEqual(StageInfo.MetersPerUnit, UEMetersPerUnit))
		{
			Value *= StageInfo.MetersPerUnit / UEMetersPerUnit;
		}

		return Value;
	}
}	 // namespace UsdToUnreal

namespace UnrealToUsd
{
	TUsdStore<std::string> ConvertString(const TCHAR* InString)
	{
		return MakeUsdStore<std::string>(TCHAR_TO_UTF8(InString));
	}

	TUsdStore<pxr::SdfPath> ConvertPath(const TCHAR* InString)
	{
		return MakeUsdStore<pxr::SdfPath>(TCHAR_TO_UTF8(InString));
	}

	TUsdStore<std::string> ConvertName(const FName& InName)
	{
		return MakeUsdStore<std::string>(TCHAR_TO_UTF8(*InName.ToString()));
	}

	TUsdStore<pxr::TfToken> ConvertToken(const TCHAR* InString)
	{
		return MakeUsdStore<pxr::TfToken>(TCHAR_TO_UTF8(InString));
	}

	pxr::GfVec4f ConvertColor(const FLinearColor& InValue)
	{
		return pxr::GfVec4f(InValue.R, InValue.G, InValue.B, InValue.A);
	}

	pxr::GfVec4f ConvertColor(const FColor& InValue)
	{
		return ConvertColor(FLinearColor(InValue));
	}

	pxr::GfVec2h ConvertVectorHalf(const FVector2D& InValue)
	{
		return pxr::GfVec2h(InValue[0], InValue[1]);
	}

	pxr::GfVec2f ConvertVectorFloat(const FVector2D& InValue)
	{
		return pxr::GfVec2f(InValue[0], InValue[1]);
	}

	pxr::GfVec2d ConvertVectorDouble(const FVector2D& InValue)
	{
		return pxr::GfVec2d(InValue[0], InValue[1]);
	}

	pxr::GfVec2i ConvertVectorInt(const FIntPoint& InValue)
	{
		return pxr::GfVec2i(InValue[0], InValue[1]);
	}

	pxr::GfVec3h ConvertVectorHalf(const FVector& InValue)
	{
		return pxr::GfVec3h(InValue[0], InValue[1], InValue[2]);
	}

	pxr::GfVec3f ConvertVectorFloat(const FVector& InValue)
	{
		return pxr::GfVec3f(InValue[0], InValue[1], InValue[2]);
	}

	pxr::GfVec3d ConvertVectorDouble(const FVector& InValue)
	{
		return pxr::GfVec3d(InValue[0], InValue[1], InValue[2]);
	}

	pxr::GfVec3i ConvertVectorInt(const FIntVector& InValue)
	{
		return pxr::GfVec3i(InValue[0], InValue[1], InValue[2]);
	}

	template<typename T>
	T ConvertVectorInner(const FUsdStageInfo& StageInfo, T Value)
	{
		const double UEMetersPerUnit = 0.01;
		if (!FMath::IsNearlyEqual(StageInfo.MetersPerUnit, UEMetersPerUnit) && !FMath::IsNearlyZero(StageInfo.MetersPerUnit))
		{
			Value *= UEMetersPerUnit / StageInfo.MetersPerUnit;
		}

		const bool bIsZUp = (StageInfo.UpAxis == EUsdUpAxis::ZAxis);

		if (bIsZUp)
		{
			Value[1] = -Value[1];
		}
		else
		{
			Swap(Value[1], Value[2]);
		}

		return Value;
	}

	pxr::GfVec3h ConvertVectorHalf(const FUsdStageInfo& StageInfo, const FVector& InValue)
	{
		return ConvertVectorInner(StageInfo, ConvertVectorHalf(InValue));
	}

	pxr::GfVec3f ConvertVectorFloat(const FUsdStageInfo& StageInfo, const FVector& InValue)
	{
		return ConvertVectorInner(StageInfo, ConvertVectorFloat(InValue));
	}

	pxr::GfVec3d ConvertVectorDouble(const FUsdStageInfo& StageInfo, const FVector& InValue)
	{
		return ConvertVectorInner(StageInfo, ConvertVectorDouble(InValue));
	}

	pxr::GfVec4h ConvertVectorHalf(const FVector4& InValue)
	{
		return pxr::GfVec4h(InValue[0], InValue[1], InValue[2], InValue[3]);
	}

	pxr::GfVec4f ConvertVectorFloat(const FVector4& InValue)
	{
		return pxr::GfVec4f(InValue[0], InValue[1], InValue[2], InValue[3]);
	}

	pxr::GfVec4d ConvertVectorDouble(const FVector4& InValue)
	{
		return pxr::GfVec4d(InValue[0], InValue[1], InValue[2], InValue[3]);
	}

	pxr::GfVec4i ConvertVectorInt(const FIntVector4& InValue)
	{
		return pxr::GfVec4i(InValue[0], InValue[1], InValue[2], InValue[3]);
	}

	pxr::GfMatrix2d ConvertMatrix(const FMatrix2D& Matrix)
	{
		pxr::GfMatrix2d UsdMatrix;
		UsdMatrix.SetRow(0, ConvertVectorDouble(Matrix.Row0));
		UsdMatrix.SetRow(1, ConvertVectorDouble(Matrix.Row1));
		return UsdMatrix;
	}

	pxr::GfMatrix3d ConvertMatrix(const FMatrix3D& Matrix)
	{
		pxr::GfMatrix3d UsdMatrix;
		UsdMatrix.SetRow(0, ConvertVectorDouble(Matrix.Row0));
		UsdMatrix.SetRow(1, ConvertVectorDouble(Matrix.Row1));
		UsdMatrix.SetRow(2, ConvertVectorDouble(Matrix.Row2));
		return UsdMatrix;
	}

	pxr::GfMatrix4d ConvertMatrix(const FMatrix& Matrix)
	{
		pxr::GfMatrix4d UsdMatrix(
			Matrix.M[0][0],
			Matrix.M[0][1],
			Matrix.M[0][2],
			Matrix.M[0][3],
			Matrix.M[1][0],
			Matrix.M[1][1],
			Matrix.M[1][2],
			Matrix.M[1][3],
			Matrix.M[2][0],
			Matrix.M[2][1],
			Matrix.M[2][2],
			Matrix.M[2][3],
			Matrix.M[3][0],
			Matrix.M[3][1],
			Matrix.M[3][2],
			Matrix.M[3][3]
		);

		return UsdMatrix;
	}

	pxr::GfQuath ConvertQuatHalf(const FQuat& InValue)
	{
		return pxr::GfQuath{
			static_cast<pxr::GfHalf>(InValue.W),
			static_cast<pxr::GfHalf>(InValue.X),
			static_cast<pxr::GfHalf>(InValue.Y),
			static_cast<pxr::GfHalf>(InValue.Z)};
	}

	pxr::GfQuatf ConvertQuatFloat(const FQuat& InValue)
	{
		return pxr::GfQuatf{
			static_cast<float>(InValue.W),
			static_cast<float>(InValue.X),
			static_cast<float>(InValue.Y),
			static_cast<float>(InValue.Z)};
	}

	pxr::GfQuatd ConvertQuatDouble(const FQuat& InValue)
	{
		return pxr::GfQuatd{InValue.W, InValue.X, InValue.Y, InValue.Z};
	}

	pxr::GfMatrix4d ConvertTransform(const FUsdStageInfo& StageInfo, const FTransform& Transform)
	{
		FTransform TransformInUsdSpace = UsdUtils::ConvertTransformToUsdSpace(StageInfo, Transform);

		return ConvertMatrix(TransformInUsdSpace.ToMatrixWithScale());
	}

	float ConvertDistance(const FUsdStageInfo& StageInfo, float Value)
	{
		const double UEMetersPerUnit = 0.01;
		if (!FMath::IsNearlyEqual(StageInfo.MetersPerUnit, UEMetersPerUnit))
		{
			Value *= UEMetersPerUnit / StageInfo.MetersPerUnit;
		}

		return Value;
	}

	TUsdStore<pxr::VtArray<pxr::GfVec3f>> ConvertBounds(const FUsdStageInfo& StageInfo, FBox Bounds)
	{
		TUsdStore<pxr::VtArray<pxr::GfVec3f>> Result;
		if (!Bounds.IsValid)
		{
			return Result;
		}

		pxr::GfVec3f ConvertedMin = UnrealToUsd::ConvertVectorFloat(StageInfo, Bounds.Min);
		pxr::GfVec3f ConvertedMax = UnrealToUsd::ConvertVectorFloat(StageInfo, Bounds.Max);

		// Because of the up axis conversion, we may have e.g. ConvertedMin.y > ConvertedMax.y, so here we
		// fix that up
		pxr::GfVec3f ActualMin = {
			FMath::Min(ConvertedMin[0], ConvertedMax[0]),
			FMath::Min(ConvertedMin[1], ConvertedMax[1]),
			FMath::Min(ConvertedMin[2], ConvertedMax[2])};
		pxr::GfVec3f ActualMax = {
			FMath::Max(ConvertedMin[0], ConvertedMax[0]),
			FMath::Max(ConvertedMin[1], ConvertedMax[1]),
			FMath::Max(ConvertedMin[2], ConvertedMax[2])};

		pxr::VtArray<pxr::GfVec3f>& ResultInner = Result.Get();
		ResultInner = {ActualMin, ActualMax};

		return Result;
	}
}	 // namespace UnrealToUsd

#endif	  // #if USE_USD_SDK
