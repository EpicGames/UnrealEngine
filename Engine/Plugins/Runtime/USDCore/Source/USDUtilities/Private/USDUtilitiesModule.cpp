// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDUtilitiesModule.h"

#include "USDErrorUtils.h"
#include "USDMemory.h"
#include "USDTypesConversion.h"

#include "Materials/MaterialInterface.h"
#include "Modules/ModuleManager.h"

#if WITH_EDITOR
#include "MessageLogModule.h"
#endif	  // WITH_EDITOR

#define LOCTEXT_NAMESPACE "UsdUtilitiesModule"

class FUsdUtilitiesModule : public IUsdUtilitiesModule
{
	FDelegateHandle OnProjectSettingsChangedHandle;

public:
	virtual void StartupModule() override
	{
		UUsdProjectSettings* ProjectSettings = GetMutableDefault<UUsdProjectSettings>();
		if (ProjectSettings->bLogUsdSdkErrors)
		{
			FUsdLogManager::RegisterDiagnosticDelegate();
		}

#if WITH_EDITOR
		LLM_SCOPE_BYTAG(Usd);

		// These strings are used for the CHECK_MATERIAL macro, but aren't inline because the localization system
		// doesn't know what to do with the LOCTEXT inside the other macro
		const static FText TranslucentMismatchMessage = LOCTEXT(
			"TranslucentMaterialMismatch",
			"Material '{0}' {1} but the property {2}! Make sure to assign translucent materials only to the translucent reference material properties!"
		);
		const static FText TwoSidedMistmachMessage = LOCTEXT(
			"TwoSidedMaterialMismatch",
			"Material '{0}' {1} but the property {2}! Make sure to assign two-sided materials only to the two-sided reference material properties!"
		);
		const static FText IsTwoSidedText = LOCTEXT("IsTwoSidedText", "is two-sided");
		const static FText IsOneSidedText = LOCTEXT("IsOneSidedText", "is one-sided");
		const static FText IsTranslucentText = LOCTEXT("IsTranslucentText", "is translucent");
		const static FText IsNotTranslucentText = LOCTEXT("IsNotTranslucentText", "is not translucent");

		OnProjectSettingsChangedHandle = ProjectSettings->OnSettingChanged().AddLambda(
			[](UObject* SettingsObject, struct FPropertyChangedEvent& PropertyChangedEvent)
			{
				UUsdProjectSettings* UsdSettings = Cast<UUsdProjectSettings>(SettingsObject);
				if (!UsdSettings)
				{
					return;
				}

				if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UUsdProjectSettings, bLogUsdSdkErrors))
				{
					if (UsdSettings->bLogUsdSdkErrors)
					{
						FUsdLogManager::RegisterDiagnosticDelegate();
					}
					else
					{
						FUsdLogManager::UnregisterDiagnosticDelegate();
					}
				}

			// Produce some immediate warning in case the user adds a an invalid material for one of the reference material properties
#define CHECK_MATERIAL(ReferenceType, bShouldBeTwoSided, bShouldBeTranslucent)                                      	   \
				if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UUsdProjectSettings, ReferenceType)) \
				{                                                                                                          \
					if (UMaterialInterface* Material = Cast<UMaterialInterface>(UsdSettings->ReferenceType.TryLoad()))     \
					{                                                                                                      \
						FScopedUsdMessageLog ScopedMessageLog;                                                             \
																														   \
						bool bRevertMaterial = false;                                                                      \
																														   \
						const bool bIsTranslucent = Material->GetBlendMode() == EBlendMode::BLEND_Translucent              \
													|| Material->GetBlendMode() == EBlendMode::BLEND_Additive              \
													|| Material->GetBlendMode() == EBlendMode::BLEND_Modulate;             \
						if (bIsTranslucent != bShouldBeTranslucent)                                                        \
						{                                                                                                  \
							USD_LOG_USERWARNING(FText::Format(                                                             \
								TranslucentMismatchMessage,                                                                \
								FText::FromString(Material->GetPathName()),                                                \
								bIsTranslucent ? IsTranslucentText : IsNotTranslucentText,                                 \
								bShouldBeTranslucent ? IsTranslucentText : IsNotTranslucentText                            \
							));                                                                                            \
							bRevertMaterial = true;                                                                        \
						}                                                                                                  \
																														   \
						if (Material->IsTwoSided() != bShouldBeTwoSided)                                                   \
						{                                                                                                  \
							USD_LOG_USERWARNING(FText::Format(                                                             \
								TwoSidedMistmachMessage,                                                                   \
								FText::FromString(Material->GetPathName()),                                                \
								Material->IsTwoSided() ? IsTwoSidedText : IsOneSidedText,                                  \
								bShouldBeTwoSided ? IsTwoSidedText : IsOneSidedText                                        \
							));                                                                                            \
							bRevertMaterial = true;                                                                        \
						}                                                                                                  \
																														   \
						if (bRevertMaterial)                                                                               \
						{                                                                                                  \
							UsdSettings->ReferenceType = UsdDefaultReferenceMaterialPaths::ReferenceType;                  \
							UsdSettings->SaveConfig();                                                                     \
						}                                                                                                  \
					}                                                                                                      \
				}
				CHECK_MATERIAL(ReferencePreviewSurfaceMaterial, false, false);
				CHECK_MATERIAL(ReferencePreviewSurfaceTranslucentMaterial, false, true);
				CHECK_MATERIAL(ReferencePreviewSurfaceTwoSidedMaterial, true, false);
				CHECK_MATERIAL(ReferencePreviewSurfaceTranslucentTwoSidedMaterial, true, true);
				CHECK_MATERIAL(ReferencePreviewSurfaceVTMaterial, false, false);
				CHECK_MATERIAL(ReferencePreviewSurfaceTranslucentVTMaterial, false, true);
				CHECK_MATERIAL(ReferencePreviewSurfaceTwoSidedVTMaterial, true, false);
				CHECK_MATERIAL(ReferencePreviewSurfaceTranslucentTwoSidedVTMaterial, true, true);
				CHECK_MATERIAL(ReferenceDisplayColorMaterial, false, false);
				CHECK_MATERIAL(ReferenceDisplayColorAndOpacityMaterial, false, true);
				CHECK_MATERIAL(ReferenceDisplayColorTwoSidedMaterial, true, false);
				CHECK_MATERIAL(ReferenceDisplayColorAndOpacityTwoSidedMaterial, true, true);
#undef CHECK_MATERIAL
			}
		);

		FMessageLogInitializationOptions InitOptions;
		FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>(TEXT("MessageLog"));
		MessageLogModule.RegisterLogListing(TEXT("USD"), NSLOCTEXT("USDUtilitiesModule", "USDLogListing", "USD"), InitOptions);
#endif	  // WITH_EDITOR
	}

	virtual void ShutdownModule() override
	{
		FUsdLogManager::UnregisterDiagnosticDelegate();

#if WITH_EDITOR
		// We can't query default objects during engine exit
		if (UObjectInitialized())
		{
			UUsdProjectSettings* ProjectSettings = GetMutableDefault<UUsdProjectSettings>();
			if (ProjectSettings)
			{
				ProjectSettings->OnSettingChanged().Remove(OnProjectSettingsChangedHandle);
			}
		}

		FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>(TEXT("MessageLog"));
		MessageLogModule.UnregisterLogListing(TEXT("USD"));
#endif	  // WITH_EDITOR
	}
};

IMPLEMENT_MODULE_USD(FUsdUtilitiesModule, USDUtilities);

#undef LOCTEXT_NAMESPACE