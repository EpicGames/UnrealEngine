// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDValueConversion.h"

#include "USDConversionUtils.h"
#include "USDErrorUtils.h"
#include "USDMemory.h"
#include "USDTypesConversion.h"
#include "UsdWrappers/UsdStage.h"
#include "UsdWrappers/VtValue.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/base/tf/stringUtils.h"
#include "pxr/base/vt/value.h"
#include "pxr/usd/sdf/payload.h"
#include "pxr/usd/sdf/reference.h"
#include "pxr/usd/sdf/schema.h"
#include "pxr/usd/sdf/types.h"
#include "USDIncludesEnd.h"
#endif	  // USE_USD_SDK

namespace UE::USDValueConversion::Private
{
	const static FString TokenListStringStart = TEXT("SdfTokenListOp(Explicit Items: [");
	const static FString TokenListStringEmpty = TEXT("SdfTokenListOp(Explicit Items: [])");
	const static char SingleQuoteChar = '\'';
	const static char DoubleQuoteChar = '\"';
	const static char EscapeChar = '\\';
	const static char AssetDelimiterChar = '@';
	const static TCHAR DoubleQuoteTCHAR = '\"';

	// Use a control character here because USD shouldn't normally let this be used for actual SdfPaths
	const static FString ReferencePayloadDelimiter = TEXT("\x1c");

#if USE_USD_SDK
	// Functions instead of just variables to make sure we don't run into any bizarre issues with these static variables being
	// initialized before the TfTypes are registered (not sure if that is possible, but we don't need to find out)
	template<typename T>
	const FString& GetSdfTypeName()
	{
		const static FString TypeName = *UsdToUnreal::ConvertString(pxr::TfType::Find<T>().GetTypeName());
		return TypeName;
	}

	enum EListOpListIndex
	{
		Explicit = 0,
		Added,
		Deleted,
		Ordered,
		Prepended,
		Appended,

		Num
	};
	static_assert((int)EListOpListIndex::Explicit == (int)pxr::SdfListOpType::SdfListOpTypeExplicit);
	static_assert((int)EListOpListIndex::Added == (int)pxr::SdfListOpType::SdfListOpTypeAdded);
	static_assert((int)EListOpListIndex::Deleted == (int)pxr::SdfListOpType::SdfListOpTypeDeleted);
	static_assert((int)EListOpListIndex::Ordered == (int)pxr::SdfListOpType::SdfListOpTypeOrdered);
	static_assert((int)EListOpListIndex::Prepended == (int)pxr::SdfListOpType::SdfListOpTypePrepended);
	static_assert((int)EListOpListIndex::Appended == (int)pxr::SdfListOpType::SdfListOpTypeAppended);
#endif	  // USE_USD_SDK
}	 // namespace UE::USDValueConversion::Private

#if USE_USD_SDK
namespace UsdToUnrealImpl
{
	/**
	 * Allows us to use the same lambda to convert either VtArray<USDType> or USDType
	 * Func should have a signature like this:
	 * []( const USDType& Val ) -> UsdUtils::FConvertedVtValueEntry {}
	 */
	template<typename UEType, typename USDType, typename Func>
	void ConvertInner(const pxr::VtValue& InValue, UsdUtils::FConvertedVtValue& OutValue, Func Function)
	{
		if (InValue.IsArrayValued())
		{
			OutValue.Entries.Reset(InValue.GetArraySize());

			for (const USDType& Val : InValue.UncheckedGet<pxr::VtArray<USDType>>())
			{
				OutValue.Entries.Add(Function(Val));
			}
		}
		else
		{
			const USDType& Val = InValue.UncheckedGet<USDType>();

			OutValue.Entries = {Function(Val)};
		}
	}

	// Bool, Uchar, Int, Uint, Int64, Uint64, Half (can cast to float), Float, Double
	template<typename UEType, typename USDType>
	void ConvertSimpleValue(const pxr::VtValue& InValue, UsdUtils::FConvertedVtValue& OutValue)
	{
		ConvertInner<UEType, USDType>(
			InValue,
			OutValue,
			[](const USDType& Val) -> UsdUtils::FConvertedVtValueEntry
			{
				return {UsdUtils::FConvertedVtValueComponent(TInPlaceType<UEType>(), static_cast<UEType>(Val))};
			}
		);
	}

	// TimeCode
	template<>
	void ConvertSimpleValue<double, pxr::SdfTimeCode>(const pxr::VtValue& InValue, UsdUtils::FConvertedVtValue& OutValue)
	{
		ConvertInner<double, pxr::SdfTimeCode>(
			InValue,
			OutValue,
			[](const pxr::SdfTimeCode& Val) -> UsdUtils::FConvertedVtValueEntry
			{
				return {UsdUtils::FConvertedVtValueComponent(TInPlaceType<double>(), Val.GetValue())};
			}
		);
	}

	// String
	template<>
	void ConvertSimpleValue<FString, std::string>(const pxr::VtValue& InValue, UsdUtils::FConvertedVtValue& OutValue)
	{
		ConvertInner<FString, std::string>(
			InValue,
			OutValue,
			[](const std::string& Val) -> UsdUtils::FConvertedVtValueEntry
			{
				return {UsdUtils::FConvertedVtValueComponent(TInPlaceType<FString>(), UsdToUnreal::ConvertString(Val))};
			}
		);
	}

	// Token
	template<>
	void ConvertSimpleValue<FString, pxr::TfToken>(const pxr::VtValue& InValue, UsdUtils::FConvertedVtValue& OutValue)
	{
		ConvertInner<FString, pxr::TfToken>(
			InValue,
			OutValue,
			[](const pxr::TfToken& Val) -> UsdUtils::FConvertedVtValueEntry
			{
				return {UsdUtils::FConvertedVtValueComponent(TInPlaceType<FString>(), UsdToUnreal::ConvertToken(Val))};
			}
		);
	}

	// Asset
	template<>
	void ConvertSimpleValue<FString, pxr::SdfAssetPath>(const pxr::VtValue& InValue, UsdUtils::FConvertedVtValue& OutValue)
	{
		ConvertInner<FString, pxr::SdfAssetPath>(
			InValue,
			OutValue,
			[](const pxr::SdfAssetPath& Val) -> UsdUtils::FConvertedVtValueEntry
			{
				return {UsdUtils::FConvertedVtValueComponent(TInPlaceType<FString>(), UsdToUnreal::ConvertString(Val.GetAssetPath()))};
			}
		);
	}

	// Matrix2d, Matrix3d, Matrix4d	(always double)
	template<typename USDType>
	void ConvertMatrixValue(const pxr::VtValue& InValue, UsdUtils::FConvertedVtValue& OutValue)
	{
		ConvertInner<double, USDType>(
			InValue,
			OutValue,
			[](const USDType& Val) -> UsdUtils::FConvertedVtValueEntry
			{
				const int32 NumElements = USDType::numRows * USDType::numColumns;

				UsdUtils::FConvertedVtValueEntry Entry;
				Entry.Reserve(NumElements);

				const double* MatrixArray = Val.GetArray();
				for (int32 Index = 0; Index < NumElements; ++Index)
				{
					Entry.Emplace(TInPlaceType<double>(), MatrixArray[Index]);
				}

				return Entry;
			}
		);
	}

	// Quath, Quatf, Quatd
	template<typename UEType, typename USDType>
	void ConvertQuatValue(const pxr::VtValue& InValue, UsdUtils::FConvertedVtValue& OutValue)
	{
		ConvertInner<UEType, USDType>(
			InValue,
			OutValue,
			[](const USDType& Val) -> UsdUtils::FConvertedVtValueEntry
			{
				// Auto here because this is the vec of the corresponding type (e.g. Quath -> Vec3h)
				const auto& Img = Val.GetImaginary();
				double Real = Val.GetReal();

				return {
					UsdUtils::FConvertedVtValueComponent(TInPlaceType<UEType>(), Img[0]),
					UsdUtils::FConvertedVtValueComponent(TInPlaceType<UEType>(), Img[1]),
					UsdUtils::FConvertedVtValueComponent(TInPlaceType<UEType>(), Img[2]),
					UsdUtils::FConvertedVtValueComponent(TInPlaceType<UEType>(), Real),
				};
			}
		);
	}

	// Double2, Float2, Half2, Int2
	// Double3, Float3, Half3, Int3
	// Double4, Float4, Half4, Int4
	template<typename UEType, typename USDType>
	void ConvertVecValue(const pxr::VtValue& InValue, UsdUtils::FConvertedVtValue& OutValue)
	{
		ConvertInner<UEType, USDType>(
			InValue,
			OutValue,
			[](const USDType& Val) -> UsdUtils::FConvertedVtValueEntry
			{
				UsdUtils::FConvertedVtValueEntry Entry;
				Entry.Reset(USDType::dimension);

				for (int32 Index = 0; Index < USDType::dimension; ++Index)
				{
					Entry.Emplace(TInPlaceType<UEType>(), Val[Index]);
				}

				return Entry;
			}
		);
	}

	template<typename UEType, typename USDType, typename ConverterFunc>
	void ConvertListOpValue(const pxr::VtValue& InValue, UsdUtils::FConvertedVtValue& OutValue, ConverterFunc Converter)
	{
		using namespace pxr;
		using namespace UsdUtils;
		using namespace UE::USDValueConversion::Private;

		OutValue.bIsArrayValued = true;
		OutValue.Entries.SetNum(EListOpListIndex::Num);

		const SdfListOp<USDType>& Val = InValue.UncheckedGet<SdfListOp<USDType>>();

		// Explicit ops only have explicit items
		if (Val.IsExplicit())
		{
			TArray<FConvertedVtValueComponent>& ExplicitItems = OutValue.Entries[EListOpListIndex::Explicit];

			for (const USDType& Item : Val.GetExplicitItems())
			{
				ExplicitItems.Add(UsdUtils::FConvertedVtValueComponent(TInPlaceType<UEType>(), Converter(Item)));
			}
		}
		// Non explicit ops may have items in all other lists
		else
		{
			TArray<FConvertedVtValueComponent>& AddedItems = OutValue.Entries[EListOpListIndex::Added];
			TArray<FConvertedVtValueComponent>& PrependedItems = OutValue.Entries[EListOpListIndex::Prepended];
			TArray<FConvertedVtValueComponent>& AppendedItems = OutValue.Entries[EListOpListIndex::Appended];
			TArray<FConvertedVtValueComponent>& DeletedItems = OutValue.Entries[EListOpListIndex::Deleted];
			TArray<FConvertedVtValueComponent>& OrderedItems = OutValue.Entries[EListOpListIndex::Ordered];

			for (const USDType& Item : Val.GetAddedItems())
			{
				FString Converted = Converter(Item);
				AddedItems.Add(UsdUtils::FConvertedVtValueComponent(TInPlaceType<UEType>(), Converter(Item)));
			}
			for (const USDType& Item : Val.GetPrependedItems())
			{
				FString Converted = Converter(Item);
				PrependedItems.Add(UsdUtils::FConvertedVtValueComponent(TInPlaceType<UEType>(), Converter(Item)));
			}
			for (const USDType& Item : Val.GetAppendedItems())
			{
				FString Converted = Converter(Item);
				AppendedItems.Add(UsdUtils::FConvertedVtValueComponent(TInPlaceType<UEType>(), Converter(Item)));
			}
			for (const USDType& Item : Val.GetDeletedItems())
			{
				FString Converted = Converter(Item);
				DeletedItems.Add(UsdUtils::FConvertedVtValueComponent(TInPlaceType<UEType>(), Converter(Item)));
			}
			for (const USDType& Item : Val.GetOrderedItems())
			{
				FString Converted = Converter(Item);
				OrderedItems.Add(UsdUtils::FConvertedVtValueComponent(TInPlaceType<UEType>(), Converter(Item)));
			}
		}
	}

	template<typename SdfRefOrPayload>
	FString StringifySdfReferenceOrPayload(const SdfRefOrPayload& Reference)
	{
		using namespace UE::USDValueConversion::Private;

		FString AssetPath = UsdToUnreal::ConvertString(Reference.GetAssetPath());
		FString PrimPath = UsdToUnreal::ConvertPath(Reference.GetPrimPath());
		double Offset = Reference.GetLayerOffset().GetOffset();
		double Scale = Reference.GetLayerOffset().GetScale();

		// TODO: Apparently there's a full CustomData VtDictionary inside the SdfReference that we should probably also stringify.
		// This doesn't seem to be used that often (and we never author it anyway), so we'll skip it for now

		return FString::Printf(
			TEXT("%s%s%s%s%f%s%f"),
			*AssetPath,
			*ReferencePayloadDelimiter,
			*PrimPath,
			*ReferencePayloadDelimiter,
			Offset,
			*ReferencePayloadDelimiter,
			Scale
		);
	}
}	 // namespace UsdToUnrealImpl
#endif	  // USE_USD_SDK

namespace UsdToUnreal
{
	bool ConvertValue(const UE::FVtValue& InValue, UsdUtils::FConvertedVtValue& OutValue)
	{
#if USE_USD_SDK
		using namespace pxr;
		using namespace UsdUtils;
		using namespace UsdToUnrealImpl;
		using namespace UE::USDValueConversion::Private;

		OutValue.SourceType = UsdUtils::EUsdBasicDataTypes::None;
		OutValue.Entries.Reset();
		OutValue.bIsArrayValued = false;
		OutValue.bIsEmpty = true;

		// We consider a success returning an empty value if our input value was also empty
		if (InValue.IsEmpty())
		{
			return true;
		}

		OutValue.bIsEmpty = false;
		OutValue.bIsArrayValued = InValue.IsArrayValued();

		const VtValue& UsdValue = InValue.GetUsdValue();
		const TfType& UnderlyingType = UsdValue.GetType();

#pragma push_macro("CHECK_TYPE")
#define CHECK_TYPE(T) UnderlyingType.IsA<T>() || UnderlyingType.IsA<VtArray<T>>()

		if (CHECK_TYPE(bool))
		{
			OutValue.SourceType = EUsdBasicDataTypes::Bool;
			ConvertSimpleValue<bool, bool>(UsdValue, OutValue);
		}
		else if (CHECK_TYPE(uint8_t))
		{
			OutValue.SourceType = EUsdBasicDataTypes::Uchar;
			ConvertSimpleValue<uint8, uint8_t>(UsdValue, OutValue);
		}
		else if (CHECK_TYPE(int32_t))
		{
			OutValue.SourceType = EUsdBasicDataTypes::Int;
			ConvertSimpleValue<int32, int32_t>(UsdValue, OutValue);
		}
		else if (CHECK_TYPE(uint32_t))
		{
			OutValue.SourceType = EUsdBasicDataTypes::Uint;
			ConvertSimpleValue<uint32, uint32_t>(UsdValue, OutValue);
		}
		else if (CHECK_TYPE(int64_t))
		{
			OutValue.SourceType = EUsdBasicDataTypes::Int64;
			ConvertSimpleValue<int64, int64_t>(UsdValue, OutValue);
		}
		else if (CHECK_TYPE(uint64_t))
		{
			OutValue.SourceType = EUsdBasicDataTypes::Uint64;
			ConvertSimpleValue<uint64, uint64_t>(UsdValue, OutValue);
		}
		else if (CHECK_TYPE(GfHalf))
		{
			OutValue.SourceType = EUsdBasicDataTypes::Half;
			ConvertSimpleValue<float, GfHalf>(UsdValue, OutValue);
		}
		else if (CHECK_TYPE(float))
		{
			OutValue.SourceType = EUsdBasicDataTypes::Float;
			ConvertSimpleValue<float, float>(UsdValue, OutValue);
		}
		else if (CHECK_TYPE(double))
		{
			OutValue.SourceType = EUsdBasicDataTypes::Double;
			ConvertSimpleValue<double, double>(UsdValue, OutValue);
		}
		else if (CHECK_TYPE(SdfTimeCode))
		{
			OutValue.SourceType = EUsdBasicDataTypes::Timecode;
			ConvertSimpleValue<double, SdfTimeCode>(UsdValue, OutValue);
		}
		else if (CHECK_TYPE(std::string))
		{
			OutValue.SourceType = EUsdBasicDataTypes::String;
			ConvertSimpleValue<FString, std::string>(UsdValue, OutValue);
		}
		else if (CHECK_TYPE(TfToken))
		{
			OutValue.SourceType = EUsdBasicDataTypes::Token;
			ConvertSimpleValue<FString, TfToken>(UsdValue, OutValue);
		}
		else if (CHECK_TYPE(SdfAssetPath))
		{
			OutValue.SourceType = EUsdBasicDataTypes::Asset;
			ConvertSimpleValue<FString, SdfAssetPath>(UsdValue, OutValue);
		}
		else if (CHECK_TYPE(GfMatrix2d))
		{
			OutValue.SourceType = EUsdBasicDataTypes::Matrix2d;
			ConvertMatrixValue<GfMatrix2d>(UsdValue, OutValue);
		}
		else if (CHECK_TYPE(GfMatrix3d))
		{
			OutValue.SourceType = EUsdBasicDataTypes::Matrix3d;
			ConvertMatrixValue<GfMatrix3d>(UsdValue, OutValue);
		}
		else if (CHECK_TYPE(GfMatrix4d))
		{
			OutValue.SourceType = EUsdBasicDataTypes::Matrix4d;
			ConvertMatrixValue<GfMatrix4d>(UsdValue, OutValue);
		}
		else if (CHECK_TYPE(GfQuatd))
		{
			OutValue.SourceType = EUsdBasicDataTypes::Quatd;
			ConvertQuatValue<double, GfQuatd>(UsdValue, OutValue);
		}
		else if (CHECK_TYPE(GfQuatf))
		{
			OutValue.SourceType = EUsdBasicDataTypes::Quatf;
			ConvertQuatValue<float, GfQuatf>(UsdValue, OutValue);
		}
		else if (CHECK_TYPE(GfQuath))
		{
			OutValue.SourceType = EUsdBasicDataTypes::Quath;
			ConvertQuatValue<float, GfQuath>(UsdValue, OutValue);
		}
		else if (CHECK_TYPE(GfVec2d))
		{
			OutValue.SourceType = EUsdBasicDataTypes::Double2;
			ConvertVecValue<double, GfVec2d>(UsdValue, OutValue);
		}
		else if (CHECK_TYPE(GfVec2f))
		{
			OutValue.SourceType = EUsdBasicDataTypes::Float2;
			ConvertVecValue<float, GfVec2f>(UsdValue, OutValue);
		}
		else if (CHECK_TYPE(GfVec2h))
		{
			OutValue.SourceType = EUsdBasicDataTypes::Half2;
			ConvertVecValue<float, GfVec2h>(UsdValue, OutValue);
		}
		else if (CHECK_TYPE(GfVec2i))
		{
			OutValue.SourceType = EUsdBasicDataTypes::Int2;
			ConvertVecValue<int32, GfVec2i>(UsdValue, OutValue);
		}
		else if (CHECK_TYPE(GfVec3d))
		{
			OutValue.SourceType = EUsdBasicDataTypes::Double3;
			ConvertVecValue<double, GfVec3d>(UsdValue, OutValue);
		}
		else if (CHECK_TYPE(GfVec3f))
		{
			OutValue.SourceType = EUsdBasicDataTypes::Float3;
			ConvertVecValue<float, GfVec3f>(UsdValue, OutValue);
		}
		else if (CHECK_TYPE(GfVec3h))
		{
			OutValue.SourceType = EUsdBasicDataTypes::Half3;
			ConvertVecValue<float, GfVec3h>(UsdValue, OutValue);
		}
		else if (CHECK_TYPE(GfVec3i))
		{
			OutValue.SourceType = EUsdBasicDataTypes::Int3;
			ConvertVecValue<int32, GfVec3i>(UsdValue, OutValue);
		}
		else if (CHECK_TYPE(GfVec4d))
		{
			OutValue.SourceType = EUsdBasicDataTypes::Double4;
			ConvertVecValue<double, GfVec4d>(UsdValue, OutValue);
		}
		else if (CHECK_TYPE(GfVec4f))
		{
			OutValue.SourceType = EUsdBasicDataTypes::Float4;
			ConvertVecValue<float, GfVec4f>(UsdValue, OutValue);
		}
		else if (CHECK_TYPE(GfVec4h))
		{
			OutValue.SourceType = EUsdBasicDataTypes::Half4;
			ConvertVecValue<float, GfVec4h>(UsdValue, OutValue);
		}
		else if (CHECK_TYPE(GfVec4i))
		{
			OutValue.SourceType = EUsdBasicDataTypes::Int4;
			ConvertVecValue<int32, GfVec4i>(UsdValue, OutValue);
		}
		// This is the type used to store values for relationships. We'll basically treat it like a string array
		else if (UnderlyingType.IsA<SdfListOp<SdfPath>>())
		{
			OutValue.SourceType = EUsdBasicDataTypes::String;
			OutValue.ExtraTypeInfo = GetSdfTypeName<SdfListOp<SdfPath>>();

			ConvertListOpValue<FString, SdfPath, FString(const pxr::SdfPath&)>(	   //
				UsdValue,
				OutValue,
				UsdToUnreal::ConvertPath
			);
		}
		else if (UnderlyingType.IsA<SdfListOp<std::string>>())
		{
			OutValue.SourceType = EUsdBasicDataTypes::String;
			OutValue.ExtraTypeInfo = GetSdfTypeName<SdfListOp<std::string>>();

			ConvertListOpValue<FString, std::string, FString(const std::string&)>(	  //
				UsdValue,
				OutValue,
				UsdToUnreal::ConvertString
			);
		}
		else if (UnderlyingType.IsA<SdfListOp<SdfReference>>())
		{
			OutValue.SourceType = EUsdBasicDataTypes::String;
			OutValue.ExtraTypeInfo = GetSdfTypeName<SdfListOp<SdfReference>>();

			ConvertListOpValue<FString, SdfReference>(	  //
				UsdValue,
				OutValue,
				StringifySdfReferenceOrPayload<SdfReference>
			);
		}
		else if (UnderlyingType.IsA<SdfListOp<SdfPayload>>())
		{
			OutValue.SourceType = EUsdBasicDataTypes::String;
			OutValue.ExtraTypeInfo = GetSdfTypeName<SdfListOp<SdfPayload>>();

			ConvertListOpValue<FString, SdfPayload>(	  //
				UsdValue,
				OutValue,
				StringifySdfReferenceOrPayload<SdfPayload>
			);
		}
		// Type most commonly used for the value of the apiSchemas prim metadata
		else if (UnderlyingType.IsA<SdfListOp<TfToken>>())
		{
			OutValue.SourceType = EUsdBasicDataTypes::Token;
			OutValue.ExtraTypeInfo = GetSdfTypeName<SdfListOp<TfToken>>();

			ConvertListOpValue<FString, TfToken, FString(const pxr::TfToken&)>(	   //
				UsdValue,
				OutValue,
				UsdToUnreal::ConvertToken
			);
		}
		// These types should only appear within metadata, and don't support arrays.
		// There are more of them (e.g. pxr/usd/usd/crateDataTypes.h), but these are the most common.
		// Also check pxr/usd/sdf/types.cpp for where these are defined. These are simple enums
		else if (UnderlyingType.IsA<SdfPermission>())
		{
			OutValue.SourceType = EUsdBasicDataTypes::Int;
			ConvertSimpleValue<int32, SdfPermission>(UsdValue, OutValue);
		}
		else if (UnderlyingType.IsA<SdfSpecifier>())
		{
			OutValue.SourceType = EUsdBasicDataTypes::Int;
			ConvertSimpleValue<int32, SdfSpecifier>(UsdValue, OutValue);
		}
		else if (UnderlyingType.IsA<SdfVariability>())
		{
			OutValue.SourceType = EUsdBasicDataTypes::Int;
			ConvertSimpleValue<int32, SdfVariability>(UsdValue, OutValue);
		}
		else if (UnderlyingType.IsA<SdfSpecType>())
		{
			OutValue.SourceType = EUsdBasicDataTypes::Int;
			ConvertSimpleValue<int32, SdfSpecType>(UsdValue, OutValue);
		}
		else
		{
			return false;
		}

#undef CHECK_TYPE
#pragma pop_macro("CHECK_TYPE")

		return true;
#else
		return false;
#endif	  // USE_USD_SDK
	}
}	 // namespace UsdToUnreal

#if USE_USD_SDK
namespace UnrealToUsdImpl
{
	/**
	 * Allows us to use the same lambda to convert either VtArray<USDType> or USDType
	 * Func should have a signature like this:
	 * []( const UsdUtils::FConvertedVtValueEntry& Entry ) -> USDElementType {}
	 */
	template<typename UEElementType, typename USDElementType, typename Func>
	void ConvertInner(const UsdUtils::FConvertedVtValue& InValue, pxr::VtValue& OutValue, Func Function)
	{
		if (InValue.bIsArrayValued)
		{
			pxr::VtArray<USDElementType> Array;
			Array.reserve(InValue.Entries.Num());

			for (const UsdUtils::FConvertedVtValueEntry& Entry : InValue.Entries)
			{
				if (Entry.Num() > 0)
				{
					Array.push_back(Function(Entry));
				}
			}

			OutValue = Array;
		}
		else if (InValue.Entries.Num() > 0)
		{
			const UsdUtils::FConvertedVtValueEntry& Entry = InValue.Entries[0];
			if (Entry.Num() > 0)
			{
				OutValue = Function(Entry);
			}
		}
	}

	template<typename UEElementType, typename USDElementType>
	void ConvertSimpleValue(const UsdUtils::FConvertedVtValue& InValue, pxr::VtValue& OutValue)
	{
		ConvertInner<UEElementType, USDElementType>(
			InValue,
			OutValue,
			[](const UsdUtils::FConvertedVtValueEntry& Entry) -> USDElementType
			{
				if (const UEElementType* Value = Entry[0].TryGet<UEElementType>())
				{
					return static_cast<USDElementType>(*Value);
				}

				return USDElementType();
			}
		);
	}

	template<>
	void ConvertSimpleValue<FString, std::string>(const UsdUtils::FConvertedVtValue& InValue, pxr::VtValue& OutValue)
	{
		ConvertInner<FString, std::string>(
			InValue,
			OutValue,
			[](const UsdUtils::FConvertedVtValueEntry& Entry) -> std::string
			{
				if (const FString* Value = Entry[0].TryGet<FString>())
				{
					return UnrealToUsd::ConvertString(**Value).Get();
				}

				return std::string();
			}
		);
	}

	template<>
	void ConvertSimpleValue<FString, pxr::TfToken>(const UsdUtils::FConvertedVtValue& InValue, pxr::VtValue& OutValue)
	{
		ConvertInner<FString, pxr::TfToken>(
			InValue,
			OutValue,
			[](const UsdUtils::FConvertedVtValueEntry& Entry) -> pxr::TfToken
			{
				if (const FString* Value = Entry[0].TryGet<FString>())
				{
					return UnrealToUsd::ConvertToken(**Value).Get();
				}

				return pxr::TfToken();
			}
		);
	}

	template<>
	void ConvertSimpleValue<FString, pxr::SdfAssetPath>(const UsdUtils::FConvertedVtValue& InValue, pxr::VtValue& OutValue)
	{
		ConvertInner<FString, pxr::SdfAssetPath>(
			InValue,
			OutValue,
			[](const UsdUtils::FConvertedVtValueEntry& Entry) -> pxr::SdfAssetPath
			{
				if (const FString* Value = Entry[0].TryGet<FString>())
				{
					return pxr::SdfAssetPath(UnrealToUsd::ConvertString(**Value).Get());
				}

				return pxr::SdfAssetPath();
			}
		);
	}

	/** We need the USDElementType parameter to do the final float to pxr::GfHalf conversions */
	template<typename UEElementType, typename USDArrayType, typename USDElementType = UEElementType>
	void ConvertCompoundValue(const UsdUtils::FConvertedVtValue& InValue, pxr::VtValue& OutValue)
	{
		ConvertInner<UEElementType, USDArrayType>(
			InValue,
			OutValue,
			[](const UsdUtils::FConvertedVtValueEntry& Entry) -> USDArrayType
			{
				USDArrayType USDVal(UEElementType(0));
				USDElementType* DataPtr = USDVal.data();

				for (int32 Index = 0; Index < Entry.Num(); ++Index)
				{
					if (const UEElementType* IndexValue = Entry[Index].TryGet<UEElementType>())
					{
						DataPtr[Index] = static_cast<USDElementType>(*IndexValue);
					}
				}
				return USDVal;
			}
		);
	}

	/** USD quaternions don't have the access operator defined, and the elements need to be reordered */
	template<typename UEElementType, typename USDQuatType>
	void ConvertQuatValue(const UsdUtils::FConvertedVtValue& InValue, pxr::VtValue& OutValue)
	{
		ConvertInner<UEElementType, USDQuatType>(
			InValue,
			OutValue,
			[](const UsdUtils::FConvertedVtValueEntry& Entry) -> USDQuatType
			{
				if (Entry.Num() == 4)
				{
					UEElementType QuatValues[4] = {0, 0, 0, 1};
					for (int32 Index = 0; Index < 4; ++Index)
					{
						if (const UEElementType* QuatValue = Entry[Index].TryGet<UEElementType>())
						{
							QuatValues[Index] = *QuatValue;
						}
					}

					return USDQuatType(
						QuatValues[3],	  // Real part comes first for USD
						QuatValues[0],
						QuatValues[1],
						QuatValues[2]
					);
				}

				return USDQuatType();
			}
		);
	}

	template<typename UEElementType, typename USDElementType, typename ConverterFunc>
	void ConvertListOpValue(const UsdUtils::FConvertedVtValue& InValue, pxr::VtValue& OutValue, ConverterFunc Converter)
	{
		using namespace pxr;
		using namespace UsdUtils;
		using namespace UE::USDValueConversion::Private;

		if (InValue.Entries.Num() == EListOpListIndex::Num)
		{
			SdfListOp<USDElementType> NewList;

			bool bAddedItems = false;
			for (int32 ListIndex = 0; ListIndex < EListOpListIndex::Num; ++ListIndex)
			{
				const TArray<FConvertedVtValueComponent>& UEItems = InValue.Entries[ListIndex];

				std::vector<USDElementType> UsdItems;
				UsdItems.reserve(UEItems.Num());

				for (const FConvertedVtValueComponent& UEItem : UEItems)
				{
					if (const UEElementType* Value = UEItem.TryGet<UEElementType>())
					{
						UsdItems.push_back(Converter(*Value));
					}
				}

				// It's very important that we set ExplicitItems first, and that we only set if they have at
				// least one item in the list. This because SdfListOp has essentially two "modes":
				// Explicit and not explicit. Any time we set explicit items and it wasn't in "explicit mode" yet,
				// it switches to explicit mode and dumps all previous items. Analogously, if we set any of the
				// non-explicit items and it was previously in "explicit mode", it dumps everything again and
				// switches to "non-explicit mode". So starting with explicit and only calling SetItems when we
				// have something prevents us from dumping items when we didn't mean to
				if (UsdItems.size() > 0)
				{
					NewList.SetItems(UsdItems, (SdfListOpType)ListIndex);
					bAddedItems = true;
				}
			}

			if (bAddedItems)
			{
				OutValue = NewList;
			}
		}
	}

	template<typename SdfRefOrPayload>
	SdfRefOrPayload UnstringifySdfReferenceOrPayload(const FString& String)
	{
		using namespace UE::USDValueConversion::Private;

		TArray<FString> Elements;
		const bool bCullEmpty = false;
		int32 NumElements = String.ParseIntoArray(Elements, *ReferencePayloadDelimiter, bCullEmpty);

		SdfRefOrPayload Result;
		if (NumElements == 4)
		{
			if (Elements[0].Len() > 0)
			{
				Result.SetAssetPath(UnrealToUsd::ConvertString(*Elements[0]).Get());
			}

			if (Elements[1].Len() > 0)
			{
				Result.SetPrimPath(UnrealToUsd::ConvertPath(*Elements[1]).Get());
			}

			pxr::SdfLayerOffset LayerOffset;
			{
				double Offset = 0.0f;
				if (LexTryParseString(Offset, *Elements[2]))
				{
					LayerOffset.SetOffset(Offset);
				}

				double Scale = 1.0f;
				if (LexTryParseString(Scale, *Elements[3]))
				{
					LayerOffset.SetScale(Scale);
				}
			}
			Result.SetLayerOffset(LayerOffset);
		}
		return Result;
	}
}	 // namespace UnrealToUsdImpl
#endif	  // USE_USD_SDK

namespace UnrealToUsd
{
	bool ConvertValue(const UsdUtils::FConvertedVtValue& InValue, UE::FVtValue& OutValue)
	{
#if USE_USD_SDK
		using namespace pxr;
		using namespace UsdUtils;
		using namespace UnrealToUsdImpl;
		using namespace UE::USDValueConversion::Private;

		// Always USD Allocs here because we may end up allocating arrays for these compound types,
		// and those arrays can be moved into the UE::FVtValue's pxr::VtValue and, when set in an
		// attribute, destroyed internally by USD
		FScopedUsdAllocs Allocs;

		VtValue& UsdValue = OutValue.GetUsdValue();

		switch (InValue.SourceType)
		{
			case EUsdBasicDataTypes::Bool:
				ConvertSimpleValue<bool, bool>(InValue, UsdValue);
				break;
			case EUsdBasicDataTypes::Uchar:
				ConvertSimpleValue<uint8, uint8_t>(InValue, UsdValue);
				break;
			case EUsdBasicDataTypes::Int:
				ConvertSimpleValue<int32, int32_t>(InValue, UsdValue);
				break;
			case EUsdBasicDataTypes::Uint:
				ConvertSimpleValue<uint32, uint32_t>(InValue, UsdValue);
				break;
			case EUsdBasicDataTypes::Int64:
				ConvertSimpleValue<int64, int64_t>(InValue, UsdValue);
				break;
			case EUsdBasicDataTypes::Uint64:
				ConvertSimpleValue<uint64, uint64_t>(InValue, UsdValue);
				break;
			case EUsdBasicDataTypes::Half:
				ConvertSimpleValue<float, GfHalf>(InValue, UsdValue);
				break;
			case EUsdBasicDataTypes::Float:
				ConvertSimpleValue<float, float>(InValue, UsdValue);
				break;
			case EUsdBasicDataTypes::Double:
				ConvertSimpleValue<double, double>(InValue, UsdValue);
				break;
			case EUsdBasicDataTypes::Timecode:
				ConvertSimpleValue<double, SdfTimeCode>(InValue, UsdValue);
				break;
			case EUsdBasicDataTypes::String:
				if (InValue.ExtraTypeInfo == GetSdfTypeName<SdfListOp<std::string>>())
				{
					ConvertListOpValue<FString, std::string>(
						InValue,
						UsdValue,
						[](const FString& Item) -> std::string
						{
							return UnrealToUsd::ConvertString(*Item).Get();
						}
					);
				}
				else if (InValue.ExtraTypeInfo == GetSdfTypeName<SdfListOp<SdfPath>>())
				{
					ConvertListOpValue<FString, SdfPath>(
						InValue,
						UsdValue,
						[](const FString& Item) -> SdfPath
						{
							return UnrealToUsd::ConvertPath(*Item).Get();
						}
					);
				}
				else if (InValue.ExtraTypeInfo == GetSdfTypeName<SdfListOp<SdfReference>>())
				{
					ConvertListOpValue<FString, SdfReference>(	  //
						InValue,
						UsdValue,
						UnstringifySdfReferenceOrPayload<SdfReference>
					);
				}
				else if (InValue.ExtraTypeInfo == GetSdfTypeName<SdfListOp<SdfPayload>>())
				{
					ConvertListOpValue<FString, SdfPayload>(	//
						InValue,
						UsdValue,
						UnstringifySdfReferenceOrPayload<SdfPayload>
					);
				}
				else
				{
					ConvertSimpleValue<FString, std::string>(InValue, UsdValue);
				}
				break;
			case EUsdBasicDataTypes::Token:
				if (InValue.ExtraTypeInfo == GetSdfTypeName<SdfListOp<TfToken>>())
				{
					ConvertListOpValue<FString, TfToken>(
						InValue,
						UsdValue,
						[](const FString& Item) -> TfToken
						{
							return UnrealToUsd::ConvertToken(*Item).Get();
						}
					);
				}
				else
				{
					ConvertSimpleValue<FString, TfToken>(InValue, UsdValue);
				}
				break;
			case EUsdBasicDataTypes::Asset:
				ConvertSimpleValue<FString, SdfAssetPath>(InValue, UsdValue);
				break;
			case EUsdBasicDataTypes::Matrix2d:
				ConvertCompoundValue<double, GfMatrix2d>(InValue, UsdValue);
				break;
			case EUsdBasicDataTypes::Matrix3d:
				ConvertCompoundValue<double, GfMatrix3d>(InValue, UsdValue);
				break;
			case EUsdBasicDataTypes::Matrix4d:
				ConvertCompoundValue<double, GfMatrix4d>(InValue, UsdValue);
				break;
			case EUsdBasicDataTypes::Quatd:
				ConvertQuatValue<double, GfQuatd>(InValue, UsdValue);
				break;
			case EUsdBasicDataTypes::Quatf:
				ConvertQuatValue<float, GfQuatf>(InValue, UsdValue);
				break;
			case EUsdBasicDataTypes::Quath:
				ConvertQuatValue<float, GfQuath>(InValue, UsdValue);
				break;
			case EUsdBasicDataTypes::Double2:
				ConvertCompoundValue<double, GfVec2d>(InValue, UsdValue);
				break;
			case EUsdBasicDataTypes::Float2:
				ConvertCompoundValue<float, GfVec2f>(InValue, UsdValue);
				break;
			case EUsdBasicDataTypes::Half2:
				ConvertCompoundValue<float, GfVec2h, GfHalf>(InValue, UsdValue);
				break;
			case EUsdBasicDataTypes::Int2:
				ConvertCompoundValue<int32, GfVec2i>(InValue, UsdValue);
				break;
			case EUsdBasicDataTypes::Double3:
				ConvertCompoundValue<double, GfVec3d>(InValue, UsdValue);
				break;
			case EUsdBasicDataTypes::Float3:
				ConvertCompoundValue<float, GfVec3f>(InValue, UsdValue);
				break;
			case EUsdBasicDataTypes::Half3:
				ConvertCompoundValue<float, GfVec3h, GfHalf>(InValue, UsdValue);
				break;
			case EUsdBasicDataTypes::Int3:
				ConvertCompoundValue<int32, GfVec3i>(InValue, UsdValue);
				break;
			case EUsdBasicDataTypes::Double4:
				ConvertCompoundValue<double, GfVec4d>(InValue, UsdValue);
				break;
			case EUsdBasicDataTypes::Float4:
				ConvertCompoundValue<float, GfVec4f>(InValue, UsdValue);
				break;
			case EUsdBasicDataTypes::Half4:
				ConvertCompoundValue<float, GfVec4h, GfHalf>(InValue, UsdValue);
				break;
			case EUsdBasicDataTypes::Int4:
				ConvertCompoundValue<int32, GfVec4i>(InValue, UsdValue);
				break;
			case EUsdBasicDataTypes::None:
			default:
				break;
		}

		// We consider a success returning an empty value if our input value was also empty
		return InValue.Entries.Num() == 0 || !OutValue.IsEmpty();
#else
		return false;
#endif	  // USE_USD_SDK
	}
}	 // namespace UnrealToUsd

FArchive& operator<<(FArchive& Ar, UsdUtils::FConvertedVtValue& Struct)
{
	Ar << Struct.Entries;
	Ar << Struct.SourceType;
	Ar << Struct.ExtraTypeInfo;
	Ar << Struct.bIsArrayValued;
	Ar << Struct.bIsEmpty;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, UsdUtils::FConvertedVtValueComponent& Component)
{
	if (Ar.IsSaving())
	{
		uint64 TypeIndex = Component.GetIndex();
		Ar << TypeIndex;

		switch (TypeIndex)
		{
			case UsdUtils::FConvertedVtValueComponent::IndexOfType<bool>():
				if (bool* Val = Component.TryGet<bool>())
				{
					Ar << *Val;
				}
				break;
			case UsdUtils::FConvertedVtValueComponent::IndexOfType<uint8>():
				if (uint8* Val = Component.TryGet<uint8>())
				{
					Ar << *Val;
				}
				break;
			case UsdUtils::FConvertedVtValueComponent::IndexOfType<int32>():
				if (int32* Val = Component.TryGet<int32>())
				{
					Ar << *Val;
				}
				break;
			case UsdUtils::FConvertedVtValueComponent::IndexOfType<uint32>():
				if (uint32* Val = Component.TryGet<uint32>())
				{
					Ar << *Val;
				}
				break;
			case UsdUtils::FConvertedVtValueComponent::IndexOfType<int64>():
				if (int64* Val = Component.TryGet<int64>())
				{
					Ar << *Val;
				}
				break;
			case UsdUtils::FConvertedVtValueComponent::IndexOfType<uint64>():
				if (uint64* Val = Component.TryGet<uint64>())
				{
					Ar << *Val;
				}
				break;
			case UsdUtils::FConvertedVtValueComponent::IndexOfType<float>():
				if (float* Val = Component.TryGet<float>())
				{
					Ar << *Val;
				}
				break;
			case UsdUtils::FConvertedVtValueComponent::IndexOfType<double>():
				if (double* Val = Component.TryGet<double>())
				{
					Ar << *Val;
				}
				break;
			case UsdUtils::FConvertedVtValueComponent::IndexOfType<FString>():
				if (FString* Val = Component.TryGet<FString>())
				{
					Ar << *Val;
				}
				break;
			default:
				break;
		}
	}
	else	// IsLoading
	{
		uint64 TypeIndex;
		Ar << TypeIndex;

		switch (TypeIndex)
		{
			case UsdUtils::FConvertedVtValueComponent::IndexOfType<bool>():
			{
				bool Val;
				Ar << Val;
				Component.Set<bool>(Val);
				break;
			}
			case UsdUtils::FConvertedVtValueComponent::IndexOfType<uint8>():
			{
				uint8 Val;
				Ar << Val;
				Component.Set<uint8>(Val);
				break;
			}
			case UsdUtils::FConvertedVtValueComponent::IndexOfType<int32>():
			{
				int32 Val;
				Ar << Val;
				Component.Set<int32>(Val);
				break;
			}
			case UsdUtils::FConvertedVtValueComponent::IndexOfType<uint32>():
			{
				uint32 Val;
				Ar << Val;
				Component.Set<uint32>(Val);
				break;
			}
			case UsdUtils::FConvertedVtValueComponent::IndexOfType<int64>():
			{
				int64 Val;
				Ar << Val;
				Component.Set<int64>(Val);
				break;
			}
			case UsdUtils::FConvertedVtValueComponent::IndexOfType<uint64>():
			{
				uint64 Val;
				Ar << Val;
				Component.Set<uint64>(Val);
				break;
			}
			case UsdUtils::FConvertedVtValueComponent::IndexOfType<float>():
			{
				float Val;
				Ar << Val;
				Component.Set<float>(Val);
				break;
			}
			case UsdUtils::FConvertedVtValueComponent::IndexOfType<double>():
			{
				double Val;
				Ar << Val;
				Component.Set<double>(Val);
				break;
			}
			case UsdUtils::FConvertedVtValueComponent::IndexOfType<FString>():
			{
				FString Val;
				Ar << Val;
				Component.Set<FString>(Val);
				break;
			}
			default:
				Component.Set<bool>(false);
				break;
		}
	}

	return Ar;
}

namespace UsdUtils
{
	FString Stringify(const UE::FVtValue& Value)
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;
		return Stringify(Value.GetUsdValue());
#else
		return FString();
#endif	  // USE_USD_SDK
	}

#if USE_USD_SDK
	FString StringifyUsdString(const pxr::VtValue& Value)
	{
		if (!ensure(Value.IsHolding<std::string>()))
		{
			return {};
		}

		return UsdToUnreal::ConvertString(Value.UncheckedGet<std::string>());
	}

	FString StringifyUsdStringArray(const pxr::VtValue& Value)
	{
		using namespace UE::USDValueConversion::Private;

		if (!ensure(Value.IsHolding<pxr::VtArray<std::string>>()))
		{
			return {};
		}

		const pxr::VtArray<std::string>& StringArray = Value.UncheckedGet<pxr::VtArray<std::string>>();

		FString Result = TEXT("[");
		for (const std::string& String : StringArray)
		{
			// If the string contains a double quote, use a single quote as a delimiter, os else the string
			// will be impossible to tokenize into the source string later when unstringifying
			if (String.find_first_of(DoubleQuoteChar) == std::string::npos)
			{
				Result += FString::Printf(TEXT("\"%s\", "), *UsdToUnreal::ConvertString(String));
			}
			else
			{
				Result += FString::Printf(TEXT("'%s', "), *UsdToUnreal::ConvertString(String));
			}
		}
		Result.RemoveFromEnd(TEXT(", "));
		Result += TEXT("]");

		return Result;
	}

	FString StringifyUsdToken(const pxr::VtValue& Value)
	{
		if (!ensure(Value.IsHolding<pxr::TfToken>()))
		{
			return {};
		}

		return UsdToUnreal::ConvertToken(Value.UncheckedGet<pxr::TfToken>());
	}

	FString StringifyUsdTokenArray(const pxr::VtValue& Value)
	{
		using namespace UE::USDValueConversion::Private;

		if (!ensure(Value.IsHolding<pxr::VtArray<pxr::TfToken>>()))
		{
			return {};
		}

		const pxr::VtArray<pxr::TfToken>& TokenArray = Value.UncheckedGet<pxr::VtArray<pxr::TfToken>>();

		FString Result = TEXT("[");
		for (const pxr::TfToken& Token : TokenArray)
		{
			const std::string& String = Token.GetString();

			// If the string contains a double quote, use a single quote as a delimiter, os else the string
			// will be impossible to tokenize into the source string later when unstringifying
			if (String.find_first_of(DoubleQuoteChar) == std::string::npos)
			{
				Result += FString::Printf(TEXT("\"%s\", "), *UsdToUnreal::ConvertString(String));
			}
			else
			{
				Result += FString::Printf(TEXT("'%s', "), *UsdToUnreal::ConvertString(String));
			}
		}
		Result.RemoveFromEnd(TEXT(", "));
		Result += TEXT("]");

		return Result;
	}

	FString StringifyUsdAsset(const pxr::VtValue& Value)
	{
		if (!ensure(Value.IsHolding<pxr::SdfAssetPath>()))
		{
			return {};
		}

		return FString::Printf(TEXT("@%s@"), *UsdToUnreal::ConvertString(Value.UncheckedGet<pxr::SdfAssetPath>().GetAssetPath()));
	}

	FString StringifyUsdAssetArray(const pxr::VtValue& Value)
	{
		if (!ensure(Value.IsHolding<pxr::VtArray<pxr::SdfAssetPath>>()))
		{
			return {};
		}

		const pxr::VtArray<pxr::SdfAssetPath>& AssetArray = Value.UncheckedGet<pxr::VtArray<pxr::SdfAssetPath>>();

		FString Result = TEXT("[");
		for (const pxr::SdfAssetPath& Asset : AssetArray)
		{
			Result += FString::Printf(TEXT("@%s@, "), *UsdToUnreal::ConvertString(Asset.GetAssetPath()));
		}
		Result.RemoveFromEnd(TEXT(", "));
		Result += TEXT("]");

		return Result;
	}

	FString StringifyTokenList(const pxr::VtValue& Value)
	{
		using namespace UE::USDValueConversion::Private;

		if (!ensure(Value.IsHolding<pxr::SdfListOp<pxr::TfToken>>()))
		{
			return {};
		}

		const pxr::SdfListOp<pxr::TfToken>& TokenList = Value.UncheckedGet<pxr::SdfListOp<pxr::TfToken>>();

		FString Result = UE::USDValueConversion::Private::TokenListStringStart;
		TUsdStore<std::vector<pxr::TfToken>> TokenListItems = TokenList.GetAppliedItems();
		for (const pxr::TfToken& Token : TokenListItems.Get())
		{
			const std::string& String = Token.GetString();

			// If the string contains a double quote, use a single quote as a delimiter, os else the string
			// will be impossible to tokenize into the source string later when unstringifying
			if (String.find_first_of(DoubleQuoteChar) == std::string::npos)
			{
				Result += FString::Printf(TEXT("\"%s\", "), *UsdToUnreal::ConvertString(String));
			}
			else
			{
				Result += FString::Printf(TEXT("'%s', "), *UsdToUnreal::ConvertString(String));
			}
		}
		Result.RemoveFromEnd(TEXT(", "));
		Result += TEXT("])");

		return Result;
	}

	FString Stringify(const pxr::VtValue& Value)
	{
		// Here we'll use custom stringifiers for the string-like types, or else USD will remove/escape the
		// quote marks, which we don't really need and makes it impossible to unstringify for the array cases.
		// Our unstringifiers expect to find these quote marks, and will fail if given e.g. the output
		// of pxr::TfStringify() called on a pxr::VtArray<std::string> directly, that removes them all.

		// Ideally we'd do some kind of IsA<std::string>() check to also cover inheritance... it's pretty unlikely
		// the user will have derived std::string or pxr::VtArray<pxr::TfToken> though, so this should
		// hopefully be faster

		using TypeToFuncMap = std::unordered_map<pxr::TfType, TFunction<FString(const pxr::VtValue&)>, pxr::TfHash>;

		static const TypeToFuncMap Stringifiers = {
			{					pxr::TfType::Find<std::string>(),	   StringifyUsdString},
			{		 pxr::TfType::Find<pxr::VtArray<std::string>>(), StringifyUsdStringArray},
			{				   pxr::TfType::Find<pxr::TfToken>(),		StringifyUsdToken},
			{	 pxr::TfType::Find<pxr::VtArray<pxr::TfToken>>(),  StringifyUsdTokenArray},
			{			  pxr::TfType::Find<pxr::SdfAssetPath>(),		 StringifyUsdAsset},
			{pxr::TfType::Find<pxr::VtArray<pxr::SdfAssetPath>>(),  StringifyUsdAssetArray},
			{	 pxr::TfType::Find<pxr::SdfListOp<pxr::TfToken>>(),		StringifyTokenList},
		};

		TypeToFuncMap::const_iterator FoundStringifier = Stringifiers.find(Value.GetType());
		if (FoundStringifier != Stringifiers.end())
		{
			return FoundStringifier->second(Value);
		}

		return UsdToUnreal::ConvertString(pxr::TfStringify(Value));
	}
#endif	  // USE_USD_SDK

	FString StringifyAsBool(bool Value)
	{
#if USE_USD_SDK
		return UsdToUnreal::ConvertString(pxr::TfStringify(pxr::VtValue{Value}));
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsUChar(uint8 Value)
	{
#if USE_USD_SDK
		return UsdToUnreal::ConvertString(pxr::TfStringify(pxr::VtValue{static_cast<uint8_t>(Value)}));
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsInt(int32 Value)
	{
#if USE_USD_SDK
		return UsdToUnreal::ConvertString(pxr::TfStringify(pxr::VtValue{static_cast<int32_t>(Value)}));
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsUInt(uint32 Value)
	{
#if USE_USD_SDK
		return UsdToUnreal::ConvertString(pxr::TfStringify(pxr::VtValue{static_cast<uint32_t>(Value)}));
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsInt64(int64 Value)
	{
#if USE_USD_SDK
		return UsdToUnreal::ConvertString(pxr::TfStringify(pxr::VtValue{static_cast<int64_t>(Value)}));
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsUInt64(uint64 Value)
	{
#if USE_USD_SDK
		return UsdToUnreal::ConvertString(pxr::TfStringify(pxr::VtValue{static_cast<uint64_t>(Value)}));
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsHalf(float Value)
	{
#if USE_USD_SDK
		return UsdToUnreal::ConvertString(pxr::TfStringify(pxr::VtValue{static_cast<pxr::GfHalf>(Value)}));
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsFloat(float Value)
	{
#if USE_USD_SDK
		return UsdToUnreal::ConvertString(pxr::TfStringify(pxr::VtValue{Value}));
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsDouble(double Value)
	{
#if USE_USD_SDK
		return UsdToUnreal::ConvertString(pxr::TfStringify(pxr::VtValue{Value}));
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsTimeCode(double Value)
	{
#if USE_USD_SDK
		return UsdToUnreal::ConvertString(pxr::TfStringify(pxr::VtValue{static_cast<pxr::SdfTimeCode>(Value)}));
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsString(const FString& Value)
	{
		return Value;
	}

	FString StringifyAsToken(const FString& Value)
	{
		return Value;
	}

	FString StringifyAsAssetPath(const FString& Value)
	{
		using namespace UE::USDValueConversion::Private;

		return FString::Printf(TEXT("@%s@"), *Value);
	}

	FString StringifyAsMatrix2d(const FMatrix2D& Value)
	{
#if USE_USD_SDK
		return UsdToUnreal::ConvertString(pxr::TfStringify(pxr::VtValue{UnrealToUsd::ConvertMatrix(Value)}));
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsMatrix3d(const FMatrix3D& Value)
	{
#if USE_USD_SDK
		return UsdToUnreal::ConvertString(pxr::TfStringify(pxr::VtValue{UnrealToUsd::ConvertMatrix(Value)}));
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsMatrix4d(const FMatrix& Value)
	{
#if USE_USD_SDK
		return UsdToUnreal::ConvertString(pxr::TfStringify(pxr::VtValue{UnrealToUsd::ConvertMatrix(Value)}));
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsQuatd(const FQuat& Value)
	{
#if USE_USD_SDK
		return UsdToUnreal::ConvertString(pxr::TfStringify(pxr::VtValue{UnrealToUsd::ConvertQuatDouble(Value)}));
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsQuatf(const FQuat& Value)
	{
#if USE_USD_SDK
		return UsdToUnreal::ConvertString(pxr::TfStringify(pxr::VtValue{UnrealToUsd::ConvertQuatFloat(Value)}));
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsQuath(const FQuat& Value)
	{
#if USE_USD_SDK
		return UsdToUnreal::ConvertString(pxr::TfStringify(pxr::VtValue{UnrealToUsd::ConvertQuatHalf(Value)}));
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsDouble2(const FVector2D& Value)
	{
#if USE_USD_SDK
		return UsdToUnreal::ConvertString(pxr::TfStringify(pxr::VtValue{UnrealToUsd::ConvertVectorDouble(Value)}));
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsFloat2(const FVector2D& Value)
	{
#if USE_USD_SDK
		return UsdToUnreal::ConvertString(pxr::TfStringify(pxr::VtValue{UnrealToUsd::ConvertVectorFloat(Value)}));
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsHalf2(const FVector2D& Value)
	{
#if USE_USD_SDK
		return UsdToUnreal::ConvertString(pxr::TfStringify(pxr::VtValue{UnrealToUsd::ConvertVectorHalf(Value)}));
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsInt2(const FIntPoint& Value)
	{
#if USE_USD_SDK
		return UsdToUnreal::ConvertString(pxr::TfStringify(pxr::VtValue{UnrealToUsd::ConvertVectorInt(Value)}));
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsDouble3(const FVector& Value)
	{
#if USE_USD_SDK
		return UsdToUnreal::ConvertString(pxr::TfStringify(pxr::VtValue{UnrealToUsd::ConvertVectorDouble(Value)}));
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsFloat3(const FVector& Value)
	{
#if USE_USD_SDK
		return UsdToUnreal::ConvertString(pxr::TfStringify(pxr::VtValue{UnrealToUsd::ConvertVectorFloat(Value)}));
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsHalf3(const FVector& Value)
	{
#if USE_USD_SDK
		return UsdToUnreal::ConvertString(pxr::TfStringify(pxr::VtValue{UnrealToUsd::ConvertVectorHalf(Value)}));
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsInt3(const FIntVector& Value)
	{
#if USE_USD_SDK
		return UsdToUnreal::ConvertString(pxr::TfStringify(pxr::VtValue{UnrealToUsd::ConvertVectorInt(Value)}));
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsDouble4(const FVector4& Value)
	{
#if USE_USD_SDK
		return UsdToUnreal::ConvertString(pxr::TfStringify(pxr::VtValue{UnrealToUsd::ConvertVectorDouble(Value)}));
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsFloat4(const FVector4& Value)
	{
#if USE_USD_SDK
		return UsdToUnreal::ConvertString(pxr::TfStringify(pxr::VtValue{UnrealToUsd::ConvertVectorFloat(Value)}));
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsHalf4(const FVector4& Value)
	{
#if USE_USD_SDK
		return UsdToUnreal::ConvertString(pxr::TfStringify(pxr::VtValue{UnrealToUsd::ConvertVectorHalf(Value)}));
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsInt4(const FIntVector4& Value)
	{
#if USE_USD_SDK
		return UsdToUnreal::ConvertString(pxr::TfStringify(pxr::VtValue{UnrealToUsd::ConvertVectorInt(Value)}));
#else
		return {};
#endif	  // USE_USD_SDK
	}

#if USE_USD_SDK
	template<typename UEType, typename USDType>
	FString StringifyArrayCastingElements(const TArray<UEType>& Value)
	{
		pxr::VtArray<USDType> UsdArray;
		UsdArray.reserve(Value.Num());
		for (const UEType Element : Value)
		{
			UsdArray.push_back(static_cast<USDType>(Element));
		}

		return UsdToUnreal::ConvertString(pxr::TfStringify(pxr::VtValue{UsdArray}));
	}
#endif	  // USE_USD_SDK

	FString StringifyAsBoolArray(const TArray<bool>& Value)
	{
#if USE_USD_SDK
		return StringifyArrayCastingElements<bool, bool>(Value);
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsUCharArray(const TArray<uint8>& Value)
	{
#if USE_USD_SDK
		return StringifyArrayCastingElements<uint8, uint8_t>(Value);
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsIntArray(const TArray<int32>& Value)
	{
#if USE_USD_SDK
		return StringifyArrayCastingElements<int32, int32_t>(Value);
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsUIntArray(const TArray<uint32>& Value)
	{
#if USE_USD_SDK
		return StringifyArrayCastingElements<uint32, uint32_t>(Value);
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsInt64Array(const TArray<int64>& Value)
	{
#if USE_USD_SDK
		return StringifyArrayCastingElements<int64, int64_t>(Value);
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsUInt64Array(const TArray<uint64>& Value)
	{
#if USE_USD_SDK
		return StringifyArrayCastingElements<uint64, uint64_t>(Value);
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsHalfArray(const TArray<float>& Value)
	{
#if USE_USD_SDK
		return StringifyArrayCastingElements<float, pxr::GfHalf>(Value);
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsFloatArray(const TArray<float>& Value)
	{
#if USE_USD_SDK
		return StringifyArrayCastingElements<float, float>(Value);
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsDoubleArray(const TArray<double>& Value)
	{
#if USE_USD_SDK
		return StringifyArrayCastingElements<double, double>(Value);
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsTimeCodeArray(const TArray<double>& Value)
	{
#if USE_USD_SDK
		return StringifyArrayCastingElements<double, pxr::SdfTimeCode>(Value);
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsStringArray(const TArray<FString>& Value)
	{
		using namespace UE::USDValueConversion::Private;

		if (Value.Num() == 0)
		{
			return TEXT("[]");
		}

		FString Result = TEXT("[");
		for (const FString& Element : Value)
		{
			// If the string contains a double quote, use a single quote as a delimiter, or else the string
			// will be impossible to tokenize into the source string later when unstringifying
			if (Element.Contains(&DoubleQuoteTCHAR))
			{
				Result += FString::Printf(TEXT("'%s', "), *Element);
			}
			else
			{
				Result += FString::Printf(TEXT("\"%s\", "), *Element);
			}
		}
		Result.RemoveFromEnd(TEXT(", "));
		Result += TEXT("]");

		return Result;
	}

	FString StringifyAsTokenArray(const TArray<FString>& Value)
	{
		return StringifyAsStringArray(Value);
	}

	FString StringifyAsAssetPathArray(const TArray<FString>& Value)
	{
		using namespace UE::USDValueConversion::Private;

		if (Value.Num() == 0)
		{
			return TEXT("[]");
		}

		FString Result = TEXT("[");
		for (const FString& Element : Value)
		{
			Result += FString::Printf(TEXT("@%s@, "), *Element);
		}
		Result.RemoveFromEnd(TEXT(", "));
		Result += TEXT("]");

		return Result;
	}

	FString StringifyAsListOpTokens(const TArray<FString>& Value)
	{
		using namespace UE::USDValueConversion::Private;

		if (Value.Num() == 0)
		{
			return TokenListStringEmpty;
		}

		FString Result = TokenListStringStart;
		for (const FString& Element : Value)
		{
			// If the string contains a double quote, use a single quote as a delimiter, or else the string
			// will be impossible to tokenize into the source string later when unstringifying
			if (Element.Contains(&DoubleQuoteTCHAR))
			{
				Result += FString::Printf(TEXT("'%s', "), *Element);
			}
			else
			{
				Result += FString::Printf(TEXT("\"%s\", "), *Element);
			}
		}
		Result.RemoveFromEnd(TEXT(", "));
		Result += TEXT("])");

		return Result;
	}

	template<typename UEType, typename USDType, USDType (*Converter)(const UEType&)>
	FString StringifyArrayConvertingElements(const TArray<UEType>& Value)
	{
		FScopedUsdAllocs Allocs;

#if USE_USD_SDK
		pxr::VtArray<USDType> UsdArray;
		UsdArray.reserve(Value.Num());
		for (const UEType& Element : Value)
		{
			UsdArray.push_back(Converter(Element));
		}

		return UsdToUnreal::ConvertString(pxr::TfStringify(pxr::VtValue{UsdArray}));
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsMatrix2dArray(const TArray<FMatrix2D>& Value)
	{
#if USE_USD_SDK
		return StringifyArrayConvertingElements<FMatrix2D, pxr::GfMatrix2d, UnrealToUsd::ConvertMatrix>(Value);
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsMatrix3dArray(const TArray<FMatrix3D>& Value)
	{
#if USE_USD_SDK
		return StringifyArrayConvertingElements<FMatrix3D, pxr::GfMatrix3d, UnrealToUsd::ConvertMatrix>(Value);
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsMatrix4dArray(const TArray<FMatrix>& Value)
	{
#if USE_USD_SDK
		return StringifyArrayConvertingElements<FMatrix, pxr::GfMatrix4d, UnrealToUsd::ConvertMatrix>(Value);
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsQuatdArray(const TArray<FQuat>& Value)
	{
#if USE_USD_SDK
		return StringifyArrayConvertingElements<FQuat, pxr::GfQuatd, UnrealToUsd::ConvertQuatDouble>(Value);
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsQuatfArray(const TArray<FQuat>& Value)
	{
#if USE_USD_SDK
		return StringifyArrayConvertingElements<FQuat, pxr::GfQuatf, UnrealToUsd::ConvertQuatFloat>(Value);
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsQuathArray(const TArray<FQuat>& Value)
	{
#if USE_USD_SDK
		return StringifyArrayConvertingElements<FQuat, pxr::GfQuath, UnrealToUsd::ConvertQuatHalf>(Value);
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsDouble2Array(const TArray<FVector2D>& Value)
	{
#if USE_USD_SDK
		return StringifyArrayConvertingElements<FVector2D, pxr::GfVec2d, UnrealToUsd::ConvertVectorDouble>(Value);
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsFloat2Array(const TArray<FVector2D>& Value)
	{
#if USE_USD_SDK
		return StringifyArrayConvertingElements<FVector2D, pxr::GfVec2f, UnrealToUsd::ConvertVectorFloat>(Value);
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsHalf2Array(const TArray<FVector2D>& Value)
	{
#if USE_USD_SDK
		return StringifyArrayConvertingElements<FVector2D, pxr::GfVec2h, UnrealToUsd::ConvertVectorHalf>(Value);
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsInt2Array(const TArray<FIntPoint>& Value)
	{
#if USE_USD_SDK
		return StringifyArrayConvertingElements<FIntPoint, pxr::GfVec2i, UnrealToUsd::ConvertVectorInt>(Value);
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsDouble3Array(const TArray<FVector>& Value)
	{
#if USE_USD_SDK
		return StringifyArrayConvertingElements<FVector, pxr::GfVec3d, UnrealToUsd::ConvertVectorDouble>(Value);
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsFloat3Array(const TArray<FVector>& Value)
	{
#if USE_USD_SDK
		return StringifyArrayConvertingElements<FVector, pxr::GfVec3f, UnrealToUsd::ConvertVectorFloat>(Value);
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsHalf3Array(const TArray<FVector>& Value)
	{
#if USE_USD_SDK
		return StringifyArrayConvertingElements<FVector, pxr::GfVec3h, UnrealToUsd::ConvertVectorHalf>(Value);
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsInt3Array(const TArray<FIntVector>& Value)
	{
#if USE_USD_SDK
		return StringifyArrayConvertingElements<FIntVector, pxr::GfVec3i, UnrealToUsd::ConvertVectorInt>(Value);
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsDouble4Array(const TArray<FVector4>& Value)
	{
#if USE_USD_SDK
		return StringifyArrayConvertingElements<FVector4, pxr::GfVec4d, UnrealToUsd::ConvertVectorDouble>(Value);
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsFloat4Array(const TArray<FVector4>& Value)
	{
#if USE_USD_SDK
		return StringifyArrayConvertingElements<FVector4, pxr::GfVec4f, UnrealToUsd::ConvertVectorFloat>(Value);
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsHalf4Array(const TArray<FVector4>& Value)
	{
#if USE_USD_SDK
		return StringifyArrayConvertingElements<FVector4, pxr::GfVec4h, UnrealToUsd::ConvertVectorHalf>(Value);
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FString StringifyAsInt4Array(const TArray<FIntVector4>& Value)
	{
#if USE_USD_SDK
		return StringifyArrayConvertingElements<FIntVector4, pxr::GfVec4i, UnrealToUsd::ConvertVectorInt>(Value);
#else
		return {};
#endif	  // USE_USD_SDK
	}
}	 // namespace UsdUtils

namespace UE::USDValueConversion::Private
{
#if USE_USD_SDK
	template<typename USDType>
	bool UnstringifyInner(const FString& String, USDType& Output)
	{
		FScopedUsdAllocs Allocs;

		// Very careful here: pxr::TfUnstringify will flip bSuccess to false if something is wrong, but it
		// will *not* flip it to true if you first provide it with false and everything is OK...
		bool bSuccess = true;
		USDType Converted = pxr::TfUnstringify<USDType>(UnrealToUsd::ConvertString(*String).Get(), &bSuccess);
		if (bSuccess)
		{
			Output = Converted;
			return true;
		}

		return false;
	}

	// Explicit instantiation for uint8 because pxr::TfUnstringify<uint8_t> will actually try parsing characters,
	// meaning the string "2" is parsed as "50" as that's the index of the character '2' on an ascii table...
	template<>
	bool UnstringifyInner(const FString& String, uint8_t& Output)
	{
		Output = static_cast<uint8_t>(FCString::Atoi(*String));
		return true;
	}

	// Explicit instantiations due to the fact that if we tried overloading pxr::TfUnstringify for e.g.
	// pxr::SdfAssetPath/pxr::TfToken we'd have to rely on std::string's operator >> to unstringify the
	// underlying string from the stream, and it annoyingly always skips whitespace in the middle of the string...
	template<>
	bool UnstringifyInner(const FString& String, pxr::SdfAssetPath& Output)
	{
		using namespace UE::USDValueConversion::Private;

		FScopedUsdAllocs Allocs;

		std::string UnderlyingString = UnrealToUsd::ConvertString(*String).Get();

		// When stringifying an asset path USD places an "@" symbol to start and end the string.
		// If we feed the string as-is to pxr::SdfAssetPath though and stringify *that*, then we will
		// get another set of "@" symbols on that output string... Here we strip the manual "@" symbols
		// before creating our pxr::SdfAssetPath to prevent that
		const bool bStartsWithDelimiter = UnderlyingString[0] == AssetDelimiterChar;
		const bool bEndsWithDelimiter = UnderlyingString[0] == AssetDelimiterChar;
		if (bStartsWithDelimiter || bEndsWithDelimiter)
		{
			UnderlyingString = UnderlyingString.substr(
				bStartsWithDelimiter ? 1 : 0,
				UnderlyingString.size() - bStartsWithDelimiter - bEndsWithDelimiter
			);
		}

		Output = pxr::SdfAssetPath{UnderlyingString};
		return true;
	}

	template<>
	bool UnstringifyInner(const FString& String, pxr::TfToken& Output)
	{
		FScopedUsdAllocs Allocs;

		std::string UnderlyingString = UnrealToUsd::ConvertString(*String).Get();
		Output = pxr::TfToken{UnderlyingString};
		return true;
	}

	template<>
	bool UnstringifyInner(const FString& String, pxr::SdfTimeCode& Output)
	{
		FScopedUsdAllocs Allocs;

		std::string UnderlyingString = UnrealToUsd::ConvertString(*String).Get();

		bool bSuccess = true;
		double ParsedDouble = pxr::TfUnstringify<double>(UnderlyingString, &bSuccess);
		if (!bSuccess)
		{
			return false;
		}

		Output = pxr::SdfTimeCode{ParsedDouble};
		return true;
	}

	template<typename T>
	bool UnstringifyMatrix(const FString& String, T& Output)
	{
		FScopedUsdAllocs Allocs;

		// The string should look something like "( (1, 0), (0, 1) )" originally

		// " 1, 0, 0, 1 "
		FString Copy = String.Replace(TEXT("("), TEXT("")).Replace(TEXT(")"), TEXT(""));

		// " 1", " 0", " 0", " 1 "
		TArray<FString> StringifiedValues;
		Copy.ParseIntoArray(StringifiedValues, TEXT(","));
		if (StringifiedValues.Num() != T::numRows * T::numColumns)
		{
			return false;
		}

		TArray<double> Values;
		Values.Reserve(StringifiedValues.Num());
		for (const FString& StringifiedValue : StringifiedValues)
		{
			bool bSuccess = true;
			double Parsed = pxr::TfUnstringify<double>(UnrealToUsd::ConvertString(*StringifiedValue).Get(), &bSuccess);
			if (!bSuccess)
			{
				return false;
			}

			Values.Add(Parsed);
		}

		FMemory::Memcpy(Output.data(), Values.GetData(), Values.Num() * Values.GetTypeSize());
		return true;
	}

	template<typename T>
	bool UnstringifyVec(const FString& String, T& Output)
	{
		FScopedUsdAllocs Allocs;

		// The string should look something like "(1, 0, 0, 1)" originally

		// "(1, 0, 0, 1)"
		FString Trimmed = String.TrimStartAndEnd();

		// "1, 0, 0, 1"
		Trimmed.MidInline(1, Trimmed.Len() - 2);

		// "1", " 0", " 0", " 1"
		TArray<FString> StringifiedValues;
		Trimmed.ParseIntoArray(StringifiedValues, TEXT(","));
		if (StringifiedValues.Num() != T::dimension)
		{
			return false;
		}

		TArray<typename T::ScalarType> Values;
		Values.Reserve(StringifiedValues.Num());
		for (const FString& StringifiedValue : StringifiedValues)
		{
			bool bSuccess = true;
			typename T::ScalarType Parsed = pxr::TfUnstringify<typename T::ScalarType>(
				UnrealToUsd::ConvertString(*StringifiedValue).Get(),
				&bSuccess
			);
			if (!bSuccess)
			{
				return false;
			}

			Values.Add(Parsed);
		}

		FMemory::Memcpy(Output.data(), Values.GetData(), Values.Num() * Values.GetTypeSize());

		return true;
	}

	template<typename QuatType, typename VecType>
	bool UnstringifyQuat(const FString& String, QuatType& Output)
	{
		FScopedUsdAllocs Allocs;

		VecType Vec;
		bool bSuccess = UnstringifyVec(String, Vec);
		if (!bSuccess)
		{
			return false;
		}

		QuatType Quat{Vec[0], Vec[1], Vec[2], Vec[3]};
		Output = Quat;

		return true;
	}

	template<typename T>
	bool UnstringifyBasicArray(const FString& String, pxr::VtArray<T>& Output)
	{
		FScopedUsdAllocs Allocs;

		// "[1, 0, 0, 1]"
		FString Trimmed = String.TrimStartAndEnd();

		// "1, 0, 0, 1"
		Trimmed.MidInline(1, Trimmed.Len() - 2);

		// "1", " 0", " 0", " 1"
		TArray<FString> StringifiedValues;
		Trimmed.ParseIntoArray(StringifiedValues, TEXT(","));

		pxr::VtArray<T> Values;
		Values.reserve(StringifiedValues.Num());
		for (const FString& StringifiedValue : StringifiedValues)
		{
			T Parsed;
			if (!UnstringifyInner<T>(StringifiedValue.TrimStartAndEnd(), Parsed))
			{
				return false;
			}

			Values.push_back(Parsed);
		}

		Output = MoveTemp(Values);
		return true;
	}

	// Parses a single string like below (outer quotes omitted for clarity)
	//
	//		[" fir,;st", 's, e@co"ynd', "t), (hird"]
	//
	// Into an array with three elements, " fir,;st", then "s, e@co"ynd", and finally "t), (hird".
	template<typename ArrayType>
	bool ParseStringArray(const FString& String, ArrayType& OutArrayLike)
	{
		// USD string array attributes seem to allow single and double quoted strings, escaped characters,
		// and can obviously have commas an spaces inside each string. This means we need some custom logic
		// to essentially "tokenize" these separate string values out of the entire stringified string array.
		// We pulled this into its separate function so that it can be reused for handling std::vector and
		// pxr::VtArray, both of either std::string and pxr::TfToken.
		//
		// Note: USD does provide utils for this like pxr::TfQuotedStringTokenize, but it seems to fail for
		// lots of our strings, so here we just have our own tokenizer instead.

		using namespace UE::USDValueConversion::Private;

		FScopedUsdAllocs Allocs;

		bool bInsideSingleQuoteString = false;
		bool bInsideDoubleQuoteString = false;
		bool bLastCharWasEscape = false;

		std::string Buffer;
		Buffer.reserve(String.Len());

		std::string StdString = UnrealToUsd::ConvertString(*String).Get();

		ArrayType Values;

		for (const char& Char : StdString)
		{
			const bool bWasInsideStringBeforeChar = bInsideSingleQuoteString || bInsideDoubleQuoteString;

			bool bCharIsDelimiter = false;
			bool bCharIsEscape = false;
			if (!bLastCharWasEscape)
			{
				// USD seems to have the Python behavior where a single quote is a regular character inside a
				// double quote string and vice versa
				if (Char == SingleQuoteChar && !bInsideDoubleQuoteString)
				{
					bCharIsDelimiter = true;
					bInsideSingleQuoteString = !bInsideSingleQuoteString;
				}
				else if (Char == DoubleQuoteChar && !bInsideSingleQuoteString)
				{
					bCharIsDelimiter = true;
					bInsideDoubleQuoteString = !bInsideDoubleQuoteString;
				}
				else if (Char == EscapeChar)
				{
					bCharIsEscape = true;
				}
			}

			// Inside the string, append to our current buffer
			if ((bInsideSingleQuoteString || bInsideDoubleQuoteString) && !bCharIsDelimiter)
			{
				Buffer += Char;
			}
			// Just finished building one of the values into Buffer, let's store it and reset the buffer
			else if (bCharIsDelimiter && bWasInsideStringBeforeChar)
			{
				// Luckily both std::vector and pxr::VtArray typedef a "value_type"
				Values.push_back(typename ArrayType::value_type{Buffer});
				Buffer.clear();
			}

			if (bCharIsEscape)
			{
				bLastCharWasEscape = true;
			}
		}

		OutArrayLike = MoveTemp(Values);
		return true;
	}

	template<typename T>
	bool UnstringifyStringOrTokenArray(const FString& String, pxr::VtArray<T>& Output)
	{
		FScopedUsdAllocs Allocs;

		// Strip the opening and closing brackets
		FString Trimmed = String.TrimStartAndEnd();
		Trimmed.MidInline(1, Trimmed.Len() - 2);

		pxr::VtArray<T> Values;
		bool bSuccess = ParseStringArray(Trimmed, Values);
		if (!bSuccess)
		{
			return false;
		}

		Output = MoveTemp(Values);
		return true;
	}

	// Unstringifies an attribute that can look as messy as this:
	//
	// 		asset[] assetArray = [@asset@, @C:\Use\"rs\'user"'\Deskt  op\layer.usda@ , @./Relative/asset.usdc@]
	//
	// This version is a bit simpler than the std::string and pxr::TfToken version above, because the individual
	// paths must be surrounded by "@", and you can't have an escaped "@" inside the path itself
	bool UnstringifyAssetPathArray(const FString& String, pxr::VtArray<pxr::SdfAssetPath>& Output)
	{
		using namespace UE::USDValueConversion::Private;

		FScopedUsdAllocs Allocs;

		FString Trimmed = String.TrimStartAndEnd();
		Trimmed.MidInline(1, Trimmed.Len() - 2);
		std::string TrimmedStr = UnrealToUsd::ConvertString(*Trimmed).Get();

		bool bInsidePath = false;

		std::string Buffer;
		Buffer.reserve(Trimmed.Len());

		pxr::VtArray<pxr::SdfAssetPath> Values;

		for (const char& Char : TrimmedStr)
		{
			const bool bWasInsideStringBeforeChar = bInsidePath;

			bool bCharIsDelimiter = false;
			if (Char == AssetDelimiterChar)
			{
				bCharIsDelimiter = true;
				bInsidePath = !bInsidePath;
			}

			// Inside the path, append to our current buffer
			if (bInsidePath && !bCharIsDelimiter)
			{
				Buffer += Char;
			}
			// Just finished building one of the values into Buffer
			else if (bCharIsDelimiter && bWasInsideStringBeforeChar)
			{
				Values.push_back(pxr::SdfAssetPath{Buffer});
				Buffer.clear();
			}
		}

		Output = MoveTemp(Values);
		return true;
	}

	// We use these for matrix and vector arrays, where the input string looks like this:
	//
	// 	"[( (10, 7), (2, 3) ), ( (12, 5), (2, 3) ), ( (13, 4), (2, 3) )]"
	//
	template<typename StructType, int NumValuesPerStruct>
	bool UnstringifyNumberArray(const FString& String, pxr::VtArray<StructType>& Output)
	{
		FScopedUsdAllocs Allocs;

		// "[( (10, 7), (2, 3) ), ( (12, 5), (2, 3) ), ( (13, 4), (2, 3) )]"
		FString Trimmed = String.TrimStartAndEnd();

		// "( (10, 7), (2, 3) ), ( (12, 5), (2, 3) ), ( (13, 4), (2, 3) )"
		Trimmed.MidInline(1, Trimmed.Len() - 2);

		// " 10, 7, 2, 3 ,  12, 5, 2, 3 ,  13, 4, 2, 3 "
		Trimmed = Trimmed.Replace(TEXT("("), TEXT("")).Replace(TEXT(")"), TEXT(""));

		// " 10", " 7", " 2", etc.
		TArray<FString> StringifiedValues;
		Trimmed.ParseIntoArray(StringifiedValues, TEXT(","));

		// We're going to parse all values into a single big flat array first and then
		// memcpy them as chunks onto instances of the target struct type (e.g. each 4 floats
		// will be copied onto each GfVec4f), so we expect the number of values to be a multiple
		// of the target struct type size
		if (StringifiedValues.Num() % NumValuesPerStruct != 0)
		{
			return false;
		}
		const int32 NumStructInstances = StringifiedValues.Num() / NumValuesPerStruct;

		TArray<typename StructType::ScalarType> FlattenedValues;
		FlattenedValues.Reserve(StringifiedValues.Num());
		for (const FString& StringifiedValue : StringifiedValues)
		{
			bool bSuccess = true;
			typename StructType::ScalarType Parsed = pxr::TfUnstringify<typename StructType::ScalarType>(
				UnrealToUsd::ConvertString(*StringifiedValue).Get(),
				&bSuccess
			);
			if (!bSuccess)
			{
				return false;
			}

			FlattenedValues.Add(Parsed);
		}

		Output.resize(NumStructInstances);
		for (int32 InstanceIndex = 0; InstanceIndex < NumStructInstances; ++InstanceIndex)
		{
			FMemory::Memcpy(
				Output[InstanceIndex].data(),
				&FlattenedValues[InstanceIndex * NumValuesPerStruct],
				NumValuesPerStruct * sizeof(typename StructType::ScalarType)
			);
		}

		return true;
	}

	// Very similar to UnstringifyNumberArray, except that since the USD quaternion types don't
	// have .data(), we need some minor tweaks
	template<typename StructType, uint32 NumValuesPerStruct>
	bool UnstringifyQuatArray(const FString& String, pxr::VtArray<StructType>& Output)
	{
		FScopedUsdAllocs Allocs;

		// "[( (10, 7), (2, 3) ), ( (12, 5), (2, 3) ), ( (13, 4), (2, 3) )]"
		FString Trimmed = String.TrimStartAndEnd();

		// "( (10, 7), (2, 3) ), ( (12, 5), (2, 3) ), ( (13, 4), (2, 3) )"
		Trimmed.MidInline(1, Trimmed.Len() - 2);

		// " 10, 7, 2, 3 ,  12, 5, 2, 3 ,  13, 4, 2, 3 "
		Trimmed = Trimmed.Replace(TEXT("("), TEXT("")).Replace(TEXT(")"), TEXT(""));

		// " 10", " 7", " 2", etc.
		TArray<FString> StringifiedValues;
		Trimmed.ParseIntoArray(StringifiedValues, TEXT(","));

		// We're going to parse all values into a single big flat array first and then
		// memcpy them as chunks onto instances of the target struct type (e.g. each 4 floats
		// will be copied onto each GfVec4f), so we expect the number of values to be a multiple
		// of the target struct type size
		if (StringifiedValues.Num() % NumValuesPerStruct != 0)
		{
			return false;
		}
		const uint32 NumStructInstances = StringifiedValues.Num() / NumValuesPerStruct;

		TArray<typename StructType::ScalarType> FlattenedValues;
		FlattenedValues.Reserve(StringifiedValues.Num());
		for (const FString& StringifiedValue : StringifiedValues)
		{
			bool bSuccess = true;
			typename StructType::ScalarType Parsed = pxr::TfUnstringify<typename StructType::ScalarType>(
				UnrealToUsd::ConvertString(*StringifiedValue).Get(),
				&bSuccess
			);
			if (!bSuccess)
			{
				return false;
			}

			FlattenedValues.Add(Parsed);
		}

		Output.resize(NumStructInstances);
		for (uint32 InstanceIndex = 0; InstanceIndex < NumStructInstances; ++InstanceIndex)
		{
			const uint32 FlattenedIndex = InstanceIndex * NumValuesPerStruct;

			Output[InstanceIndex] = StructType{
				FlattenedValues[FlattenedIndex + 0],
				FlattenedValues[FlattenedIndex + 1],
				FlattenedValues[FlattenedIndex + 2],
				FlattenedValues[FlattenedIndex + 3]
			};
		}

		return true;
	}

	// Unstringifies the types of string we get when stringifying the apiSchemas metadata values (or likely
	// other list-edited fields too). An actual example string looks like this:
	//
	// 		SdfTokenListOp(Explicit Items: ["SkelBindingAPI", "SomeAPI"])
	//
	// I have never seen anything other than "Explicit Items", and it seems to be the fully composed
	// list (e.g. even if we have multiple layers appending, prepending and deleting itiems we just get the
	// resolved result inside that "ExplicitItems" array).
	//
	// Note that if you have USD stringify this type directly (via pxr::TfStringify()) it will omit the
	// quotes around the individual API schemas. Our stringify functions will keep those, however, as
	// it makes them unstringifiable
	bool UnstringifyListOpTokens(const FString& String, pxr::SdfListOp<pxr::TfToken>& Output)
	{
		using namespace UE::USDValueConversion::Private;

		FScopedUsdAllocs Allocs;

		// "SdfTokenListOp(Explicit Items: ["SkelBindingAPI", "SomeAPI"])"
		FString Trimmed = String.TrimStartAndEnd();
		if (!Trimmed.StartsWith(TokenListStringStart))
		{
			return false;
		}

		// "SkelBindingAPI", "SomeAPI"
		const size_t NumCharsInPrefix = TokenListStringStart.Len();
		Trimmed.MidInline(NumCharsInPrefix, Trimmed.Len() - NumCharsInPrefix - 2);

		std::vector<pxr::TfToken> Values;
		bool bSuccess = ParseStringArray(Trimmed, Values);
		if (!bSuccess)
		{
			return false;
		}

		Output = pxr::SdfListOp<pxr::TfToken>::CreateExplicit(Values);
		return true;
	}

	// We use this to adapt the unstringify function signatures so that we can use them directly when
	// we want to get the converted type right away, but also get a type-erased VtValue that we can
	// use in Unstringify()
	template<typename USDType, bool (*Converter)(const FString&, USDType&)>
	bool WrapInVtValue(const FString& String, pxr::VtValue& OutValue)
	{
		FScopedUsdAllocs Allocs;

		USDType ConvertedValue;
		if (Converter(String, ConvertedValue))
		{
			OutValue = ConvertedValue;
			return true;
		}

		return false;
	}
#endif	  // USE_USD_SDK
}	 // namespace UE::USDValueConversion::Private

namespace UsdUtils
{
#if USE_USD_SDK
	bool Unstringify(const FString& String, const FString& TypeName, pxr::VtValue& Output)
	{
		if (String.IsEmpty() || TypeName.IsEmpty())
		{
			return false;
		}

		using namespace UE::USDValueConversion::Private;

		static const TMap<FString, TFunction<bool(const FString&, pxr::VtValue&)>> Unstringifiers = {
			// Basic datatypes
			{TEXT("bool"), WrapInVtValue<bool, UnstringifyInner<bool>>},
			{TEXT("uchar"), WrapInVtValue<uint8_t, UnstringifyInner<uint8_t>>},
			{TEXT("int"), WrapInVtValue<int32_t, UnstringifyInner<int32_t>>},
			{TEXT("uint"), WrapInVtValue<uint32_t, UnstringifyInner<uint32_t>>},
			{TEXT("int64"), WrapInVtValue<int64_t, UnstringifyInner<int64_t>>},
			{TEXT("uint64"), WrapInVtValue<uint64_t, UnstringifyInner<uint64_t>>},
			{TEXT("half"), WrapInVtValue<pxr::GfHalf, UnstringifyInner<pxr::GfHalf>>},
			{TEXT("float"), WrapInVtValue<float, UnstringifyInner<float>>},
			{TEXT("double"), WrapInVtValue<double, UnstringifyInner<double>>},
			{TEXT("timecode"), WrapInVtValue<pxr::SdfTimeCode, UnstringifyInner<pxr::SdfTimeCode>>},
			{TEXT("string"), WrapInVtValue<std::string, UnstringifyInner<std::string>>},
			{TEXT("token"), WrapInVtValue<pxr::TfToken, UnstringifyInner<pxr::TfToken>>},
			{TEXT("asset"), WrapInVtValue<pxr::SdfAssetPath, UnstringifyInner<pxr::SdfAssetPath>>},
			{TEXT("matrix2d"), WrapInVtValue<pxr::GfMatrix2d, UnstringifyMatrix<pxr::GfMatrix2d>>},
			{TEXT("matrix3d"), WrapInVtValue<pxr::GfMatrix3d, UnstringifyMatrix<pxr::GfMatrix3d>>},
			{TEXT("matrix4d"), WrapInVtValue<pxr::GfMatrix4d, UnstringifyMatrix<pxr::GfMatrix4d>>},
			{TEXT("quatd"), WrapInVtValue<pxr::GfQuatd, UnstringifyQuat<pxr::GfQuatd, pxr::GfVec4d>>},
			{TEXT("quatf"), WrapInVtValue<pxr::GfQuatf, UnstringifyQuat<pxr::GfQuatf, pxr::GfVec4f>>},
			{TEXT("quath"), WrapInVtValue<pxr::GfQuath, UnstringifyQuat<pxr::GfQuath, pxr::GfVec4h>>},
			{TEXT("double2"), WrapInVtValue<pxr::GfVec2d, UnstringifyVec<pxr::GfVec2d>>},
			{TEXT("float2"), WrapInVtValue<pxr::GfVec2f, UnstringifyVec<pxr::GfVec2f>>},
			{TEXT("half2"), WrapInVtValue<pxr::GfVec2h, UnstringifyVec<pxr::GfVec2h>>},
			{TEXT("int2"), WrapInVtValue<pxr::GfVec2i, UnstringifyVec<pxr::GfVec2i>>},
			{TEXT("double3"), WrapInVtValue<pxr::GfVec3d, UnstringifyVec<pxr::GfVec3d>>},
			{TEXT("float3"), WrapInVtValue<pxr::GfVec3f, UnstringifyVec<pxr::GfVec3f>>},
			{TEXT("half3"), WrapInVtValue<pxr::GfVec3h, UnstringifyVec<pxr::GfVec3h>>},
			{TEXT("int3"), WrapInVtValue<pxr::GfVec3i, UnstringifyVec<pxr::GfVec3i>>},
			{TEXT("double4"), WrapInVtValue<pxr::GfVec4d, UnstringifyVec<pxr::GfVec4d>>},
			{TEXT("float4"), WrapInVtValue<pxr::GfVec4f, UnstringifyVec<pxr::GfVec4f>>},
			{TEXT("half4"), WrapInVtValue<pxr::GfVec4h, UnstringifyVec<pxr::GfVec4h>>},
			{TEXT("int4"), WrapInVtValue<pxr::GfVec4i, UnstringifyVec<pxr::GfVec4i>>},

			// Array versions of the ones above
			{TEXT("bool[]"), WrapInVtValue<pxr::VtArray<bool>, UnstringifyBasicArray<bool>>},
			{TEXT("uchar[]"), WrapInVtValue<pxr::VtArray<uint8_t>, UnstringifyBasicArray<uint8_t>>},
			{TEXT("int[]"), WrapInVtValue<pxr::VtArray<int32_t>, UnstringifyBasicArray<int32_t>>},
			{TEXT("uint[]"), WrapInVtValue<pxr::VtArray<uint32_t>, UnstringifyBasicArray<uint32_t>>},
			{TEXT("int64[]"), WrapInVtValue<pxr::VtArray<int64_t>, UnstringifyBasicArray<int64_t>>},
			{TEXT("uint64[]"), WrapInVtValue<pxr::VtArray<uint64_t>, UnstringifyBasicArray<uint64_t>>},
			{TEXT("half[]"), WrapInVtValue<pxr::VtArray<pxr::GfHalf>, UnstringifyBasicArray<pxr::GfHalf>>},
			{TEXT("float[]"), WrapInVtValue<pxr::VtArray<float>, UnstringifyBasicArray<float>>},
			{TEXT("double[]"), WrapInVtValue<pxr::VtArray<double>, UnstringifyBasicArray<double>>},
			{TEXT("timecode[]"), WrapInVtValue<pxr::VtArray<pxr::SdfTimeCode>, UnstringifyBasicArray<pxr::SdfTimeCode>>},
			{TEXT("string[]"), WrapInVtValue<pxr::VtArray<std::string>, UnstringifyStringOrTokenArray<std::string>>},
			{TEXT("token[]"), WrapInVtValue<pxr::VtArray<pxr::TfToken>, UnstringifyStringOrTokenArray<pxr::TfToken>>},
			{TEXT("asset[]"), WrapInVtValue<pxr::VtArray<pxr::SdfAssetPath>, UnstringifyAssetPathArray>},
			{TEXT("matrix2d[]"), WrapInVtValue<pxr::VtArray<pxr::GfMatrix2d>, UnstringifyNumberArray<pxr::GfMatrix2d, 4>>},
			{TEXT("matrix3d[]"), WrapInVtValue<pxr::VtArray<pxr::GfMatrix3d>, UnstringifyNumberArray<pxr::GfMatrix3d, 9>>},
			{TEXT("matrix4d[]"), WrapInVtValue<pxr::VtArray<pxr::GfMatrix4d>, UnstringifyNumberArray<pxr::GfMatrix4d, 16>>},
			{TEXT("quatd[]"), WrapInVtValue<pxr::VtArray<pxr::GfQuatd>, UnstringifyQuatArray<pxr::GfQuatd, 4>>},
			{TEXT("quatf[]"), WrapInVtValue<pxr::VtArray<pxr::GfQuatf>, UnstringifyQuatArray<pxr::GfQuatf, 4>>},
			{TEXT("quath[]"), WrapInVtValue<pxr::VtArray<pxr::GfQuath>, UnstringifyQuatArray<pxr::GfQuath, 4>>},
			{TEXT("double2[]"), WrapInVtValue<pxr::VtArray<pxr::GfVec2d>, UnstringifyNumberArray<pxr::GfVec2d, 2>>},
			{TEXT("float2[]"), WrapInVtValue<pxr::VtArray<pxr::GfVec2f>, UnstringifyNumberArray<pxr::GfVec2f, 2>>},
			{TEXT("half2[]"), WrapInVtValue<pxr::VtArray<pxr::GfVec2h>, UnstringifyNumberArray<pxr::GfVec2h, 2>>},
			{TEXT("int2[]"), WrapInVtValue<pxr::VtArray<pxr::GfVec2i>, UnstringifyNumberArray<pxr::GfVec2i, 2>>},
			{TEXT("double3[]"), WrapInVtValue<pxr::VtArray<pxr::GfVec3d>, UnstringifyNumberArray<pxr::GfVec3d, 3>>},
			{TEXT("float3[]"), WrapInVtValue<pxr::VtArray<pxr::GfVec3f>, UnstringifyNumberArray<pxr::GfVec3f, 3>>},
			{TEXT("half3[]"), WrapInVtValue<pxr::VtArray<pxr::GfVec3h>, UnstringifyNumberArray<pxr::GfVec3h, 3>>},
			{TEXT("int3[]"), WrapInVtValue<pxr::VtArray<pxr::GfVec3i>, UnstringifyNumberArray<pxr::GfVec3i, 3>>},
			{TEXT("double4[]"), WrapInVtValue<pxr::VtArray<pxr::GfVec4d>, UnstringifyNumberArray<pxr::GfVec4d, 4>>},
			{TEXT("float4[]"), WrapInVtValue<pxr::VtArray<pxr::GfVec4f>, UnstringifyNumberArray<pxr::GfVec4f, 4>>},
			{TEXT("half4[]"), WrapInVtValue<pxr::VtArray<pxr::GfVec4h>, UnstringifyNumberArray<pxr::GfVec4h, 4>>},
			{TEXT("int4[]"), WrapInVtValue<pxr::VtArray<pxr::GfVec4i>, UnstringifyNumberArray<pxr::GfVec4i, 4>>},

			// Exotic types found in some scenarios
			{TEXT("SdfListOp<TfToken>"), WrapInVtValue<pxr::SdfListOp<pxr::TfToken>, UnstringifyListOpTokens>}, // This is used for apiSchemas
																												   // metadata
		};

		if (const TFunction<bool(const FString&, pxr::VtValue&)>* FoundUnstringifier = Unstringifiers.Find(TypeName))
		{
			FScopedUsdAllocs Allocs;

			bool bSuccess = (*FoundUnstringifier)(String, Output);
			if (bSuccess)
			{
				return true;
			}
			else
			{
				USD_LOG_ERROR(TEXT("Found unstringifier for typeName '%s', but failed to unstringify string '%s'"), *TypeName, *String);
			}
		}
		else
		{
			USD_LOG_ERROR(TEXT("Failed to find unstringifier for typeName '%s'"), *TypeName);
		}

		return false;
	}
#endif	  // USE_USD_SDK

	using namespace UE::USDValueConversion::Private;

	bool UnstringifyAsBool(const FString& String, bool& OutValue)
	{
#if USE_USD_SDK
		return UnstringifyInner<bool>(String, OutValue);
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool UnstringifyAsUChar(const FString& String, uint8& OutValue)
	{
#if USE_USD_SDK
		return UnstringifyInner<uint8_t>(String, OutValue);
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool UnstringifyAsInt(const FString& String, int32& OutValue)
	{
#if USE_USD_SDK
		return UnstringifyInner<int32_t>(String, OutValue);
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool UnstringifyAsUInt(const FString& String, uint32& OutValue)
	{
#if USE_USD_SDK
		return UnstringifyInner<uint32_t>(String, OutValue);
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool UnstringifyAsInt64(const FString& String, int64& OutValue)
	{
#if USE_USD_SDK
		// Note we're using the UE int64 here instead of int64_t, because on Linux
		// you can't cast references from one into the other
		return UnstringifyInner<int64>(String, OutValue);
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool UnstringifyAsUInt64(const FString& String, uint64& OutValue)
	{
#if USE_USD_SDK
		// Note we're using the UE uint64 here instead of uint64_t, because on Linux
		// you can't cast references from one into the other
		return UnstringifyInner<uint64>(String, OutValue);
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool UnstringifyAsHalf(const FString& String, float& OutValue)
	{
#if USE_USD_SDK
		pxr::GfHalf InnerValue;
		if (UnstringifyInner<pxr::GfHalf>(String, InnerValue))
		{
			OutValue = static_cast<float>(InnerValue);
			return true;
		}
#endif	  // USE_USD_SDK

		return false;
	}

	bool UnstringifyAsFloat(const FString& String, float& OutValue)
	{
#if USE_USD_SDK
		return UnstringifyInner<float>(String, OutValue);
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool UnstringifyAsDouble(const FString& String, double& OutValue)
	{
#if USE_USD_SDK
		return UnstringifyInner<double>(String, OutValue);
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool UnstringifyAsTimeCode(const FString& String, double& OutValue)
	{
#if USE_USD_SDK
		pxr::SdfTimeCode InnerValue;
		if (UnstringifyInner<pxr::SdfTimeCode>(String, InnerValue))
		{
			OutValue = static_cast<double>(InnerValue);
			return true;
		}
#endif	  // USE_USD_SDK

		return false;
	}

	bool UnstringifyAsString(const FString& String, FString& OutValue)
	{
#if USE_USD_SDK
		OutValue = String;
		return true;
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool UnstringifyAsToken(const FString& String, FString& OutValue)
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		pxr::TfToken InnerValue;
		if (UnstringifyInner<pxr::TfToken>(String, InnerValue))
		{
			OutValue = UsdToUnreal::ConvertToken(InnerValue);
			return true;
		}
#endif	  // USE_USD_SDK

		return false;
	}

	bool UnstringifyAsAssetPath(const FString& String, FString& OutValue)
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		pxr::SdfAssetPath InnerValue;
		if (UnstringifyInner<pxr::SdfAssetPath>(String, InnerValue))
		{
			OutValue = UsdToUnreal::ConvertString(InnerValue.GetAssetPath());
			return true;
		}
#endif	  // USE_USD_SDK

		return false;
	}

	bool UnstringifyAsMatrix2d(const FString& String, FMatrix2D& OutValue)
	{
#if USE_USD_SDK
		pxr::GfMatrix2d InnerValue;
		if (UnstringifyMatrix<pxr::GfMatrix2d>(String, InnerValue))
		{
			OutValue = UsdToUnreal::ConvertMatrix(InnerValue);
			return true;
		}
#endif	  // USE_USD_SDK

		return false;
	}

	bool UnstringifyAsMatrix3d(const FString& String, FMatrix3D& OutValue)
	{
#if USE_USD_SDK
		pxr::GfMatrix3d InnerValue;
		if (UnstringifyMatrix<pxr::GfMatrix3d>(String, InnerValue))
		{
			OutValue = UsdToUnreal::ConvertMatrix(InnerValue);
			return true;
		}
#endif	  // USE_USD_SDK

		return false;
	}

	bool UnstringifyAsMatrix4d(const FString& String, FMatrix& OutValue)
	{
#if USE_USD_SDK
		pxr::GfMatrix4d InnerValue;
		if (UnstringifyMatrix<pxr::GfMatrix4d>(String, InnerValue))
		{
			OutValue = UsdToUnreal::ConvertMatrix(InnerValue);
			return true;
		}
#endif	  // USE_USD_SDK

		return false;
	}

	bool UnstringifyAsQuatd(const FString& String, FQuat& OutValue)
	{
#if USE_USD_SDK
		pxr::GfQuatd InnerValue;
		if (UnstringifyQuat<pxr::GfQuatd, pxr::GfVec4d>(String, InnerValue))
		{
			OutValue = UsdToUnreal::ConvertQuat(InnerValue);
			return true;
		}
#endif	  // USE_USD_SDK

		return false;
	}

	bool UnstringifyAsQuatf(const FString& String, FQuat& OutValue)
	{
#if USE_USD_SDK
		pxr::GfQuatf InnerValue;
		if (UnstringifyQuat<pxr::GfQuatf, pxr::GfVec4f>(String, InnerValue))
		{
			OutValue = UsdToUnreal::ConvertQuat(InnerValue);
			return true;
		}
#endif	  // USE_USD_SDK

		return false;
	}

	bool UnstringifyAsQuath(const FString& String, FQuat& OutValue)
	{
#if USE_USD_SDK
		pxr::GfQuath InnerValue;
		if (UnstringifyQuat<pxr::GfQuath, pxr::GfVec4h>(String, InnerValue))
		{
			OutValue = UsdToUnreal::ConvertQuat(InnerValue);
			return true;
		}
#endif	  // USE_USD_SDK

		return false;
	}

	bool UnstringifyAsDouble2(const FString& String, FVector2D& OutValue)
	{
#if USE_USD_SDK
		pxr::GfVec2d InnerValue;
		if (UnstringifyVec<pxr::GfVec2d>(String, InnerValue))
		{
			OutValue = UsdToUnreal::ConvertVector(InnerValue);
			return true;
		}
#endif	  // USE_USD_SDK

		return false;
	}

	bool UnstringifyAsFloat2(const FString& String, FVector2D& OutValue)
	{
#if USE_USD_SDK
		pxr::GfVec2f InnerValue;
		if (UnstringifyVec<pxr::GfVec2f>(String, InnerValue))
		{
			OutValue = UsdToUnreal::ConvertVector(InnerValue);
			return true;
		}
#endif	  // USE_USD_SDK

		return false;
	}

	bool UnstringifyAsHalf2(const FString& String, FVector2D& OutValue)
	{
#if USE_USD_SDK
		pxr::GfVec2h InnerValue;
		if (UnstringifyVec<pxr::GfVec2h>(String, InnerValue))
		{
			OutValue = UsdToUnreal::ConvertVector(InnerValue);
			return true;
		}
#endif	  // USE_USD_SDK

		return false;
	}

	bool UnstringifyAsInt2(const FString& String, FIntPoint& OutValue)
	{
#if USE_USD_SDK
		pxr::GfVec2i InnerValue;
		if (UnstringifyVec<pxr::GfVec2i>(String, InnerValue))
		{
			OutValue = UsdToUnreal::ConvertVector(InnerValue);
			return true;
		}
#endif	  // USE_USD_SDK

		return false;
	}

	bool UnstringifyAsDouble3(const FString& String, FVector& OutValue)
	{
#if USE_USD_SDK
		pxr::GfVec3d InnerValue;
		if (UnstringifyVec<pxr::GfVec3d>(String, InnerValue))
		{
			OutValue = UsdToUnreal::ConvertVector(InnerValue);
			return true;
		}
#endif	  // USE_USD_SDK

		return false;
	}

	bool UnstringifyAsFloat3(const FString& String, FVector& OutValue)
	{
#if USE_USD_SDK
		pxr::GfVec3f InnerValue;
		if (UnstringifyVec<pxr::GfVec3f>(String, InnerValue))
		{
			OutValue = UsdToUnreal::ConvertVector(InnerValue);
			return true;
		}
#endif	  // USE_USD_SDK

		return false;
	}

	bool UnstringifyAsHalf3(const FString& String, FVector& OutValue)
	{
#if USE_USD_SDK
		pxr::GfVec3h InnerValue;
		if (UnstringifyVec<pxr::GfVec3h>(String, InnerValue))
		{
			OutValue = UsdToUnreal::ConvertVector(InnerValue);
			return true;
		}
#endif	  // USE_USD_SDK

		return false;
	}

	bool UnstringifyAsInt3(const FString& String, FIntVector& OutValue)
	{
#if USE_USD_SDK
		pxr::GfVec3i InnerValue;
		if (UnstringifyVec<pxr::GfVec3i>(String, InnerValue))
		{
			OutValue = UsdToUnreal::ConvertVector(InnerValue);
			return true;
		}
#endif	  // USE_USD_SDK

		return false;
	}

	bool UnstringifyAsDouble4(const FString& String, FVector4& OutValue)
	{
#if USE_USD_SDK
		pxr::GfVec4d InnerValue;
		if (UnstringifyVec<pxr::GfVec4d>(String, InnerValue))
		{
			OutValue = UsdToUnreal::ConvertVector(InnerValue);
			return true;
		}
#endif	  // USE_USD_SDK

		return false;
	}

	bool UnstringifyAsFloat4(const FString& String, FVector4& OutValue)
	{
#if USE_USD_SDK
		pxr::GfVec4f InnerValue;
		if (UnstringifyVec<pxr::GfVec4f>(String, InnerValue))
		{
			OutValue = UsdToUnreal::ConvertVector(InnerValue);
			return true;
		}
#endif	  // USE_USD_SDK

		return false;
	}

	bool UnstringifyAsHalf4(const FString& String, FVector4& OutValue)
	{
#if USE_USD_SDK
		pxr::GfVec4h InnerValue;
		if (UnstringifyVec<pxr::GfVec4h>(String, InnerValue))
		{
			OutValue = UsdToUnreal::ConvertVector(InnerValue);
			return true;
		}
#endif	  // USE_USD_SDK

		return false;
	}

	bool UnstringifyAsInt4(const FString& String, FIntVector4& OutValue)
	{
#if USE_USD_SDK
		pxr::GfVec4i InnerValue;
		if (UnstringifyVec<pxr::GfVec4i>(String, InnerValue))
		{
			OutValue = UsdToUnreal::ConvertVector(InnerValue);
			return true;
		}
#endif	  // USE_USD_SDK

		return false;
	}

#if USE_USD_SDK
	template<typename UEType, typename USDType>
	bool UnstringifyBasicArrayWithMemcpy(const FString& String, TArray<UEType>& OutValue)
	{
		FScopedUsdAllocs Allocs;

		static_assert(sizeof(UEType) == sizeof(USDType));

		pxr::VtArray<USDType> InnerValue;
		if (UnstringifyBasicArray<USDType>(String, InnerValue))
		{
			OutValue.SetNum(InnerValue.size());
			FMemory::Memcpy(OutValue.GetData(), InnerValue.cdata(), OutValue.Num() * OutValue.GetTypeSize());
			return true;
		}

		return false;
	}
#endif	  // USE_USD_SDK

	bool UnstringifyAsBoolArray(const FString& String, TArray<bool>& OutValue)
	{
#if USE_USD_SDK
		return UnstringifyBasicArrayWithMemcpy<bool, bool>(String, OutValue);
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool UnstringifyAsUCharArray(const FString& String, TArray<uint8>& OutValue)
	{
#if USE_USD_SDK
		return UnstringifyBasicArrayWithMemcpy<uint8, uint8_t>(String, OutValue);
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool UnstringifyAsIntArray(const FString& String, TArray<int32>& OutValue)
	{
#if USE_USD_SDK
		return UnstringifyBasicArrayWithMemcpy<int32, int32_t>(String, OutValue);
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool UnstringifyAsUIntArray(const FString& String, TArray<uint32>& OutValue)
	{
#if USE_USD_SDK
		return UnstringifyBasicArrayWithMemcpy<uint32, uint32_t>(String, OutValue);
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool UnstringifyAsInt64Array(const FString& String, TArray<int64>& OutValue)
	{
#if USE_USD_SDK
		return UnstringifyBasicArrayWithMemcpy<int64, int64_t>(String, OutValue);
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool UnstringifyAsUInt64Array(const FString& String, TArray<uint64>& OutValue)
	{
#if USE_USD_SDK
		return UnstringifyBasicArrayWithMemcpy<uint64, uint64_t>(String, OutValue);
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool UnstringifyAsHalfArray(const FString& String, TArray<float>& OutValue)
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		pxr::VtArray<pxr::GfHalf> InnerValue;
		if (UnstringifyBasicArray(String, InnerValue))
		{
			OutValue.Empty(InnerValue.size());
			for (pxr::GfHalf Half : InnerValue)
			{
				OutValue.Add(static_cast<float>(Half));
			}
			return true;
		}
#endif	  // USE_USD_SDK

		return false;
	}

	bool UnstringifyAsFloatArray(const FString& String, TArray<float>& OutValue)
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;
		return UnstringifyBasicArrayWithMemcpy<float, float>(String, OutValue);
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool UnstringifyAsDoubleArray(const FString& String, TArray<double>& OutValue)
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;
		return UnstringifyBasicArrayWithMemcpy<double, double>(String, OutValue);
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool UnstringifyAsTimeCodeArray(const FString& String, TArray<double>& OutValue)
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;
		return UnstringifyBasicArrayWithMemcpy<double, pxr::SdfTimeCode>(String, OutValue);
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool UnstringifyAsStringArray(const FString& String, TArray<FString>& OutValue)
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		pxr::VtArray<std::string> InnerValue;
		if (UnstringifyStringOrTokenArray(String, InnerValue))
		{
			OutValue.Empty(InnerValue.size());
			for (const std::string& InnerElement : InnerValue)
			{
				OutValue.Add(UsdToUnreal::ConvertString(InnerElement));
			}
			return true;
		}
#endif	  // USE_USD_SDK

		return false;
	}

	bool UnstringifyAsTokenArray(const FString& String, TArray<FString>& OutValue)
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		pxr::VtArray<pxr::TfToken> InnerValue;
		if (UnstringifyStringOrTokenArray(String, InnerValue))
		{
			OutValue.Empty(InnerValue.size());
			for (const pxr::TfToken& InnerElement : InnerValue)
			{
				OutValue.Add(UsdToUnreal::ConvertString(InnerElement));
			}
			return true;
		}
#endif	  // USE_USD_SDK

		return false;
	}

	bool UnstringifyAsAssetPathArray(const FString& String, TArray<FString>& OutValue)
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		pxr::VtArray<pxr::SdfAssetPath> InnerValue;
		if (UnstringifyAssetPathArray(String, InnerValue))
		{
			OutValue.Empty(InnerValue.size());
			for (const pxr::SdfAssetPath& InnerElement : InnerValue)
			{
				OutValue.Add(UsdToUnreal::ConvertString(InnerElement.GetAssetPath()));
			}
			return true;
		}
#endif	  // USE_USD_SDK

		return false;
	}

	bool UnstringifyAsListOpTokens(const FString& String, TArray<FString>& OutValue)
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		pxr::SdfListOp<pxr::TfToken> InnerValue;
		if (UnstringifyListOpTokens(String, InnerValue))
		{
			std::vector<pxr::TfToken> ItemVector = InnerValue.GetAppliedItems();
			OutValue.Empty(ItemVector.size());
			for (const pxr::TfToken& InnerElement : ItemVector)
			{
				OutValue.Add(UsdToUnreal::ConvertToken(InnerElement));
			}
			return true;
		}
#endif	  // USE_USD_SDK

		return false;
	}

	bool UnstringifyAsMatrix2dArray(const FString& String, TArray<FMatrix2D>& OutValue)
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		pxr::VtArray<pxr::GfMatrix2d> InnerValue;
		if (UnstringifyNumberArray<pxr::GfMatrix2d, 4>(String, InnerValue))
		{
			OutValue.Empty(InnerValue.size());
			for (const pxr::GfMatrix2d& InnerElement : InnerValue)
			{
				OutValue.Add(UsdToUnreal::ConvertMatrix(InnerElement));
			}
			return true;
		}
#endif	  // USE_USD_SDK

		return false;
	}

	bool UnstringifyAsMatrix3dArray(const FString& String, TArray<FMatrix3D>& OutValue)
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		pxr::VtArray<pxr::GfMatrix3d> InnerValue;
		if (UnstringifyNumberArray<pxr::GfMatrix3d, 9>(String, InnerValue))
		{
			OutValue.Empty(InnerValue.size());
			for (const pxr::GfMatrix3d& InnerElement : InnerValue)
			{
				OutValue.Add(UsdToUnreal::ConvertMatrix(InnerElement));
			}
			return true;
		}
#endif	  // USE_USD_SDK

		return false;
	}

	bool UnstringifyAsMatrix4dArray(const FString& String, TArray<FMatrix>& OutValue)
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		pxr::VtArray<pxr::GfMatrix4d> InnerValue;
		if (UnstringifyNumberArray<pxr::GfMatrix4d, 16>(String, InnerValue))
		{
			OutValue.Empty(InnerValue.size());
			for (const pxr::GfMatrix4d& InnerElement : InnerValue)
			{
				OutValue.Add(UsdToUnreal::ConvertMatrix(InnerElement));
			}
			return true;
		}
#endif	  // USE_USD_SDK

		return false;
	}

	bool UnstringifyAsQuatdArray(const FString& String, TArray<FQuat>& OutValue)
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		pxr::VtArray<pxr::GfQuatd> InnerValue;
		if (UnstringifyQuatArray<pxr::GfQuatd, 4>(String, InnerValue))
		{
			OutValue.Empty(InnerValue.size());
			for (const pxr::GfQuatd& InnerElement : InnerValue)
			{
				OutValue.Add(UsdToUnreal::ConvertQuat(InnerElement));
			}
			return true;
		}
#endif	  // USE_USD_SDK

		return false;
	}

	bool UnstringifyAsQuatfArray(const FString& String, TArray<FQuat>& OutValue)
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		pxr::VtArray<pxr::GfQuatf> InnerValue;
		if (UnstringifyQuatArray<pxr::GfQuatf, 4>(String, InnerValue))
		{
			OutValue.Empty(InnerValue.size());
			for (const pxr::GfQuatf& InnerElement : InnerValue)
			{
				OutValue.Add(UsdToUnreal::ConvertQuat(InnerElement));
			}
			return true;
		}
#endif	  // USE_USD_SDK

		return false;
	}

	bool UnstringifyAsQuathArray(const FString& String, TArray<FQuat>& OutValue)
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		pxr::VtArray<pxr::GfQuath> InnerValue;
		if (UnstringifyQuatArray<pxr::GfQuath, 4>(String, InnerValue))
		{
			OutValue.Empty(InnerValue.size());
			for (const pxr::GfQuath& InnerElement : InnerValue)
			{
				OutValue.Add(UsdToUnreal::ConvertQuat(InnerElement));
			}
			return true;
		}
#endif	  // USE_USD_SDK

		return false;
	}

#if USE_USD_SDK
	template<typename USDType, typename UEType>
	bool UnstringifyVectorArray(const FString& String, TArray<UEType>& OutValue)
	{
		FScopedUsdAllocs Allocs;

		pxr::VtArray<USDType> InnerValue;
		if (UnstringifyNumberArray<USDType, USDType::dimension>(String, InnerValue))
		{
			OutValue.Empty(InnerValue.size());
			for (const USDType& InnerElement : InnerValue)
			{
				OutValue.Add(UsdToUnreal::ConvertVector(InnerElement));
			}
			return true;
		}

		return false;
	}
#endif	  // USE_USD_SDK

	bool UnstringifyAsDouble2Array(const FString& String, TArray<FVector2D>& OutValue)
	{
#if USE_USD_SDK
		return UnstringifyVectorArray<pxr::GfVec2d>(String, OutValue);
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool UnstringifyAsFloat2Array(const FString& String, TArray<FVector2D>& OutValue)
	{
#if USE_USD_SDK
		return UnstringifyVectorArray<pxr::GfVec2f>(String, OutValue);
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool UnstringifyAsHalf2Array(const FString& String, TArray<FVector2D>& OutValue)
	{
#if USE_USD_SDK
		return UnstringifyVectorArray<pxr::GfVec2h>(String, OutValue);
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool UnstringifyAsInt2Array(const FString& String, TArray<FIntPoint>& OutValue)
	{
#if USE_USD_SDK
		return UnstringifyVectorArray<pxr::GfVec2i>(String, OutValue);
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool UnstringifyAsDouble3Array(const FString& String, TArray<FVector>& OutValue)
	{
#if USE_USD_SDK
		return UnstringifyVectorArray<pxr::GfVec3d>(String, OutValue);
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool UnstringifyAsFloat3Array(const FString& String, TArray<FVector>& OutValue)
	{
#if USE_USD_SDK
		return UnstringifyVectorArray<pxr::GfVec3f>(String, OutValue);
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool UnstringifyAsHalf3Array(const FString& String, TArray<FVector>& OutValue)
	{
#if USE_USD_SDK
		return UnstringifyVectorArray<pxr::GfVec3h>(String, OutValue);
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool UnstringifyAsInt3Array(const FString& String, TArray<FIntVector>& OutValue)
	{
#if USE_USD_SDK
		return UnstringifyVectorArray<pxr::GfVec3i>(String, OutValue);
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool UnstringifyAsDouble4Array(const FString& String, TArray<FVector4>& OutValue)
	{
#if USE_USD_SDK
		return UnstringifyVectorArray<pxr::GfVec4d>(String, OutValue);
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool UnstringifyAsFloat4Array(const FString& String, TArray<FVector4>& OutValue)
	{
#if USE_USD_SDK
		return UnstringifyVectorArray<pxr::GfVec4f>(String, OutValue);
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool UnstringifyAsHalf4Array(const FString& String, TArray<FVector4>& OutValue)
	{
#if USE_USD_SDK
		return UnstringifyVectorArray<pxr::GfVec4h>(String, OutValue);
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool UnstringifyAsInt4Array(const FString& String, TArray<FIntVector4>& OutValue)
	{
#if USE_USD_SDK
		return UnstringifyVectorArray<pxr::GfVec4i>(String, OutValue);
#else
		return false;
#endif	  // USE_USD_SDK
	}

	template<typename T>
	TOptional<T> GetUnderlyingValue(const UE::FVtValue& InValue)
	{
#if USE_USD_SDK
		const pxr::VtValue& UsdValue = InValue.GetUsdValue();
		if (UsdValue.IsHolding<T>())
		{
			return {UsdValue.UncheckedGet<T>()};
		}
#endif	  // USE_USD_SDK

		return {};
	}
	template USDUTILITIES_API TOptional<bool> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<uint8_t> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<int32_t> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<uint32_t> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<int64_t> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<uint64_t> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<float> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<double> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<std::string> GetUnderlyingValue(const UE::FVtValue& InValue);
#if USE_USD_SDK
	template USDUTILITIES_API TOptional<pxr::GfHalf> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::SdfTimeCode> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::TfToken> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::SdfAssetPath> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::GfMatrix2d> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::GfMatrix3d> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::GfMatrix4d> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::GfQuatd> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::GfQuatf> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::GfQuath> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::GfVec2d> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::GfVec2f> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::GfVec2h> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::GfVec2i> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::GfVec3d> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::GfVec3f> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::GfVec3h> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::GfVec3i> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::GfVec4d> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::GfVec4f> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::GfVec4h> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::GfVec4i> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::VtArray<bool>> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::VtArray<uint8_t>> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::VtArray<int32_t>> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::VtArray<uint32_t>> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::VtArray<int64_t>> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::VtArray<uint64_t>> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::VtArray<pxr::GfHalf>> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::VtArray<float>> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::VtArray<double>> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::VtArray<pxr::SdfTimeCode>> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::VtArray<std::string>> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::VtArray<pxr::TfToken>> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::VtArray<pxr::SdfAssetPath>> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::VtArray<pxr::GfMatrix2d>> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::VtArray<pxr::GfMatrix3d>> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::VtArray<pxr::GfMatrix4d>> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::VtArray<pxr::GfQuatd>> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::VtArray<pxr::GfQuatf>> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::VtArray<pxr::GfQuath>> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::VtArray<pxr::GfVec2d>> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::VtArray<pxr::GfVec2f>> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::VtArray<pxr::GfVec2h>> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::VtArray<pxr::GfVec2i>> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::VtArray<pxr::GfVec3d>> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::VtArray<pxr::GfVec3f>> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::VtArray<pxr::GfVec3h>> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::VtArray<pxr::GfVec3i>> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::VtArray<pxr::GfVec4d>> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::VtArray<pxr::GfVec4f>> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::VtArray<pxr::GfVec4h>> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::VtArray<pxr::GfVec4i>> GetUnderlyingValue(const UE::FVtValue& InValue);
	template USDUTILITIES_API TOptional<pxr::SdfListOp<pxr::TfToken>> GetUnderlyingValue(const UE::FVtValue& InValue);
#endif	  // USE_USD_SDK

	template<typename T>
	bool SetUnderlyingValue(UE::FVtValue& InValue, const T& UnderlyingValue)
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		pxr::VtValue& UsdValue = InValue.GetUsdValue();
		if (UsdValue.IsHolding<T>() || UsdValue.IsEmpty())
		{
			UsdValue = UnderlyingValue;
			return true;
		}
#endif	  // USE_USD_SDK

		return false;
	}
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const bool& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const uint8_t& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const int32_t& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const uint32_t& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const int64_t& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const uint64_t& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const float& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const double& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const std::string& UnderlyingValue);
#if USE_USD_SDK
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::GfHalf& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::SdfTimeCode& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::TfToken& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::SdfAssetPath& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::GfMatrix2d& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::GfMatrix3d& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::GfMatrix4d& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::GfQuatd& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::GfQuatf& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::GfQuath& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::GfVec2d& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::GfVec2f& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::GfVec2h& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::GfVec2i& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::GfVec3d& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::GfVec3f& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::GfVec3h& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::GfVec3i& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::GfVec4d& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::GfVec4f& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::GfVec4h& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::GfVec4i& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::VtArray<bool>& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::VtArray<uint8_t>& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::VtArray<int32_t>& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::VtArray<uint32_t>& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::VtArray<int64_t>& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::VtArray<uint64_t>& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::VtArray<pxr::GfHalf>& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::VtArray<float>& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::VtArray<double>& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::VtArray<pxr::SdfTimeCode>& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::VtArray<std::string>& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::VtArray<pxr::TfToken>& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::VtArray<pxr::SdfAssetPath>& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::VtArray<pxr::GfMatrix2d>& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::VtArray<pxr::GfMatrix3d>& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::VtArray<pxr::GfMatrix4d>& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::VtArray<pxr::GfQuatd>& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::VtArray<pxr::GfQuatf>& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::VtArray<pxr::GfQuath>& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::VtArray<pxr::GfVec2d>& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::VtArray<pxr::GfVec2f>& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::VtArray<pxr::GfVec2h>& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::VtArray<pxr::GfVec2i>& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::VtArray<pxr::GfVec3d>& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::VtArray<pxr::GfVec3f>& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::VtArray<pxr::GfVec3h>& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::VtArray<pxr::GfVec3i>& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::VtArray<pxr::GfVec4d>& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::VtArray<pxr::GfVec4f>& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::VtArray<pxr::GfVec4h>& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::VtArray<pxr::GfVec4i>& UnderlyingValue);
	template USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const pxr::SdfListOp<pxr::TfToken>& UnderlyingValue);
#endif	  // USE_USD_SDK

	FString GetImpliedTypeName(const UE::FVtValue& Value)
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		const pxr::VtValue& UsdValue = Value.GetUsdValue();
		pxr::SdfValueTypeName TypeName = pxr::SdfSchema::GetInstance().FindType(UsdValue.GetType());

		return UsdToUnreal::ConvertToken(TypeName.GetAsToken());
#else
		return FString();
#endif	  // USE_USD_SDK
	}
}	 // namespace UsdUtils
