// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"

#include "UsdWrappers/SdfPath.h"

#define UE_API USDUTILITIES_API

struct FUsdSchemaTranslationContext;
namespace UE
{
	class FSdfPath;
	class FUsdPrim;
}

enum class ECollapsingType
{
	Assets,
	Components
};

namespace UsdUtils
{
	struct FUsdPrimMaterialSlot;
}

struct FUsdInfoCacheImpl;

/**
 * Caches information about a specific USD Stage
 */
class FUsdInfoCache
{
public:
	UE_API FUsdInfoCache();
	UE_API virtual ~FUsdInfoCache();

	UE_API void CopyImpl(const FUsdInfoCache& Other);

	UE_API bool Serialize(FArchive& Ar);

	// Returns whether we contain any info about prim at 'Path' at all
	UE_API bool ContainsInfoAboutPrim(const UE::FSdfPath& Path) const;

	// Retrieves the children of a prim from the cached information
	UE_API TArray<UE::FSdfPath> GetChildren(const UE::FSdfPath& ParentPath) const;

	UE_API void RebuildCacheForSubtree(const UE::FUsdPrim& Prim, FUsdSchemaTranslationContext& Context);
	UE_API void RebuildCacheForSubtrees(const TArray<UE::FSdfPath>& SubtreeRoots, FUsdSchemaTranslationContext& Context);

	UE_API void Clear();
	UE_API bool IsEmpty();

public:
	UE_API bool IsPathCollapsed(const UE::FSdfPath& Path, ECollapsingType CollapsingType) const;
	UE_API bool DoesPathCollapseChildren(const UE::FSdfPath& Path, ECollapsingType CollapsingType) const;

	// Returns Path in case it represents an uncollapsed prim, or returns the path to the prim that collapsed it
	UE_API UE::FSdfPath UnwindToNonCollapsedPath(const UE::FSdfPath& Path, ECollapsingType CollapsingType) const;

public:
	// Returns the paths to prims that, when translated into assets or components, also require reading the prim at
	// 'Path'. e.g. providing the path to a Shader prim will return the paths to all Material prims for which the
	// translation involves reading that particular Shader.
	UE_API TSet<UE::FSdfPath> GetMainPrims(const UE::FSdfPath& AuxPrimPath) const;

	// The inverse of the function above: Provide it with the path to a Material prim and it will return the set of
	// paths to all Shader prims that need to be read to translate that Material prim into material assets
	UE_API TSet<UE::FSdfPath> GetAuxiliaryPrims(const UE::FSdfPath& MainPrimPath) const;

public:
	UE_API TSet<UE::FSdfPath> GetMaterialUsers(const UE::FSdfPath& Path) const;
	UE_API bool IsMaterialUsed(const UE::FSdfPath& Path) const;

public:
	// Provides the total vertex or material slots counts for each prim *and* its subtree.
	// This is built inside RebuildCacheForSubtree, so it will factor in the used Context's bMergeIdenticalMaterialSlots.
	// Note that these aren't affected by actual collapsing: A prim that doesn't collapse its children will still
	// provide the total sum of vertex counts of its entire subtree when queried
	UE_API TOptional<uint64> GetSubtreeVertexCount(const UE::FSdfPath& Path);
	UE_API TOptional<uint64> GetSubtreeMaterialSlotCount(const UE::FSdfPath& Path);
	UE_API TOptional<TArray<UsdUtils::FUsdPrimMaterialSlot>> GetSubtreeMaterialSlots(const UE::FSdfPath& Path);

public:
	// Marks/checks if the provided path to a prototype prim is already being translated.
	// This is used during scene translation with instanceables, so that the schema translators can early out
	// in case they have been created to translate multiple instances of the same prototype
	UE_API void ResetTranslatedPrototypes();
	UE_API bool IsPrototypeTranslated(const UE::FSdfPath& PrototypePath);
	UE_API void MarkPrototypeAsTranslated(const UE::FSdfPath& PrototypePath);

private:
	friend class FUsdGeomXformableTranslator;
	friend class FUsdGeometryCacheTranslator;

	// Returns true if every prim on the subtree below RootPath (including the RootPath prim itself) returns true for
	// CanBeCollapsed(), according to their own schema translators.
	//
	// WARNING: This is intended for internal use, and exclusively during the actual info cache build process as it will
	// need to query the prim/stage directly. Calling it after the info cache build may yield back an empty optional,
	// meaning it is unknown at this point whether the prim CanBeCollapsed or not.
	//
	// In general, you shouldn't call this, but just use "IsPathCollapsed" or "DoesPathCollapseChildren" instead.
	UE_API TOptional<bool> CanXformableSubtreeBeCollapsed(const UE::FSdfPath& RootPath, FUsdSchemaTranslationContext& Context) const;

	// Analogous to the function above, this overload of IsPotentialGeometryCacheRoot is meant for internal use, and exists because
	// during the info cache build (in some contexts) we can fill in this geometry cache information on-demand, for better performance.
	UE_API bool IsPotentialGeometryCacheRoot(const UE::FUsdPrim& Prim) const;

private:
	TUniquePtr<FUsdInfoCacheImpl> Impl;
};

#undef UE_API
