// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"

#include "UsdWrappers/SdfPath.h"

#define UE_API USDUTILITIES_API

namespace UE
{
	class FSdfPath;
}

class FUsdPrimLinkCache
{
public:
	struct FUsdPrimLinkCacheImpl;

	UE_API FUsdPrimLinkCache();
	UE_API ~FUsdPrimLinkCache();

	// Begin UObject interface
	UE_API void Serialize(FArchive& Ar);
	// End UObject interface

	// Returns whether we contain any info about prim at 'Path' at all
	UE_API bool ContainsInfoAboutPrim(const UE::FSdfPath& Path) const;

	UE_API void Clear();
	UE_API bool IsEmpty();

public:
	UE_API void LinkAssetToPrim(const UE::FSdfPath& Path, UObject* Asset);
	UE_API void UnlinkAssetFromPrim(const UE::FSdfPath& Path, UObject* Asset);

	UE_API TArray<TWeakObjectPtr<UObject>> RemoveAllAssetPrimLinks(const UE::FSdfPath& Path);
	UE_API TArray<UE::FSdfPath> RemoveAllAssetPrimLinks(const UObject* Asset);
	UE_API void RemoveAllAssetPrimLinks();

	UE_API TArray<TWeakObjectPtr<UObject>> GetAllAssetsForPrim(const UE::FSdfPath& Path) const;

	template<typename T = UObject>
	T* GetSingleAssetForPrim(const UE::FSdfPath& Path) const
	{
		TArray<TWeakObjectPtr<UObject>> Assets = GetAllAssetsForPrim(Path);

		// Search back to front so that if we generate a new version of an asset type we prefer
		// returning that
		for (int32 Index = Assets.Num() - 1; Index >= 0; --Index)
		{
			if (T* CastAsset = Cast<T>(Assets[Index].Get()))
			{
				return CastAsset;
			}
		}

		return nullptr;
	}

	template<typename T>
	TArray<T*> GetAssetsForPrim(const UE::FSdfPath& Path) const
	{
		TArray<TWeakObjectPtr<UObject>> Assets = GetAllAssetsForPrim(Path);

		TArray<T*> CastAssets;
		CastAssets.Reserve(Assets.Num());

		for (const TWeakObjectPtr<UObject>& Asset : Assets)
		{
			if (T* CastAsset = Cast<T>(Asset.Get()))
			{
				CastAssets.Add(CastAsset);
			}
		}

		return CastAssets;
	}

	UE_API TArray<UE::FSdfPath> GetPrimsForAsset(const UObject* Asset) const;
	UE_API TMap<UE::FSdfPath, TArray<TWeakObjectPtr<UObject>>> GetAllAssetPrimLinks() const;

private:
	TUniquePtr<FUsdPrimLinkCacheImpl> Impl;
};

#undef UE_API
