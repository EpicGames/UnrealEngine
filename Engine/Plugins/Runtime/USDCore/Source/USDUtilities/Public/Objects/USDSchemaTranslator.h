// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Objects/USDInfoCache.h"
#include "UnrealUSDWrapper.h"
#include "USDAssetCache2.h"
#include "USDAssetCache3.h"
#include "USDMetadataImportOptions.h"
#include "USDSkeletalDataConversion.h"

#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdStage.h"
#include "UsdWrappers/UsdTyped.h"

#include "Async/Future.h"
#include "GroomAssetInterpolation.h"
#include "HAL/ThreadSafeBool.h"
#include "Misc/Optional.h"
#include "Templates/SubclassOf.h"
#include "UObject/ObjectMacros.h"
#include "UObject/StrongObjectPtr.h"

class FRegisteredSchemaTranslator;
class FUsdPrimLinkCache;
class FUsdSchemaTranslator;
class FUsdSchemaTranslatorTaskChain;
class ULevel;
class UMaterialInterface;
class USceneComponent;
class UStaticMesh;
class UTexture;
struct FUsdBlendShape;
struct FUsdSchemaTranslationContext;
namespace UE
{
	class FUsdGeomBBoxCache;
}

class FRegisteredSchemaTranslatorHandle
{
public:
	FRegisteredSchemaTranslatorHandle()
		: Id(CurrentSchemaTranslatorId++)
	{
	}

	explicit FRegisteredSchemaTranslatorHandle(const FString& InSchemaName)
		: FRegisteredSchemaTranslatorHandle()
	{
		SchemaName = InSchemaName;
	}

	int32 GetId() const
	{
		return Id;
	}
	void SetId(int32 InId)
	{
		Id = InId;
	}

	const FString& GetSchemaName() const
	{
		return SchemaName;
	}
	void SetSchemaName(const FString& InSchemaName)
	{
		SchemaName = InSchemaName;
	}

private:
	static USDUTILITIES_API int32 CurrentSchemaTranslatorId;

	FString SchemaName;
	int32 Id;
};

class FUsdSchemaTranslatorRegistry
{
	using FCreateTranslator = TFunction<TSharedRef<FUsdSchemaTranslator>(TSharedRef<FUsdSchemaTranslationContext>, const UE::FUsdTyped&)>;
	using FSchemaTranslatorsStack = TArray<FRegisteredSchemaTranslator, TInlineAllocator<1>>;

public:
	static USDUTILITIES_API FUsdSchemaTranslatorRegistry& Get();

	/**
	 * Returns the translator to use for InSchema
	 */
	USDUTILITIES_API TSharedPtr<FUsdSchemaTranslator> CreateTranslatorForSchema(
		TSharedRef<FUsdSchemaTranslationContext> InTranslationContext,
		const UE::FUsdTyped& InSchema
	);

	/**
	 * Registers SchemaTranslatorType to translate schemas of type SchemaName.
	 * Registration order is important as the last to register for a given schema will be the one handling it.
	 * Thus, you will want to register base schemas before the more specialized ones.
	 */
	template<typename SchemaTranslatorType>
	FRegisteredSchemaTranslatorHandle Register(const FString& SchemaName)
	{
		auto CreateSchemaTranslator = [](TSharedRef<FUsdSchemaTranslationContext> InContext,
										 const UE::FUsdTyped& InSchema) -> TSharedRef<FUsdSchemaTranslator>
		{
			return MakeShared<SchemaTranslatorType>(InContext, InSchema);
		};

		return Register(SchemaName, CreateSchemaTranslator);
	}

	USDUTILITIES_API void Unregister(const FRegisteredSchemaTranslatorHandle& TranslatorHandle);

	USDUTILITIES_API int32 GetExternalSchemaTranslatorCount();

protected:
	USDUTILITIES_API FRegisteredSchemaTranslatorHandle Register(const FString& SchemaName, FCreateTranslator CreateFunction);

	USDUTILITIES_API FSchemaTranslatorsStack* FindSchemaTranslatorStack(const FString& SchemaName);

	TArray<TPair<FString, FSchemaTranslatorsStack>> RegisteredSchemaTranslators;

private:
	// Small machinery that lets us collect basic analytics about how many custom schema translators are being used in this session
	friend class FUsdSchemasModule;
	USDUTILITIES_API void ResetExternalTranslatorCount();
	int32 ExternalSchemaTranslatorCount = 0;
};

class FRegisteredSchemaTranslator
{
	using FCreateTranslator = TFunction<TSharedRef<FUsdSchemaTranslator>(TSharedRef<FUsdSchemaTranslationContext>, const UE::FUsdTyped&)>;

public:
	FRegisteredSchemaTranslatorHandle Handle;
	FCreateTranslator CreateFunction;
};

struct FUsdSchemaTranslationContext : public TSharedFromThis<FUsdSchemaTranslationContext>
{
	USDUTILITIES_API explicit FUsdSchemaTranslationContext(const UE::FUsdStage& InStage);
	USDUTILITIES_API explicit FUsdSchemaTranslationContext(const UE::FUsdStage& InStage, UUsdAssetCache3& InAssetCache);

	/** True if we're a context created by the USDStageImporter to fully import to persistent assets and actors */
	bool bIsImporting = false;

	/** True if we're just re-adding animations onto the LevelSequence, and not creating/updating components */
	bool bIsJustRepopulatingLevelSequence = false;

	/**
	 * True if we're building the InfoCache assigned to this context. This usually means we shouldn't query it for information, and should instead
	 * compute it manually so that it can be cached
	 */
	bool bIsBuildingInfoCache = false;

	/** pxr::UsdStage we're translating from */
	UE::FUsdStage Stage;

	/** Level to spawn actors in */
	ULevel* Level = nullptr;

	/** Flags used when creating UObjects */
	EObjectFlags ObjectFlags;

	/** The parent component when translating children */
	USceneComponent* ParentComponent = nullptr;

	/** The time at which we are translating */
	float Time = 0.f;

	/** We're only allowed to load prims with purposes that match these flags */
	EUsdPurpose PurposesToLoad;

	/** The render context to use when translating materials */
	FName RenderContext;

	/** The material purpose to use when translating material bindings */
	FName MaterialPurpose;

	/** Describes what to add to the root bone animation within generated AnimSequences, if anything */
	EUsdRootMotionHandling RootMotionHandling = EUsdRootMotionHandling::NoAdditionalRootMotion;

	/** What type of collision to use for static meshes generated from Prims that don't have physics schemas applied */
	EUsdCollisionType FallbackCollisionType = EUsdCollisionType::ConvexHull;

	/** How geometry caches are handled in the stage workflow */
	EGeometryCacheImport GeometryCacheImport = EGeometryCacheImport::Never;

	/** Subdivision level to use for all subdivision meshes on the opened stage. 0 means "don't subdivide" */
	int32 SubdivisionLevel = 0;

	FUsdMetadataImportOptions MetadataOptions;

	/** If a generated UStaticMesh has at least this many triangles we will attempt to enable Nanite */
	int32 NaniteTriangleThreshold;

	/** Where the translated assets will be stored */
	TStrongObjectPtr<UUsdAssetCache3> UsdAssetCache;

	/** Caches various information about prims that are expensive to query */
	FUsdInfoCache* UsdInfoCache;

	/** Caches which assets were generated from which USD prims */
	FUsdPrimLinkCache* PrimLinkCache;

	/** Bounding box cache used for the USD stage in case we have to spawn bounds components */
	TSharedPtr<UE::FUsdGeomBBoxCache> BBoxCache;

	/** Where we place imported blend shapes, if available */
	UsdUtils::FBlendShapeMap* BlendShapesByPath = nullptr;

	/**
	 * Sometimes we must upgrade a material from non-VT to VT, and so upgrade all of its textures to VT (and then
	 * upgrade all materials that use them to VT, etc.).
	 * This member lets us cache which generated materials use which generated textures in order to help with that.
	 * Material parsing is synchronous. If we ever upgrade it to paralllel/async-task-based, we'll need a mutex around
	 * this member.
	 */
	TMap<UTexture*, TSet<UMaterialInterface*>> TextureToUserMaterials;

	/**
	 * Whether to try to combine individual assets and components of the same type on a kind-per-kind basis,
	 * like multiple Mesh prims into a single Static Mesh
	 */
	EUsdDefaultKind KindsToCollapse = EUsdDefaultKind::Component | EUsdDefaultKind::Subcomponent;

	/**
	 * Use KindsToCollapse to determine when to collapse prim subtrees or not (defaults to enabled).
	 * Disable this if you want to prevent collapsing, or to control it manually by right-clicking on individual prims.
	 */
	bool bUsePrimKindsForCollapsing = true;

	/**
	 * Identical material slots will be combined into a single slot if this is enabled. This is only performed in the context
	 * of mesh collapsing, or when parsing LOD variant sets (see bInterpretLODs)
	 */
	bool bMergeIdenticalMaterialSlots = true;

	/**
	 * If true, whenever two prims would have generated identical UAssets (like identical StaticMeshes or materials) then only one instance of
	 * that asset is generated, and the asset is shared by the components generated for both prims.
	 * If false, we will always generate a dedicated asset for each prim.
	 */
	bool bShareAssetsForIdenticalPrims = true;

	/**
	 * If true, prims with a "LOD" variant set, and "LOD0", "LOD1", etc. variants containing each
	 * a prim can be parsed into a single UStaticMesh asset with multiple LODs
	 */
	bool bAllowInterpretingLODs = true;

	/** If true, we will also try creating UAnimSequence skeletal animation assets when parsing Skeleton prims */
	bool bAllowParsingSkeletalAnimations = true;

	/** If true, means we will try generating GroomAssets, GroomCaches and GroomBindings */
	bool bAllowParsingGroomAssets = true;

	/** If true, means we will try generating Sparse Volume Textures */
	bool bAllowParsingSparseVolumeTextures = true;

	/** If true, means we will try generating SoundWave assets from sound files referenced by UsdMediaSpatialAudio prims */
	bool bAllowParsingSounds = true;

	/** Skip the import of materials that aren't being used by any prim on the stage */
	bool bTranslateOnlyUsedMaterials = false;

	/**
	 * We set material overrides within UsdGeomXformableTranslator::UpdateComponents when this flag is set. Since that is a non-trivial
	 * amount of computation, this flag can be disabled for situations where material overrides shouldn't change (e.g. animating components)
	 */
	bool bAllowRecomputingMaterialOverrides = true;

	/** Groom group interpolation settings */
	TArray<FHairGroupsInterpolation> GroomInterpolationSettings;

	/**
	 * True if the Sequencer is currently opened and animating the stage level sequence.
	 * Its relevant to know this because some translator ::UpdateComponents overloads may try to animate their components
	 * by themselves, which could be wasteful and glitchy in case the sequencer is opened: It will likely also have an
	 * animation track for that component and on next editor tick would override the animation with what is sampled from
	 * the track.
	 * In the future we'll likely get rid of the "Time" track on the generated LevelSequence, at which point we can
	 * remove this
	 */
	bool bSequencerIsAnimating = false;

	bool IsValid() const
	{
		return Level != nullptr;
	}

	USDUTILITIES_API void CompleteTasks();

	TArray<TSharedPtr<FUsdSchemaTranslatorTaskChain>> TranslatorTasks;
};

enum class ESchemaTranslationStatus
{
	Pending,
	InProgress,
	Done
};

enum class ESchemaTranslationLaunchPolicy
{
	/**
	 * Task will run on main thread, with the guarantee that no other tasks are being run concurrently to it.
	 * Note: This is slow, and should not be used for realtime workflows (i.e. USDStage editor)
	 */
	ExclusiveSync,

	/** Task will run on main thread, while other tasks may be running concurrently */
	Sync,

	/** Task may run on another thread, while other tasks may be running concurrently */
	Async
};

class FUsdSchemaTranslator
{
public:
	explicit FUsdSchemaTranslator(TSharedRef<FUsdSchemaTranslationContext> InContext, const UE::FUsdTyped& InSchema)
		: PrimPath(InSchema.GetPrim().GetPrimPath())
		, Context(InContext)
	{
	}

	virtual ~FUsdSchemaTranslator() = default;

	virtual void CreateAssets()
	{
	}

	virtual USceneComponent* CreateComponents()
	{
		return nullptr;
	}
	virtual void UpdateComponents(USceneComponent* SceneComponent)
	{
	}

	virtual bool CollapsesChildren(ECollapsingType CollapsingType) const
	{
		return false;
	}

	/**
	 * Returns the set of prims that also need to be read in order to translate the prim at PrimPath.
	 * Note: This function never needs to return PrimPath itself, as the query function in the InfoCache will always
	 * append it to the result
	 */
	virtual TSet<UE::FSdfPath> CollectAuxiliaryPrims() const
	{
		return {};
	}

	USDUTILITIES_API bool IsCollapsed(ECollapsingType CollapsingType) const;
	virtual bool CanBeCollapsed(ECollapsingType CollapsingType) const
	{
		return false;
	}

	/**
	 * This checks if the current prim is an instance, and if so, whether its
	 * prototype is already being translated. Returns false otherwise.
	 *
	 * WARNING: In case this prim is an instance but the prototype is not being translated yet,
	 * running this check will also mark that prototype as being currently translated on the info cache!
	 *
	 * The intent here is that the first schema translator that calls this for a prototype
	 * will "own" the translation for that prototype, and any subsequent calls by other schema translators
	 * with the same prototype will just return true so they can early out.
	 */
	USDUTILITIES_API bool ShouldSkipInstance() const;

	/**
	 * If this prim is a prototype or an instance proxy, returns the prototype path (or the path to the
	 * analogue prim in the prototype's hierarchy).
	 * If this prim is just a regular non-instance prim, this just returns our PrimPath member.
	 */
	USDUTILITIES_API UE::FSdfPath GetPrototypePrimPath() const;

	UE::FUsdPrim GetPrim() const
	{
		return Context->Stage.GetPrimAtPath(PrimPath);
	}

protected:
	UE::FSdfPath PrimPath;
	TSharedRef<FUsdSchemaTranslationContext> Context;
};

struct FSchemaTranslatorTask
{
	explicit FSchemaTranslatorTask(ESchemaTranslationLaunchPolicy InPolicy, TFunction<bool()> InCallable)
		: Callable(InCallable)
		, LaunchPolicy(InPolicy)
		, bIsDone(false)
	{
	}
	FSchemaTranslatorTask(const FSchemaTranslatorTask& Other) = delete;

	TFunction<bool()> Callable;
	TOptional<TFuture<bool>> Result;
	TSharedPtr<FSchemaTranslatorTask> Continuation;

	ESchemaTranslationLaunchPolicy LaunchPolicy;

	FThreadSafeBool bIsDone;

	USDUTILITIES_API void Start();
	USDUTILITIES_API void StartIfAsync();

	bool IsStarted() const
	{
		return Result.IsSet() && Result->IsValid();
	}

	USDUTILITIES_API bool DoWork();

	bool IsDone() const
	{
		return bIsDone;
	}
};

class FUsdSchemaTranslatorTaskChain
{
public:
	virtual ~FUsdSchemaTranslatorTaskChain() = default;

	USDUTILITIES_API FUsdSchemaTranslatorTaskChain& Do(ESchemaTranslationLaunchPolicy Policy, TFunction<bool()> Callable);
	USDUTILITIES_API FUsdSchemaTranslatorTaskChain& Then(ESchemaTranslationLaunchPolicy Policy, TFunction<bool()> Callable);

	USDUTILITIES_API ESchemaTranslationStatus Execute(bool bExclusiveSyncTasks = false);

private:
	TSharedPtr<FSchemaTranslatorTask> CurrentTask;
};
