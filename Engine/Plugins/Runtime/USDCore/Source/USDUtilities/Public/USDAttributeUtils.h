// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UsdWrappers/ForwardDeclarations.h"

#if USE_USD_SDK
PXR_NAMESPACE_OPEN_SCOPE
	class UsdAttribute;
	class UsdPrim;
	class UsdProperty;
PXR_NAMESPACE_CLOSE_SCOPE
#endif	  // USE_USD_SDK

namespace UE
{
	class FUsdPrim;
	class FUsdAttribute;
}

namespace UsdUtils
{
	/** Adds a "Muted" CustomData entry to the attribute at the stage's UE state sublayer, which will prevent it from being animated when loaded into
	 * UE */
	USDUTILITIES_API bool MuteAttribute(UE::FUsdAttribute& Attribute, const UE::FUsdStage& Stage);

	/** Removes the "Muted" CustomData entry from the attribute at the stage's UE state sublayer, letting it be animated when loaded into UE */
	USDUTILITIES_API bool UnmuteAttribute(UE::FUsdAttribute& Attribute, const UE::FUsdStage& Stage);

	/** Returns whether this attribute has the "Muted" CustomData on the stage's UE state sublayer, meaning it shouldn't be animated when loaded into
	 * UE */
	USDUTILITIES_API bool IsAttributeMuted(const UE::FUsdAttribute& Attribute, const UE::FUsdStage& Stage);

	/** Returns true only if the attribute has an authored default opinion. If the attribute only has timeSamples, splines, or no opinion at all, this will return false */
	USDUTILITIES_API bool HasAuthoredDefaultOpinion(const UE::FUsdAttribute& Attribute);

	/** Returns true only if the attribute has an authored animated (timeSample or splines) opinion. If the attribute only has a default opinion (or no opinion at all) this will return false */
	USDUTILITIES_API bool HasAuthoredAnimatedOpinion(const UE::FUsdAttribute& Attribute);

#if USE_USD_SDK
	/**
	 * Calls pxr::UsdAttribute::GetTimeSamples(), and then pxr::UsdAttribute::ClearAtTime() for each timeSample value, clearing
	 * all timeSample opinions at the current edit target.
	 */
	USDUTILITIES_API bool ClearAllTimeSamples(const pxr::UsdAttribute& Attribute);

	/**
	 * If Attribute has opinions authored on layers stronger than the current edit target this will emit a warning to
	 * the user, indicating that the new value may not be visible on the USD Stage, and may cease to be visible on
	 * the UE level after a reload.
	 *
	 * Use this after setting any attribute: This function does nothing in case the Stage's current edit target has
	 * the strongest opinion for the attribute already
	 */
	USDUTILITIES_API void NotifyIfOverriddenOpinion(const pxr::UsdProperty& Property);

	/**
	 * Generally we are not allowed to author opinions inside instanceables. Our actual results when doing that will range from
	 * USD coding errors to seeming like nothing has happened on the stage.
	 * This will show a warning and return true if Prim is an instance, letting us avoid those scenarios
	 */
	USDUTILITIES_API bool NotifyIfInstanceProxy(const pxr::UsdPrim& Prim);

	/** Returns the attribute that has the strongest spec, acrosss the provided Attributes array */
	USDUTILITIES_API UE::FUsdAttribute GetStrongestAttribute(const TArray<UE::FUsdAttribute>& Attributes, double TimeCode);
#endif	  // USE_USD_SDK
}
