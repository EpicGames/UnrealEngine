// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MeshDescription.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "UnrealUSDWrapper.h"
#include "USDProjectSettings.h"
#include "USDSetUtils.h"
#include "UsdWrappers/ForwardDeclarations.h"
#include "UsdWrappers/SdfLayer.h"
#include "UsdWrappers/SdfPath.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "pxr/usd/usdShade/tokens.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class GfMatrix4d;
	class GfVec2f;
	class GfVec3f;
	class GfVec4f;
	class SdfPath;
	class TfToken;
	class TfType;
	class UsdAttribute;
	class UsdGeomMesh;
	class UsdGeomPrimvar;
	class UsdPrim;
	class UsdTimeCode;
	class VtValue;

	class UsdStage;
	template<typename T>
	class TfRefPtr;
	using UsdStageRefPtr = TfRefPtr<UsdStage>;

	class SdfPrimSpec;
	template<class T>
	class SdfHandle;
	using SdfPrimSpecHandle = SdfHandle<SdfPrimSpec>;
PXR_NAMESPACE_CLOSE_SCOPE

#endif	  // #if USE_USD_SDK

class UAssetImportData;
class UMaterial;
class USceneComponent;
class UUsdAssetImportData;
class UUsdAssetUserData;
enum class EUsdDrawMode : int32;
enum class EUsdDuplicateType : uint8;
enum class EUsdUpAxis : uint8;
struct FAnalyticsEventAttribute;
struct FUsdUnrealAssetInfo;
template<typename UsdObjectType>
class TUsdStore;

namespace UE
{
	class FUsdPrim;
}

namespace UsdUtils
{
	struct FUsdPrimMaterialAssignmentInfo;
}

namespace UsdUtils
{
	template<typename T>
	T* FindOrCreateObject(UObject* InParent, const FString& InName, EObjectFlags Flags)
	{
		T* Object = FindObject<T>(InParent, *InName);

		if (!Object)
		{
			Object = NewObject<T>(InParent, FName(*InName), Flags);
		}

		return Object;
	}

	/**
	 * Adds or removes a case sensitive hash suffix to the PrimPath string.
	 *
	 * We mainly use this to make sure that prim paths that differ only by case don't cause collisions inside
	 * case insensitive TMaps and TSets. We can sometimes use the case sensitive versions of those containers
	 * (see USDSetUtils.h), but sometimes that is not possible, due to backwards compatibility or due to the
	 * container being a UPROPERTY
	 */
	USDUTILITIES_API FString AddHashToPrimPath(const FString& PrimPath);
	USDUTILITIES_API FString RemoveHashFromPrimPath(const FString& HashedPrimPath);

#if USE_USD_SDK
	template<typename ValueType>
	USDUTILITIES_API ValueType GetUsdValue(const pxr::UsdAttribute& Attribute, pxr::UsdTimeCode TimeCode);

	USDUTILITIES_API pxr::TfToken GetUsdStageUpAxis(const pxr::UsdStageRefPtr& Stage);
	USDUTILITIES_API EUsdUpAxis GetUsdStageUpAxisAsEnum(const pxr::UsdStageRefPtr& Stage);

	USDUTILITIES_API void SetUsdStageUpAxis(const pxr::UsdStageRefPtr& Stage, pxr::TfToken Axis);
	USDUTILITIES_API void SetUsdStageUpAxis(const pxr::UsdStageRefPtr& Stage, EUsdUpAxis Axis);

	USDUTILITIES_API double GetUsdStageMetersPerUnit(const pxr::UsdStageRefPtr& Stage);
	USDUTILITIES_API void SetUsdStageMetersPerUnit(const pxr::UsdStageRefPtr& Stage, double MetersPerUnit);

	USDUTILITIES_API int32 GetUsdStageNumFrames(const pxr::UsdStageRefPtr& Stage);

	/**
	 * Simple API to interact with the UsdUtils' singleton StageCache from modules not setup to interact with the USD SDK directly
	 * like Interchange's Import module.
	 *
	 * Note: We do this instead of having a UsdStageCache wrapper type because when dealing with stage caches we are realistically going
	 * to be using the UsdUtils' singleton instance. The wrapper types all *own* the USD version of their objects, and so having the
	 * FUsdStageCache wrapper *refer* to an external, existing instance of the USD version of itself instead (the singleton instance)
	 * would likely be very misleading and confusing.
	 */
	USDUTILITIES_API int64 GetUsdUtilsStageCacheStageId(const pxr::UsdStageRefPtr& Stage);
	USDUTILITIES_API UE::FUsdStage FindUsdUtilsStageCacheStageId(int64 Id);
	USDUTILITIES_API int64 InsertStageIntoUsdUtilsStageCache(const pxr::UsdStageRefPtr& Stage);
	USDUTILITIES_API bool RemoveStageFromUsdUtilsStageCache(int64 StageId);

	USDUTILITIES_API bool HasCompositionArcs(const pxr::UsdPrim& Prim);
	USDUTILITIES_API bool HasCompositionArcs(const pxr::SdfPrimSpecHandle& PrimSpec);

	USDUTILITIES_API UClass* GetActorTypeForPrim(const pxr::UsdPrim& Prim);
	USDUTILITIES_API UClass* GetComponentTypeForPrim(const pxr::UsdPrim& Prim);

	/** Returns the USD schema name that should be used when exporting Component (e.g. "Xform", "Mesh", "Camera", etc.) */
	USDUTILITIES_API FString GetSchemaNameForComponent(const USceneComponent& Component);

	/**
	 * Returns a prim path to use when exporting a given ActorOrComponent, assuming we're exporting according to the component attachment hierarchy.
	 * e.g. We have a top-level actor with label "Actor" with an attached child actor "Child" that has a component "Comp1" attached to its root, and
	 * "Comp2" attached to "Comp1". If Comp2 is provided to this function, it will return "/<RootPrimName>/Actor/Child/Comp1/Comp2".
	 *
	 * The actor label is always used in place of the root component name.
	 *
	 * ParentPrimPath is optional, and without it the function will recurse upwards and build the full path all the way to the root. Actor folders
	 * are handled just like if they were actors/components
	 */
	USDUTILITIES_API FString GetPrimPathForObject(
		const UObject* ActorOrComponent,
		const FString& ParentPrimPath = TEXT(""),
		bool bUseActorFolders = false,
		const FString& RootPrimName = TEXT("Root")
	);

	USDUTILITIES_API TUsdStore<pxr::TfToken> GetUVSetName(int32 UVChannelIndex);

	/**
	 * Gets the primvars that should be used as UV sets, per index, for this UsdPrim.
	 * This will return between 0 and MaxNumPrimvars primvars, where the first item of array is the primvar that should be used for
	 * UV index 0 (if any), the second the primvar that should be used for UV set 1, etc.
	 *
	 * In case there are more than MaxNumPrimvars available primvars, the returned list will contain all the 'texCoord2f' role
	 * primvars (sorted lexicographically), followed by the regular float2 primvars (also separately sorted lexicographically).
	 *
	 * @param UsdPrim - UsdPrim that contains primvars that can be used as texture coordinates.
	 * @param MaxNumPrimvars - Maximum number of primvars to return from this function.
	 * @return Array of up to MaxNumPrimvars primvars sorted by priority (most important come first)
	 */
	USDUTILITIES_API TArray<TUsdStore<pxr::UsdGeomPrimvar>> GetUVSetPrimvars(
		const pxr::UsdPrim& UsdPrim,
		int32 MaxNumPrimvars = USD_PREVIEW_SURFACE_MAX_UV_SETS
	);

	/**
	 * Rearranges an array of primvars into another array of up to USD_PREVIEW_SURFACE_MAX_UV_SETS primvars, describing which
	 * primvars should be used for each UV set according to the provided AllowedPrimvarsToUVIndex mapping.
	 *
	 * The intent is to use this when e.g. a Mesh prim is to be collapsed together with other Mesh prims into a single
	 * StaticMesh, and they all need consistent UV sets across them. So even though AllMeshUVPrimvars contains [aaa,
	 * bbb, eee], AllowedPrimvarsToUVIndex may instruct us to assign these to UV indices [0, 1, 4] (and not just [0, 1,
	 * 2]) because our Mesh is to be merged with another Mesh prim that contains [aaa, bbb, ccc, ddd], so that ccc can
	 * be put on index 2 and ddd on index 3.
	 *
	 * @param AllMeshUVPrimvars - Array of primvars extracted from some mesh sorted by priority (most important come
	 * first)
	 * @param AllowedPrimvarsToUVIndex - Mapping from primvar names to the UV index that they're allowed to be assigned
	 * to
	 * @return Array where each index gives the primvar that should be used for that UV index
	 */
	USDUTILITIES_API TArray<TUsdStore<pxr::UsdGeomPrimvar>> AssemblePrimvarsIntoUVSets(
		const TArray<TUsdStore<pxr::UsdGeomPrimvar>>& AllMeshUVPrimvars,
		const TMap<FString, int32>& AllowedPrimvarsToUVIndex
	);

	/**
	 * Constructs a mapping from primvar name to their position in the provided AllMeshUVPrimvars array.
	 * @param AllMeshUVPrimvars - Array of primvars extracted from some mesh sorted by priority (most important come
	 * first)
	 * @return Mapping from primvar names to the UV index
	 */
	USDUTILITIES_API TMap<FString, int32> AssemblePrimvarsIntoPrimvarToUVIndexMap(const TArray<TUsdStore<pxr::UsdGeomPrimvar>>& AllMeshUVPrimvars);

	/**
	 * Given an array of primvars, and an array of which of those are preferred (AllPrimvars should also include
	 * PreferredPrimvars), this will sort lexicographically and return a prioritized PrimvarToUVIndex assignment map of
	 * up to USD_PREVIEW_SURFACE_MAX_UV_SETS UV indices
	 */
	USDUTILITIES_API TMap<FString, int32> CombinePrimvarsIntoUVSets(const TSet<FString>& AllPrimvars, const TSet<FString>& PreferredPrimvars);

	/**
	 * Calls GetUVSetPrimvars() followed by AssemblePrimvarsIntoPrimvarToUVIndexMap().
	 * This exists so that it can be called without having to manipulate the USD types (e.g. for Interchange)
	 */
	USDUTILITIES_API TMap<FString, int32> GetPrimvarToUVIndexMap(const pxr::UsdPrim& UsdPrim, int32 MaxNumPrimvars = USD_PREVIEW_SURFACE_MAX_UV_SETS);

	template<typename T, typename U>
	FString StringifyMap(const TMap<T, U>& Map)
	{
		FString Result = TEXT("{");
		for (const TPair<T, U>& Pair : Map)
		{
			Result += FString::Printf(TEXT("'%s': %s, "), *LexToString(Pair.Key), *LexToString(Pair.Value));
		}
		Result.RemoveFromEnd(TEXT(", "));
		Result += TEXT("}");
		return Result;
	}

	USDUTILITIES_API bool IsAnimated(const pxr::UsdPrim& Prim);
	USDUTILITIES_API bool HasAnimatedAttributes(const pxr::UsdPrim& Prim);
	USDUTILITIES_API bool HasAnimatedTransform(const pxr::UsdPrim& Prim);
	USDUTILITIES_API bool HasAnimatedVisibility(const pxr::UsdPrim& Prim);

	/**
	 * Returns true if Prim should have animated bounds (either from having authored animated `extent` or `extentsHint`
	 * attributes, or potentially from having computed bounds and child prims that should make those computed bounds animated,
	 * like having animated child Xforms)
	 */
	USDUTILITIES_API bool HasAnimatedBounds(
		const pxr::UsdPrim& Prim,
		EUsdPurpose IncludedPurposes = EUsdPurpose::Proxy | EUsdPurpose::Render,
		bool bUseExtentsHint = true,
		bool bIgnoreVisibility = false
	);

	/**
	 * Similar to HasAnimatedBounds, except that this will also collect all the timeSamples that are relevant for sampling
	 * animated bounds. For example this could just be the timeSamples for an authored `extentsHint` or `extent` animation, but
	 * this could instead be the timeSamples of child Xforms transform animations instead.
	 * Note that OutTimeSamples may end up with duplicate timeSamples, but those will always be at least sorted.
	 */
	USDUTILITIES_API bool GetAnimatedBoundsTimeSamples(
		const pxr::UsdPrim& InPrim,
		TArray<double>& OutTimeSamples,
		EUsdPurpose InIncludedPurposes = EUsdPurpose::Proxy | EUsdPurpose::Render,
		bool bInUseExtentsHint = true,
		bool bInIgnoreVisibility = false
	);

	/** Returns all of the Prim attributes that contain timeSamples */
	USDUTILITIES_API TArray<UE::FUsdAttribute> GetAnimatedAttributes(const pxr::UsdPrim& Prim);

	/** Returns a list of all the registered property names for a provided APISchema name. Also works for codeless schemas */
	USDUTILITIES_API TArray<FString> GetAppliedAPISchemaPropertyNames(const FString& SchemaName);

	/** Returns true if this prim has any authored opinion for prim kind */
	USDUTILITIES_API bool HasAuthoredKind(const pxr::UsdPrim& Prim);

	/** Returns whether Prim belongs to any of the default kinds, or a kind derived from them. The result can be a union of different kinds. */
	USDUTILITIES_API EUsdDefaultKind GetDefaultKind(const pxr::UsdPrim& Prim);

	/**
	 * Sets the prim kind for Prim to the provided NewKind.
	 * Differs from IUsdPrim::SetKind in that we don't have to provide the USD token.
	 * @param NewKind - New kind to set. Must be a single flag, and not a combination of multiple plags
	 * @param Prim - Prim to receive the new Kind
	 * @return Whether we managed to set the kind or not.
	 */
	USDUTILITIES_API bool SetDefaultKind(pxr::UsdPrim& Prim, EUsdDefaultKind NewKind);

	enum class ECollapsingPreference : uint8
	{
		Default,
		Allow,
		Never
	};

	/**
	 * Gets how this prim wants to be collapsed: Whether we should allow/never try to do it, or whether we should check its kind for that purpose
	 */
	USDUTILITIES_API ECollapsingPreference GetCollapsingPreference(const pxr::UsdPrim& Prim);

	/**
	 * Sets how this prim wants to be collapsed: Whether we should always/never try to do it, or whether we should check its kind for that purpose
	 */
	USDUTILITIES_API bool SetCollapsingPreference(const pxr::UsdPrim& Prim, ECollapsingPreference NewPreference);

	/** Checks the prim kind and the collapsing schema to determine whether the provided prim should collapse or not */
	USDUTILITIES_API bool PrimCollapses(
		const pxr::UsdPrim& Prim,
		EUsdDefaultKind KindsToCollapse,
		bool bUsePrimKindsForCollapsing,
		bool bUseSchemaForCollapsing,
		bool bCanCollapseByDefault
	);

	/**
	 * Returns whether the prim has the UsdGeomModelAPI schema and should be drawn with one of the alternative draw modes, such as cards or bounds.
	 * Will return EUsdDrawMode::Default in case the prim should be drawn as usual instead, or in case of error.
	 */
	USDUTILITIES_API EUsdDrawMode GetAppliedDrawMode(const pxr::UsdPrim& Prim);

	/** Copy-pasted from OpenVDBImportOptions.h because that is editor-only, and we want to avoid spreading more #if WITH_EDITORs around */
	enum class ESparseVolumeAttributesFormat : uint8
	{
		Unorm8 = 0,
		Float16 = 1,
		Float32 = 2,
	};

	struct FVolumePrimInfo
	{
		FString SourceVDBFilePath;

		TArray<double> TimeSamplePathTimeCodes;
		TArray<int32> TimeSamplePathIndices;
		TArray<FString> TimeSamplePaths;	// First item may equal SourceVDBFilePath. May have duplicate entries

		// Fields on the Volume prim that were connected to this .vdb file
		TArray<FString> VolumeFieldNames;
		TArray<FString> SourceOpenVDBAssetPrimPaths;

		// e.g. {'velocity': {'AttributesA.R': 'X', 'AttributesA.G': 'Y', 'AttributesA.B': 'Z'}}
		TMap<FString, TMap<FString, FString>> GridNameToChannelComponentMapping;
		TOptional<ESparseVolumeAttributesFormat> AttributesAFormat;
		TOptional<ESparseVolumeAttributesFormat> AttributesBFormat;
	};

	/**
	 * Collects an FVolumePrimInfo for each SparseVolumeTexture that should be generated for VolumePrim.
	 * The map keys are FilePathHashes, because some of these SparseVolumeTextures may consume multiple .vdb files (e.g .animations)
	 */
	USDUTILITIES_API TMap<FString, FVolumePrimInfo> GetVolumeInfoByFilePathHash(const pxr::UsdPrim& VolumePrim);

	/**
	 * Extracts from the volume prim a desired mapping between Unreal material parameter and the volume field name
	 * (note the corresponding VolumeFieldNames field within FVolumePrimInfo)
	 */
	USDUTILITIES_API TMultiMap<FString, FString> GetVolumeMaterialParameterToFieldNameMap(const pxr::UsdPrim& VolumePrim);

	/** Gets the names of all SparseVolumeTexture material parameters within the provided material */
	USDUTILITIES_API TArray<FString> GetSparseVolumeTextureParameterNames(const UMaterial* Material);

	/**
	 * Returns all prims of type SchemaType (or a descendant type) in the subtree of prims rooted at StartPrim.
	 * Stops going down the subtrees when it hits a schema type to exclude.
	 */
	USDUTILITIES_API TArray<TUsdStore<pxr::UsdPrim>> GetAllPrimsOfType(
		const pxr::UsdPrim& StartPrim,
		const pxr::TfType& SchemaType,
		const TArray<TUsdStore<pxr::TfType>>& ExcludeSchemaTypes = {}
	);
	USDUTILITIES_API TArray<TUsdStore<pxr::UsdPrim>> GetAllPrimsOfType(
		const pxr::UsdPrim& StartPrim,
		const pxr::TfType& SchemaType,
		TFunction<bool(const pxr::UsdPrim&)> PruneChildren,
		const TArray<TUsdStore<pxr::TfType>>& ExcludeSchemaTypes = {}
	);

	USDUTILITIES_API FString GetAssetPathFromPrimPath(const FString& RootContentPath, const pxr::UsdPrim& Prim);
#endif	  // #if USE_USD_SDK

	USDUTILITIES_API TArray<UE::FUsdPrim> GetAllPrimsOfType(const UE::FUsdPrim& StartPrim, const TCHAR* SchemaName);
	USDUTILITIES_API TArray<UE::FUsdPrim> GetAllPrimsOfType(
		const UE::FUsdPrim& StartPrim,
		const TCHAR* SchemaName,
		TFunction<bool(const UE::FUsdPrim&)> PruneChildren,
		const TArray<const TCHAR*>& ExcludeSchemaNames = {}
	);

	/** Returns the time code for non-timesampled values. Usually a quiet NaN. */
	USDUTILITIES_API double GetDefaultTimeCode();

	/** Returns the earliest possible timecode. Use it to always fetch the first frame of an animated attribute */
	USDUTILITIES_API double GetEarliestTimeCode();

#if USE_USD_SDK
	/**
	 * Simple utility that generates a prefix that should be added to the hash when using it to cache/query an UsdAssetCache2 object.
	 * The idea is to use this prefix alongside the hash whenever bShareAssetsForIdenticalPrims is false, so as to generate a new hash for each prim.
	 * Whenever bShareAssetsForIdenticalPrims is true, the prefix will be the empty string.
	 * Whenever bShareAssetsForIdenticalPrims is false, the prefix will be a SHA1 hash of the prim path and its stage identifier.
	 */
	USDUTILITIES_API FString GetAssetHashPrefix(const pxr::UsdPrim& PrimForAsset, bool bShareAssetsForIdenticalPrims);
#endif	  // #if USE_USD_SDK

	/**
	 * Adds a reference on Prim to the layer at AbsoluteFilePath, optionally specifying a target prim inside that layer.
	 * @param Prim - Referencer prim
	 * @param AbsoluteFilePath - Path to the layer to reference (can be empty for internal references). e.g. "C:\MyFolder\Cube.usda"
	 * @param TargetPrimPath - Path to a specific prim to reference within the target layer. e.g. "/SomeOtherPrim"
	 * @param TimeCodeOffset - Offset in timeCodes to author on the reference
	 * @param TimeCodeScale - TimeCode scale to author on the reference
	 * @param ReferencerSchemaHandling - How to behave when Prim has a different type than the referenced prim.
	 *                                   Passing an unset TOptional will check the project settings for the value instead.
	 *                                   The default value is MatchReferencedType for backwards compatibility.
	 *                                   Note: If this is set to ShowDialog, the call may create an Editor transaction.
	 */
	USDUTILITIES_API void AddReference(
		UE::FUsdPrim& Prim,
		const TCHAR* AbsoluteFilePath,
		const UE::FSdfPath& TargetPrimPath = {},
		double TimeCodeOffset = 0.0,
		double TimeCodeScale = 1.0
	);
	USDUTILITIES_API void AddReference(
		UE::FUsdPrim& Prim,
		const TCHAR* AbsoluteFilePath,
		TOptional<EReferencerTypeHandling> ReferencerTypeHandling,
		const UE::FSdfPath& TargetPrimPath = {},
		double TimeCodeOffset = 0.0,
		double TimeCodeScale = 1.0
	);

	/** Gets the strongest direct reference on Prim with the given FileExtension. Returns true if there was one. */
	USDUTILITIES_API bool GetReferenceFilePath(const UE::FUsdPrim& Prim, const FString& FileExtension, FString& OutReferenceFilePath);

	/** Returns the filepaths that this prim is referencing or using as a payload, in case it has any external reference or payload authored */
	USDUTILITIES_API TArray<FString> GetReferenceOrPayloadFilePaths(const UE::FUsdPrim& Prim);

	/**
	 * Adds a payload on Prim pointing at the layer at AbsoluteFilePath, optionally specifying a target prim inside that layer.
	 * @param Prim - Prim to receive the payload
	 * @param AbsoluteFilePath - Path to the layer to use as payload (can be empty for internal references). e.g. "C:\MyFolder\Cube.usda"
	 * @param TargetPrimPath - Path to a specific prim to use as payload within the target layer. e.g. "/SomeOtherPrim"
	 * @param TimeCodeOffset - Offset in timeCodes to author on the payload
	 * @param TimeCodeScale - TimeCode scale to author on the payload
	 * @param ReferencerSchemaHandling - How to behave when Prim has a different type than the prim used by the payload.
	 *                                   Passing an unset TOptional will check the project settings for the value instead.
	 *                                   The default value is MatchReferencedType for backwards compatibility.
	 *                                   Note: If this is set to ShowDialog, the call may create an Editor transaction.
	 */
	USDUTILITIES_API void AddPayload(
		UE::FUsdPrim& Prim,
		const TCHAR* AbsoluteFilePath,
		const UE::FSdfPath& TargetPrimPath = {},
		double TimeCodeOffset = 0.0,
		double TimeCodeScale = 1.0
	);
	USDUTILITIES_API void AddPayload(
		UE::FUsdPrim& Prim,
		const TCHAR* AbsoluteFilePath,
		TOptional<EReferencerTypeHandling> ReferencerTypeHandling,
		const UE::FSdfPath& TargetPrimPath = {},
		double TimeCodeOffset = 0.0,
		double TimeCodeScale = 1.0
	);

	/**
	 * Renames a single prim to a new name
	 * WARNING: This will lead to issues if called from within a pxr::SdfChangeBlock. This because it needs to be able
	 * to send separate notices: One notice about the renaming, that the transactor can record on the current edit target,
	 * and one extra notice about the definition of an auxiliary prim on the session layer, that the transactor *must* record
	 * as having taken place on the session layer.
	 * @param Prim - Prim to rename
	 * @param NewPrimName - New name for the prim e.g. "MyNewName"
	 * @return True if we managed to rename
	 */
	USDUTILITIES_API bool RenamePrim(UE::FUsdPrim& Prim, const TCHAR* NewPrimName);

#if USE_USD_SDK
	/**
	 * Returns a sanitized, valid version of 'InName' that can be used as the name for a child prim of 'ParentPrim'.
	 * Returns the empty string in case of an error.
	 */
	USDUTILITIES_API FString GetValidChildName(FString InName, const pxr::UsdPrim& ParentPrim);
#endif	  // #if USE_USD_SDK

	/**
	 * Returns a modified version of InIdentifier that can be used as a USD prim or property name.
	 * This means only allowing letters, numbers and the underscore character. All others are replaced with underscores.
	 * Additionally, the first character cannot be a number.
	 * Note that this obviously doesn't check for a potential name collision.
	 */
	USDUTILITIES_API FString SanitizeUsdIdentifier(const TCHAR* InIdentifier);

	/** Will call UsdGeomImageable::MakeVisible/MakeInvisible if Prim is a UsdGeomImageable */
	USDUTILITIES_API void MakeVisible(UE::FUsdPrim& Prim, double TimeCode = UsdUtils::GetDefaultTimeCode());
	USDUTILITIES_API void MakeInvisible(UE::FUsdPrim& Prim, double TimeCode = UsdUtils::GetDefaultTimeCode());

	/** Returns if the ComputedVisibility for Prim says it should be visible */
	USDUTILITIES_API bool IsVisible(const UE::FUsdPrim& Prim, double TimeCode = UsdUtils::GetDefaultTimeCode());

	/** Returns whether Prim has visibility set to 'inherited' */
	USDUTILITIES_API bool HasInheritedVisibility(const UE::FUsdPrim& Prim, double TimeCode = UsdUtils::GetDefaultTimeCode());

	/**
	 * Travels up from Prim and returns true if we hit any invisible prim before we hit the stage pseudoroot.
	 * Does not check `Prim` itself (or `RootPrim`, so that you can just pass the PseudoRoot in most cases).
	 * Completely ignores whether non-UsdGeomImageable prims are visible or not and keeps travelling up.
	 */
	USDUTILITIES_API bool HasInvisibleParent(
		const UE::FUsdPrim& Prim,
		const UE::FUsdPrim& RootPrim,
		double TimeCode = UsdUtils::GetDefaultTimeCode()
	);

	/**
	 * Returns the prims in the subtree of Prim (potentially including Prim itself) that are visible due to the
	 * visibility attribute and prim purpose
	 */
	USDUTILITIES_API TArray<UE::FUsdPrim> GetVisibleChildren(const UE::FUsdPrim& Prim, EUsdPurpose AllowedPurposes);

	/**
	 * Returns a path exactly like Prim.GetPrimPath(), except that if the prim is within variant sets, it will return the
	 * full path with variant selections in it (i.e. the spec path), like "/Root/Child{Varset=Var}Inner" instead of just
	 * "/Root/Child/Inner".
	 *
	 * It needs a layer because it is possible for a prim to be defined within a variant set in some layer, but then
	 * have an 'over' opinion defined in another layer without a variant, meaning the actual spec path depends on the layer.
	 *
	 * Note that stage operations that involve manipulating specs require this full path instead (like removing/renaming prims),
	 * while other operations need the path with the stripped variant selections (like getting/defining/overriding prims)
	 *
	 * Returns an empty path in case the layer doesn't have a spec for this prim.
	 */
	USDUTILITIES_API UE::FSdfPath GetPrimSpecPathForLayer(const UE::FUsdPrim& Prim, const UE::FSdfLayer& Layer);

	/**
	 * Removes all the prim specs for Prim on the given Layer, if the layer belongs to the stage's local layer stack.
	 *
	 * This function is useful in case the prim is inside a variant set: In that case, just calling FUsdStage::RemovePrim()
	 * will attempt to remove the "/Root/Example/Child", which wouldn't remove the "/Root{Varset=Var}Example/Child" spec,
	 * meaning the prim may still be left on the stage. Note that it's even possible to have both of those specs at the same time:
	 * for example when we have a prim inside a variant set, but outside of it we have overrides to the same prim. This function
	 * will remove both.
	 *
	 * @param Prim - Prim to remove
	 * @param Layer - Layer to remove prim specs from. This can be left with the invalid layer (default) in order to remove all
	 *				  specs from the entire stage's local layer stack.
	 */
	USDUTILITIES_API void RemoveAllLocalPrimSpecs(const UE::FUsdPrim& Prim, const UE::FSdfLayer& Layer = UE::FSdfLayer{});

	/**
	 * Copies flattened versions of the input prims onto the clipboard stage and removes all the prim specs for Prims from their stages.
	 * These cut prims can then be pasted with PastePrims.
	 *
	 * @param Prims - Prims to cut
	 * @return True if we managed to cut
	 */
	USDUTILITIES_API bool CutPrims(const TArray<UE::FUsdPrim>& Prims);

	/**
	 * Copies flattened versions of the input prims onto the clipboard stage.
	 * These copied prims can then be pasted with PastePrims.
	 *
	 * @param Prims - Prims to copy
	 * @return True if we managed to copy
	 */
	USDUTILITIES_API bool CopyPrims(const TArray<UE::FUsdPrim>& Prims);

	/**
	 * Pastes the prims from the clipboard stage as children of ParentPrim.
	 *
	 * The pasted prims may be renamed in order to have valid names for the target location, which is why this function
	 * returns the pasted prim paths.
	 * This function returns just paths instead of actual prims because USD needs to respond to the notices about
	 * the created prim specs before the prims are fully created, which means we wouldn't be able to return the
	 * created prims yet, in case this function was called from within an SdfChangeBlock.
	 *
	 * @param ParentPrim - Prim that will become parent to the pasted prims
	 * @return Paths to the pasted prim specs, after they were added as children of ParentPrim
	 */
	USDUTILITIES_API TArray<UE::FSdfPath> PastePrims(const UE::FUsdPrim& ParentPrim);

	/** Returns true if we have prims that we can paste within our clipboard stage */
	USDUTILITIES_API bool CanPastePrims();

	/** Clears all prims from our clipboard stage */
	USDUTILITIES_API void ClearPrimClipboard();

	/**
	 * Duplicates all provided Prims one-by-one, performing the requested DuplicateType.
	 * See the documentation on EUsdDuplicateType for the different operation types.
	 *
	 * The duplicated prims may be renamed in order to have valid names for the target location, which is why this
	 * function returns the pasted prim paths.
	 * This function returns just paths instead of actual prims because USD needs to respond to the notices about
	 * the created prim specs before the prims are fully created, which means we wouldn't be able to return the
	 * created prims yet, in case this function was called from within an SdfChangeBlock.
	 *
	 * @param Prims - Prims to duplicate
	 * @param DuplicateType - Type of prim duplication to perform
	 * @param TargetLayer - Target layer to use when duplicating, if relevant for that duplication type
	 * @return Paths to the duplicated prim specs, after they were added as children of ParentPrim.
	 */
	USDUTILITIES_API TArray<UE::FSdfPath> DuplicatePrims(
		const TArray<UE::FUsdPrim>& Prims,
		EUsdDuplicateType DuplicateType,
		const UE::FSdfLayer& TargetLayer = UE::FSdfLayer{}
	);

	/** Adds to Prim the assetInfo metadata the values described in Info */
	USDUTILITIES_API void SetPrimAssetInfo(UE::FUsdPrim& Prim, const FUsdUnrealAssetInfo& Info);

	/** Retrieves from Prim the assetInfo metadata values that we use as export metadata, when exporting Unreal assets */
	USDUTILITIES_API FUsdUnrealAssetInfo GetPrimAssetInfo(const UE::FUsdPrim& Prim);

#if USE_USD_SDK
	/* Removes all metadata from Prim, except the typeName and specifier entries */
	USDUTILITIES_API bool ClearNonEssentialPrimMetadata(const pxr::UsdPrim& Prim);
#endif	  // USE_USD_SDK

	/** Collects how many times each schema shows up on the provided stage and appends those attributes to the provided OutAttributes */
	USDUTILITIES_API void CollectSchemaAnalytics(const UE::FUsdStage& Stage, TArray<FAnalyticsEventAttribute>& OutAttributes);

	/** Parses and stores primary Stage metadata key-info pairs in the MetadataMap */
	USDUTILITIES_API void ReadStageMetaData(UE::FUsdStage Stage, TMap<FString, FString>& OutMetaDataMap);

	// Whether or not instancing-aware translation is enabled
	USDUTILITIES_API bool IsInstancingAwareTranslationEnabled();

	// Retrieves the equivalent of this prim in the prototype it's referenced from, where applicable
	USDUTILITIES_API UE::FUsdPrim GetPrototypePrim(const UE::FUsdPrim& Prim);

	// Retrieves the prototype path for the prim
	USDUTILITIES_API UE::FSdfPath GetPrototypePrimPath(const UE::FUsdPrim& Prim);

}	 // namespace UsdUtils
