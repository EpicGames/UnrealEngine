// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Logging/TokenizedMessage.h"
#include "Misc/SourceLocation.h"

#define UE_API USDUTILITIES_API

namespace UsdUtils
{
	/**
	 * Pushes an USD error monitoring object into the stack and catches any emitted errors
	 */
	UE_DEPRECATED(5.6, "Use FScopedUsdMessageLog instead.")
	USDUTILITIES_API void StartMonitoringErrors();

	/**
	* Returns all errors that were captured since StartMonitoringErrors(), clears and pops an
	* error monitoring object from the stack
	 */
	UE_DEPRECATED(5.6, "Use FScopedUsdMessageLog and FUsdLogManager::HasAccumulatedErrors() instead.")
	USDUTILITIES_API TArray<FString> GetErrorsAndStopMonitoring();

	/**
	 * Displays the error messages for each captured error since StartMonitoringErrors(),
	 * clears and pops an error monitoring object from the stack.
	 * If ToastMessage is empty, a default message will be displayed.
	 * Returns true if there were any errors.
	 */
	UE_DEPRECATED(5.6, "Use FScopedUsdMessageLog instead.")
	USDUTILITIES_API bool ShowErrorsAndStopMonitoring(const FText& ToastMessage = FText());
}

class FUsdLogManager
{
public:
	using FMessageDisplayCallback = TFunction<void(const TArray<TSharedRef<FTokenizedMessage>>&)>;

	// Copied from AssertionMacros.h as it's private there
	UE_NODEBUG static constexpr uint32 FileHashForEnsure(const char* Filename)
	{
		uint32 Result = 5381;
		for (;;)
		{
			char Ch = *Filename++;
			if (!Ch)
			{
				return Result;
			}
			Result = ((Result << 5) + Result) + Ch;
		}
	}

	// Log a message that shows only on the output log.
	// Prefer using the USD_LOG_INFO, USD_LOG_WARNING, USD_LOG_ERROR, USD_LOG_USERINFO, USD_LOG_USERWARNING, USD_LOG_USERERROR macros.
	static UE_API void Log(EMessageSeverity::Type Severity, const FString& Message, uint32 MessageID);

	// Log a message that shows on the output log and the message log, if we're inside of a FScopedUsdMessageLog scope.
	// Prefer using the USD_LOG_INFO, USD_LOG_WARNING, USD_LOG_ERROR, USD_LOG_USERINFO, USD_LOG_USERWARNING, USD_LOG_USERERROR macros.
	static UE_API void Log(EMessageSeverity::Type Severity, const FText& Message, uint32 MessageID);

	// Returns whether we have currently accumulated any message with severity warning or higher.
	// If we're not in the scope of any FScopedUsdMessageLog, returns false.
	static UE_API bool HasAccumulatedErrors();

	static UE_API void RegisterDiagnosticDelegate();

	static UE_API void UnregisterDiagnosticDelegate();

	static UE_API void DisplayMessagesOnMessageLog(const TArray<TSharedRef<FTokenizedMessage>>& Messages);

	UE_DEPRECATED(5.6, "Use Log, or ideally the USD_LOG_INFO, USD_LOG_WARNING, etc. macros")
	static UE_API void LogMessage(EMessageSeverity::Type Severity, const FText& Message);

	UE_DEPRECATED(5.6, "Use Log, or ideally the USD_LOG_INFO, USD_LOG_WARNING, etc. macros")
	static UE_API void LogMessage(const TSharedRef<FTokenizedMessage>& Message);

	UE_DEPRECATED(5.6, "Used FScopedUsdMessageLog instead.")
	static UE_API void EnableMessageLog();

	UE_DEPRECATED(5.6, "Used FScopedUsdMessageLog instead.")
	static UE_API void DisableMessageLog();
};

/**
 * Display an Info/Warn/Error message on the Output Log (USD_LOG_INFO, USD_LOG_WARNING, USD_LOG_ERROR),
 * or display a user-facing Info/Warn/Error message, that is additionally added to the Message Log
 * (USD_LOG_USERINFO, USD_LOG_USERWARNING, USD_LOG_USERERROR).
 *
 * The user-facing message must be within the scope of a FScopedUsdMessageLog to be added to the Message Log, otherwise it is treated as the
 * non-user analogue. See FScopedUsdMessageLog right below this for more info.
 *
 * Usage examples:
 *     USD_LOG_WARNING(TEXT("Some error directly to the Output Log: %s"), *MyString);
 *     USD_LOG_USERERROR(LOCTEXT("MyMessage", "This is an error message that will show on the Message Log"));
 *     USD_LOG_USERWARNING(FText::Format(LOCTEXT("InvalidBinding", "Binding {0} is invalid!"), FText::FromString(UsdToUnreal::ConvertPath(InSkeletonRoot.GetPath()))));
 */
#define USD_LOG_INFO(Format, ...) 		FUsdLogManager::Log(EMessageSeverity::Info, 	FString::Printf(Format, ##__VA_ARGS__), HashCombine(FUsdLogManager::FileHashForEnsure(__FILE__), __LINE__));
#define USD_LOG_WARNING(Format, ...) 	FUsdLogManager::Log(EMessageSeverity::Warning, 	FString::Printf(Format, ##__VA_ARGS__), HashCombine(FUsdLogManager::FileHashForEnsure(__FILE__), __LINE__));
#define USD_LOG_ERROR(Format, ...) 		FUsdLogManager::Log(EMessageSeverity::Error, 	FString::Printf(Format, ##__VA_ARGS__), HashCombine(FUsdLogManager::FileHashForEnsure(__FILE__), __LINE__));
#define USD_LOG_USERINFO(text) 			FUsdLogManager::Log(EMessageSeverity::Info, 	text, 									HashCombine(FUsdLogManager::FileHashForEnsure(__FILE__), __LINE__));
#define USD_LOG_USERWARNING(text) 		FUsdLogManager::Log(EMessageSeverity::Warning, 	text, 									HashCombine(FUsdLogManager::FileHashForEnsure(__FILE__), __LINE__));
#define USD_LOG_USERERROR(text) 		FUsdLogManager::Log(EMessageSeverity::Error, 	text, 									HashCombine(FUsdLogManager::FileHashForEnsure(__FILE__), __LINE__));

/**
 * Begins a scope where all logged messages (user and non-user facing, USD SDK errors and even USD error mark messages) are collected and deduplicated.
 *
 * The deduplication only happens if the bOptimizeUsdLog project setting is enabled (default).
 *
 * If the USD.UseMessageLog cvar is enabled (default), it will add the user-facing messages (and the USD SDK and error mark messages) to the Message Log,
 * (whether deduplication is enabled or disabled), also displaying a toast letting the user know that those messages can be viewed there.
 */
class FScopedUsdMessageLog
{
public:
	UE_API FScopedUsdMessageLog(FUsdLogManager::FMessageDisplayCallback = FUsdLogManager::DisplayMessagesOnMessageLog);
	UE_API ~FScopedUsdMessageLog();

	FScopedUsdMessageLog(const FScopedUsdMessageLog&) = delete;
	FScopedUsdMessageLog& operator=(const FScopedUsdMessageLog&) = delete;
	FScopedUsdMessageLog(FScopedUsdMessageLog&&) = delete;
	FScopedUsdMessageLog& operator=(FScopedUsdMessageLog&&) = delete;
};

#undef UE_API
