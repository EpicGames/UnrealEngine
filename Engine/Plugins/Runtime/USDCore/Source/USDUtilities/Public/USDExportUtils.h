// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API USDUTILITIES_API

namespace UsdUnreal::ExportUtils
{
	/**
	 * Begins a UniquePathScope, incrementing the internal scope counter.
	 *
	 * During a UniquePathScope, all paths returned by GetUniqueFilePathForExport will be globally unique (i.e. it will
	 * never return the same path twice).
	 *
	 * Opening a scope while another scope is opened has no effect but to increment the scope counter further.
	 */
	USDUTILITIES_API void BeginUniquePathScope();

	/**
	 * Ends a UniquePathScope, decrementing the internal scope counter.
	 *
	 * If the internal scope counter reaches zero (i.e. all previously opened scopes are ended) this also clears the
	 * cache of unique paths.
	 */
	USDUTILITIES_API void EndUniquePathScope();

	/** Utility to call BeginUniquePathScope() on construction and EndUniquePathScope() on destruction */
	class FUniquePathScope final
	{
	public:
		UE_API FUniquePathScope();
		UE_API ~FUniquePathScope();

		FUniquePathScope(const FUniquePathScope&) = delete;
		FUniquePathScope(FUniquePathScope&&) = delete;
		FUniquePathScope& operator=(const FUniquePathScope&) = delete;
		FUniquePathScope& operator=(FUniquePathScope&&) = delete;
	};

	/**
	 * If we're inside of a UniquePathScope, returns a sanitized (and potentially suffixed) path that is guaranteed
	 * to not collide with any other path returned from this function during the UniquePathScope.
	 *
	 * If we're not inside of a UniquePathScope, returns the sanitized version of DesiredPathWithExtension.
	 */
	USDUTILITIES_API FString GetUniqueFilePathForExport(const FString& DesiredPathWithExtension);

	/**
	 * Sanitizes the Path to be used as a clean absolute file path.
	 * Normalizes separators, removes duplicate separators, collapses relative paths, etc.
	 */
	USDUTILITIES_API void SanitizeFilePath(FString& Path);
}

#undef UE_API
