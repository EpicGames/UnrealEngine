// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UnrealUSDWrapper.h"
#include "USDClassesModule.h"	 // Include this for now so that we can redirect FDisplayColorMaterial to the USDClasses definition
#include "UsdWrappers/ForwardDeclarations.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/usd/timeCode.h"
#include "pxr/usd/usdShade/tokens.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class TfToken;
	class UsdGeomMesh;
	class UsdGeomGprim;
	class UsdGeomPointInstancer;
	class UsdPrim;
	class UsdShadeMaterial;
	class UsdTyped;
	class UsdSkelSkinningQuery;
	class UsdSkelSkeletonQuery;
PXR_NAMESPACE_CLOSE_SCOPE
#endif	  // USE_USD_SDK

class UAnimSequence;
class UGeometryCache;
class UMaterialInstanceConstant;
class UMaterialInstanceDynamic;
class USkeletalMesh;
class USkeletalMeshComponent;
class UStaticMesh;
struct FMeshDescription;
struct FStaticMeshLODResources;
struct FUsdCollapsingCache;
struct FUsdStageInfo;
enum class EUsdCollisionType : uint8;

namespace UsdUtils
{
	struct FUsdPrimMaterialAssignmentInfo;
}

namespace UE
{
	class FUsdPrim;
}

namespace UsdUnreal::MaterialUtils
{
	struct FDisplayColorMaterial;
}

#if USE_USD_SDK
namespace UsdToUnreal
{
	/** Common options to mesh conversion functions */
	struct FUsdMeshConversionOptions
	{
		/** This enum must match EInterchangeUsdPrimvar in InterchangeUsdDefinitions.h*/
		enum EImportPrimvar
		{
			/** Store only the standard primvars such as UVs, VertexColors, etc.*/
			Standard = 0,

			/** Store only primvars in the Mesh Description used for baking to textures (basically <geompropvalue> node from MaterialX shadergraphs that are converted to <image>)*/
			Bake,

			/** Store all primvars in the MeshDescription, regardless*/
			All
		};

		// These need to be explicitly defaulted surrounded by the deprecation macros or else the implicit versions will
		// be generated elsewhere, use MaterialToPrimvarToUVIndex and trigger the deprecation warnings
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FUsdMeshConversionOptions(const FUsdMeshConversionOptions& Other) = default;
		FUsdMeshConversionOptions& operator=(const FUsdMeshConversionOptions& Other) = default;
		USDUTILITIES_API PRAGMA_ENABLE_DEPRECATION_WARNINGS

		const static FUsdMeshConversionOptions DefaultOptions;
		USDUTILITIES_API FUsdMeshConversionOptions();

		// List of the primvars that should be stored on the MeshDescription
		TSet<FString> PrimvarNames;

		// Additional UE-space transform to bake into the mesh's vertices
		FTransform AdditionalTransform;

		// Only prims with these purposes will be traversed
		EUsdPurpose PurposesToLoad;

		// Render context to use when parsing Mesh materials
		pxr::TfToken RenderContext;

		// Material purpose to use when parsing Mesh materials
		pxr::TfToken MaterialPurpose;

		// USD timecode when the prims should be sampled for mesh data and converted
		pxr::UsdTimeCode TimeCode;

		// Whether to try reusing material slots (both local and the ones already in MaterialAssignments) when
		// converting the material assignments from the mesh
		bool bMergeIdenticalMaterialSlots;

		// What subdivision level to subdivide to, if the converted mesh is a subdivision mesh.
		// 0 means no subdivision should take place (default).
		int32 SubdivisionLevel;

		// How to store the primvars in the MeshDescription
		EImportPrimvar ImportPrimvars;

		// Whether to add an additional offset to vertex join indices if the skinned mesh's skeleton has multiple root bones
		bool bOffsetJointIndicesForMultipleRootBones;

		// Whether to always check the deprecated "unrealMaterial" string attribute on prims regardless of the configured RenderContext.
		// When false (the default), the unrealMaterial attribute is only checked when the RenderContext is "unreal".
		// This is mainly used by Interchange: The render context is resolved only later (and we always provide 'universal'), so we need
		// this additional parameter to not ignore the string attribute completely
		bool bForceCheckUnrealMaterialAttribute;

		// If true, a bound material is only used if it has a surface shader output for the configured RenderContext.
		// If it doesn't, the binding is rejected and processing falls through to the DisplayColor fallback.
		// Legacy USD importers set this to true; Interchange sets it to false because the render context is resolved at a later stage.
		bool bRequireMaterialsHaveProvidedRenderContext;
	};

	/**
	 * Converts an UsdGeomMesh and appends the converted data to InOutMeshDescription and InOutMaterialAssignments.
	 *
	 * If a PrimvarToUVIndex mapping is provided in InOutMaterialAssignments, it will be used to remap the read
	 * primvars into UV sets. Otherwise, a new PrimvarToUVIndex map will be constructed on-demand and assigned
	 * to InOutMaterialAssignments instead.
	 * If SkinningQuery and SkeletonQuery are provided we parse the Skeletal Attributes and fill the FMeshDescription accordingly.
	 *
	 * @param Mesh - Mesh prim to convert
	 * @param InOutMeshDescription - Input/Output parameter that will be filled with the converted data
	 * @param InOutMaterialAssignments - Input/Output parameter that will be filled with the material data extracted from Mesh
	 * @param CommonOptions - Shared options used for the conversion
	 * @param SkinningQuery - For computing Skinned Points, if the target and source data set is for a SkeletalMesh.
	 * @param SkeletonQuery - For computing Skinned Points, if the target and source data set is for a SkeletalMesh. (primarily for
	 * MeshToSkeletonRestPose calculations)
	 * @return Whether the conversion was successful or not.
	 */
	USDUTILITIES_API bool ConvertGeomMesh(
		const pxr::UsdGeomMesh& Mesh,
		FMeshDescription& InOutMeshDescription,
		UsdUtils::FUsdPrimMaterialAssignmentInfo& InOutMaterialAssignments,
		const FUsdMeshConversionOptions& CommonOptions = FUsdMeshConversionOptions::DefaultOptions,
		const pxr::UsdSkelSkinningQuery* SkinningQuery = nullptr,
		const pxr::UsdSkelSkeletonQuery* SkeletonQuery = nullptr
	);
	USDUTILITIES_API bool ConvertGeomMesh(
		const pxr::UsdPrim& MeshPrim,
		FMeshDescription& InOutMeshDescription,
		UsdUtils::FUsdPrimMaterialAssignmentInfo& InOutMaterialAssignments,
		const FUsdMeshConversionOptions& CommonOptions = FUsdMeshConversionOptions::DefaultOptions,
		const pxr::UsdSkelSkinningQuery* SkinningQuery = nullptr,
		const pxr::UsdSkelSkeletonQuery* SkeletonQuery = nullptr
	);

	/** Bake a point instancer prim into a single FMeshDescription and MaterialAssignments, manually "instancing" all
	 *  the prototype instances into a single combined mesh.
	 *
	 * @param PointInstancer - UsdGeomPointInstancer to bake
	 * @param OutMeshDescription - Output parameter that will be filled with the converted data
	 * @param OutMaterialAssignments - Output parameter that will be filled with the material data extracted from
	 *								   PointInstancer
	 * @param CommonOptions - Shared options used for the conversion
	 * @return Whether the conversion was successful or not.
	 */
	USDUTILITIES_API bool ConvertPointInstancerToMesh(
		const pxr::UsdGeomPointInstancer& PointInstancer,
		FMeshDescription& OutMeshDescription,
		UsdUtils::FUsdPrimMaterialAssignmentInfo& OutMaterialAssignments,
		const FUsdMeshConversionOptions& CommonOptions = FUsdMeshConversionOptions::DefaultOptions
	);

	/**
	 * Recursively traverses down Prim's subtree parsing all Mesh data (including Prim's), baking all transforms and
	 * putting the resulting mesh data within OutMeshDescription and OutMaterialAssignments. It will not bake Prim's
	 * own transform into the Mesh data.
	 *
	 * If a PrimvarToUVIndex mapping is provided in InOutMaterialAssignments, it will be used to remap the read
	 * primvars into UV sets. Otherwise, a new PrimvarToUVIndex map will be constructed on-demand and assigned
	 * to the InOutMaterialAssignments instead.
	 *
	 * @param Prim - Prim that will be converted (result will include its mesh and its children's)
	 * @param OutMeshDescription - Input/Output parameter that will be filled with the converted mesh descriptions
	 * @param OutMaterialAssignments - Input/Output parameter that will be filled with the material data extracted from
									   UsdSchema. Each item of the array matches the mesh description of same index
	 * @param CommonOptions - Shared options used for the conversion
	 * @param bSkipRootPrimTransform - Whether to bake in the transform of SubtreeRootPrim into the mesh
	 * @param bSkipRootPrimVisiblity - Whether to bake in the visibility of SubtreeRootPrim into the mesh
	 * @return Whether the conversion was successful or not.
	 */
	USDUTILITIES_API bool ConvertGeomMeshSubtree(
		const pxr::UsdPrim& SubtreeRootPrim,
		FMeshDescription& InOutMeshDescription,
		UsdUtils::FUsdPrimMaterialAssignmentInfo& InOutMaterialAssignments,
		const FUsdMeshConversionOptions& CommonOptions = FUsdMeshConversionOptions::DefaultOptions,
		bool bSkipRootPrimTransform = false,
		bool bSkipRootPrimVisiblity = false
	);
	UE_DEPRECATED(5.6, "Use ConvertGeomMeshSubtree instead.")
	USDUTILITIES_API bool ConvertGeomMeshHierarchy(
		const pxr::UsdPrim& Prim,
		FMeshDescription& InOutMeshDescription,
		UsdUtils::FUsdPrimMaterialAssignmentInfo& InOutMaterialAssignments,
		const FUsdMeshConversionOptions& CommonOptions = FUsdMeshConversionOptions::DefaultOptions,
		bool bSkipRootPrimTransformAndVisibility = false
	);

	/**
	 * Converts an UsdGeomCapsule(_1), UsdGeomCone, UsdGeomCube, UsdGeomCylinder(_1), UsdGeomPlane or UsdGeomSphere and appends
	 * the converted data to InOutMeshDescription and InOutMaterialAssignments.
	 *
	 * Note that all generated meshes will have extents within the [-0.5, 0.5] box centered on the origin.
	 * You can pass additional transforms on CommonOptions.AdditionalTransform though.
	 *
	 * @param Prim - USD Prim prim to convert
	 * @param InOutMeshDescription - Input/Output parameter that will be filled with the converted data
	 * @param InOutMaterialAssignments - Input/Output parameter that will be filled with the material data extracted from Mesh
	 * @param CommonOptions - Shared options used for the conversion
	 * @return Whether the conversion was successful or not.
	 */
	USDUTILITIES_API bool ConvertGeomPrimitive(
		const pxr::UsdPrim& Prim,
		FMeshDescription& InOutMeshDescription,
		UsdUtils::FUsdPrimMaterialAssignmentInfo& InOutMaterialAssignments,
		const FUsdMeshConversionOptions& CommonOptions = FUsdMeshConversionOptions::DefaultOptions
	);

	/**
	 * Extracts a "primitive transform" from the provided UsdGeomCapsule(_1), UsdGeomCone, UsdGeomCube, UsdGeomCylinder(_1),
	 * UsdGeomPlane or UsdGeomSphere and places the converted, UE coordinate system transform on OutTransform.
	 *
	 * This is intended to be applied to the primitive data generated by ConvertGeomPrimitive, and concatenated with
	 * any actual Xform on the prim (Xform applied after). The idea is that the meshes generated there are standard
	 * size and fill the [-0.5, 0.5] box centered on the origin (length 1). If a Cylinder has "height=2.5", this function
	 * will produce a scaling of 2.5 on the longitudinal axis for that cylinder.
	 *
	 * @param Prim - USD Prim prim to convert
	 * @param InTimeCode - TimeCode to sample the
	 * @param OutTransform - The converted transform on the UE coordinate system
	 * @return Whether the conversion was successful or not.
	 */
	USDUTILITIES_API bool ConvertGeomPrimitiveTransform(const pxr::UsdPrim& InPrim, const pxr::UsdTimeCode& InTimeCode, FTransform& OutTransform);
}	 // namespace UsdToUnreal

namespace UnrealToUsd
{
	/**
	 * Extracts mesh data from StaticMesh and places the results in UsdPrim, as children UsdGeomMeshes.
	 * This function receives the parent UsdPrim as it may create a variant set named 'LOD', and create a separate
	 * UsdGeomMeshes for each LOD, as a variant of 'LOD'
	 * @param StaticMesh - StaticMesh to convert
	 * @param UsdPrim - Prim to receive the mesh data or LOD variant set
	 * @param TimeCode - TimeCode to author the attributes with
	 * @param StageForMaterialAssignments - Stage to use when authoring material assignments (we use this when we want to export the mesh to a payload
	 * layer, but the material assignments to an asset layer)
	 * @param LowestMeshLOD - Lowest LOD of the UStaticMesh to export (start of the LOD range)
	 * @param HighestMeshLOD - Lowest LOD of the UStaticMesh to export (end of the LOD range)
	 * @param bExportStaticMeshSourceData - Whether to export the source mesh description data (if true) or the render data (if false)
	 * @return Whether the conversion was successful or not.
	 */
	USDUTILITIES_API bool ConvertStaticMesh(
		const UStaticMesh* StaticMesh,
		pxr::UsdPrim& UsdPrim,
		pxr::UsdTimeCode TimeCode = pxr::UsdTimeCode::Default(),
		UE::FUsdStage* StageForMaterialAssignments = nullptr,
		int32 LowestMeshLOD = 0,
		int32 HighestMeshLOD = INT32_MAX,
		bool bExportStaticMeshSourceData = false
	);

	/**
	 * Converts an array of mesh descriptions into mesh data, and places that data within the UsdGeomMesh UsdPrim.
	 * If only one MeshDescription is provided, the mesh data is added directly to the prim.
	 * If more than one MeshDescription are provided, a 'LOD' variant set will be created for UsdPrim, and LOD0, LOD1, etc. variants will be
	 * created for each provided LOD index. Within each variant, a single Mesh prim also named LOD0, LOD1, etc. will contain the mesh data.
	 */
	USDUTILITIES_API bool ConvertMeshDescriptions(
		const TArray<FMeshDescription>& LODIndexToMeshDescription,
		pxr::UsdPrim& UsdPrim,
		const FMatrix& AdditionalTransform,
		const pxr::UsdTimeCode TimeCode = pxr::UsdTimeCode::Default()
	);

	/**
	 * Extracts animated mesh data from GeometryCache and places the results in UsdPrim.
	 * @param GeometryCache - GeometryCache to convert
	 * @param UsdPrim - Prim to receive the mesh data
	 * @param StageForMaterialAssignments - Stage to use when authoring material assignments (we use this when we want to export the mesh to a payload
	 * layer, but the material assignments to an asset layer)
	 * @return Whether the conversion was successful or not.
	 */
	USDUTILITIES_API bool ConvertGeometryCache(
		const UGeometryCache* GeometryCache,
		pxr::UsdPrim& UsdPrim,
		UE::FUsdStage* StageForMaterialAssignments = nullptr
	);

	USDUTILITIES_API bool ConvertSkeletalMeshToStaticMesh(
		const USkeletalMesh* SkeletalMesh,
		pxr::UsdPrim& UsdPrim,
		const pxr::UsdTimeCode TimeCode = pxr::UsdTimeCode::Default(),
		UE::FUsdStage* StageForMaterialAssignments = nullptr
	);

	USDUTILITIES_API bool ConvertAnimSequenceToAnimatedMesh(
		UAnimSequence* AnimSequence,
		USkeletalMesh* SkeletalMesh,
		pxr::UsdPrim& UsdPrim,
		UE::FUsdStage* StageForMaterialAssignments = nullptr
	);

	/**
	 * Creates a baker responsible for baking the skinned mesh of a skeletal mesh component to a Mesh prim,
	 * returning true if a EBakingType::Skeletal baker was created
	 */
	USDUTILITIES_API bool CreateSkeletalAnimationToMeshBaker(
		UE::FUsdPrim& MeshPrim,
		USkeletalMeshComponent& Component,
		struct FComponentBaker& OutBaker
	);
}
#endif	  // USE_USD_SDK

namespace UsdUtils
{
	using FDisplayColorMaterial = UsdUnreal::MaterialUtils::FDisplayColorMaterial;

	/** Describes what type of material assignment a FUsdPrimMaterialSlot represents */
	enum class EPrimAssignmentType : uint8
	{
		None,			  // There is no assignment for this material slot (or no material override)
		DisplayColor,	  // MaterialSource is a serialized FDisplayColorMaterial (e.g. '!DisplayColor_0_0')
		MaterialPrim,	  // MaterialSource is the USD path to a Material prim on the stage (e.g. '/Root/Materials/Red')
		UnrealMaterial	  // MaterialSource is the package path to an UE material (e.g. '/Game/Materials/Red.Red')
	};

	/**
	 * Description of a material slot we need to add to a UStaticMesh or USkeletalMesh to have it represent a material binding in a prim.
	 * It may or may not contain an actual material assignment.
	 */
	struct FUsdPrimMaterialSlot
	{
		/**
		 * Path to the prims that contain this material assignment (e.g. '/Root/my_cube' or '/Root/my_cube/geomsubset_0'.
		 * This is a set because this single material slot in UE may be merged from N identical prims.
		 */
		TSet<FString> PrimPaths;

		/** What this represents depends on AssignedMaterialType */
		FString MaterialSource;
		EPrimAssignmentType AssignmentType = EPrimAssignmentType::None;
		bool bMeshIsDoubleSided = false;

		friend bool operator==(const FUsdPrimMaterialSlot& Lhs, const FUsdPrimMaterialSlot& Rhs)
		{
			// Note: We intentionally don't compare the slot name itself, so that otherwise identical
			// slots with different names can be merged together
			return Lhs.AssignmentType == Rhs.AssignmentType &&									   //
				   Lhs.MaterialSource.Equals(Rhs.MaterialSource, ESearchCase::CaseSensitive) &&	   //
				   Lhs.bMeshIsDoubleSided == Rhs.bMeshIsDoubleSided;
		}

		friend uint32 GetTypeHash(const FUsdPrimMaterialSlot& Slot)
		{
			// Note: We intentionally don't hash the slot name itself, so that otherwise identical
			// slots with different names can be merged together
			return HashCombine(
				Slot.bMeshIsDoubleSided,	//
				HashCombine(				//
					GetTypeHash(Slot.MaterialSource),
					static_cast<uint32>(Slot.AssignmentType)
				)
			);
		}

		friend FArchive& operator<<(FArchive& Ar, FUsdPrimMaterialSlot& Slot)
		{
			Ar << Slot.MaterialSource;
			Ar << Slot.AssignmentType;
			Ar << Slot.bMeshIsDoubleSided;
			Ar << Slot.PrimPaths;

			return Ar;
		}
	};

	/** Complete description of material assignment data of a pxr::UsdPrim */
	struct FUsdPrimMaterialAssignmentInfo
	{
		TArray<FUsdPrimMaterialSlot> Slots;

		/** Describes the index of the Slot that each polygon/face of a mesh uses. Matches the order in Slots */
		TArray<int32> MaterialIndices;

		/** Describes which primvars should be assigned to each UV index for this mesh. */
		TMap<FString, int32> PrimvarToUVIndex;
	};

	USDUTILITIES_API void RepairNormalsAndTangents(const FString& PrimPath, FMeshDescription& MeshDescription);

#if USE_USD_SDK
	/** Creates a FDisplayColorMaterial object describing the vertex color/opacity data from Gprim at time TimeCode */
	USDUTILITIES_API TOptional<FDisplayColorMaterial> ExtractDisplayColorMaterial(
		const pxr::UsdGeomGprim& Gprim,
		const pxr::UsdTimeCode TimeCode = pxr::UsdTimeCode::EarliestTime()
	);

	/**
	 * Extracts all material assignment data from UsdPrim, including material binding, multiple assignment with GeomSubsets, and the unrealMaterial
	 * custom USD attribute. Guaranteed to return at least one material slot. If UsdPrim is a UsdGeomMesh, it is also guaranteed to have valid
	 * material indices (one for every face).
	 * @param UsdPrim - Prim to extract material assignments from
	 * @param TimeCode - Instant where the material data is sampled
	 * @param bProvideMaterialIndices - Whether to fill out the material index information for the assignment info (which can be expensive). If this
	 * is false, MaterialIndices on the result Struct will have zero values
	 * @param RenderContext - Which render context to get the materials for. Defaults to universal.
	 * @param MaterialPurpose - Which material purpose to use when retrieving material bindings
	 * @param bForceCheckUnrealMaterialAttribute - Whether to always check the "unrealMaterial" string attribute regardless of render context
	 * @param bRequireMaterialsHaveProvidedRenderContext - If this is enabled, we'll only consider a material binding if it has a surface output
	 * for the provided render context. If this is false, we'll use the bound material regardless
	 * @return Struct containing an array of material assignments, and a corresponding array of material indices for all polygons of the prim,
	 * matching the ordering of the material assignments.
	 */
	USDUTILITIES_API FUsdPrimMaterialAssignmentInfo GetPrimMaterialAssignments(
		const pxr::UsdPrim& UsdPrim,
		const pxr::UsdTimeCode TimeCode = pxr::UsdTimeCode::EarliestTime(),
		bool bProvideMaterialIndices = true,
		const pxr::TfToken& RenderContext = pxr::UsdShadeTokens->universalRenderContext,
		const pxr::TfToken& MaterialPurpose = pxr::UsdShadeTokens->allPurpose,
		bool bForceCheckUnrealMaterialAttribute = false,
		bool bRequireMaterialsHaveProvidedRenderContext = true
	);

	/** Returns whether this prim can be interpreted as describing a static mesh with multiple LODs (i.e. if the prim holds the LOD variant set
	 * itself) */
	USDUTILITIES_API bool DoesPrimContainMeshLODs(const pxr::UsdPrim& Prim);

	/** Returns whether this UsdMesh can be interpreted as a LOD of a mesh with multiple LODs (i.e. if the Mesh prim is inside a LOD variant) */
	USDUTILITIES_API bool IsGeomMeshALOD(const pxr::UsdPrim& UsdMeshPrim);

	/** Returns whether this UsdPrim has the UsdPhysicsCollisionAPI schema and has enabled collisions */
	USDUTILITIES_API bool IsCollisionEnabledForPrim(const pxr::UsdPrim& UsdPrim);

	/** Returns whether this UsdPrim is a custom collision mesh */
	USDUTILITIES_API bool IsCollisionMesh(const pxr::UsdPrim& UsdPrim);

	/** Returns which collision type should be generated for the provided prim, according to the applied physics schemas */
	USDUTILITIES_API EUsdCollisionType GetCollisionApproximationType(const pxr::UsdPrim& UsdPrim);

	/** Returns how many LOD variants the Prim has. Note that this will return 0 if called on one of the LOD meshes themselves, it's meant to be
	 * called on its parent */
	USDUTILITIES_API int32 GetNumberOfLODVariants(const pxr::UsdPrim& Prim);

	/**
	 * If a prim has a variant set named "LOD", with variants named "LOD0", "LOD1", etc., and each has a single Mesh prim, this function
	 * will actively switch the variants of ParentPrim so that each child mesh is loaded, and call Func on each.
	 * Returns whether ParentPrim met the aforementioned criteria and Func was called at least once.
	 * WARNING: There is no guarantee about LOD index ordering! Func may receive LOD2, followed by LOD0, then LOD1, etc.
	 * WARNING: This will temporarily mutate the stage, and can invalidate references to children of ParentPrim!
	 */
	USDUTILITIES_API bool IterateLODMeshes(const pxr::UsdPrim& ParentPrim, TFunction<bool(const pxr::UsdGeomMesh& LODMesh, int32 LODIndex)> Func);

	/**
	 * Traverses `Stage` and authors material binding attributes for all `unrealMaterials` that were baked into USD material assets.
	 * @param Stage - Stage to replace the unrealMaterials attributes in. All of it's layers will be traversed.
	 * @param LayerToAuthorIn - Where the material binding opinions are authored
	 * @param BakedMaterials - Maps from material path names to file paths where they were baked
	 *                         Example: { "/Game/MyMaterials/Red.Red": "C:/MyFolder/Red.usda" }
	 * @param bIsAssetLayer - True when we're exporting a single mesh/animation asset. False when we're exporting a level. Dictates minor behaviors
	 *                        when authoring the material binding relationships, e.g. whether we author them inside variants or not
	 * @param bUsePayload - Should be True if the Stage was exported using payload files to store the actual Mesh prims. Also dictates minor
	 *                      behaviors when authoring the material binding relationships.
	 */
	USDUTILITIES_API void ReplaceUnrealMaterialsWithBaked(
		const UE::FUsdStage& Stage,
		const UE::FSdfLayer& LayerToAuthorIn,
		const TMap<FString, FString>& BakedMaterials,
		bool bIsAssetLayer,
		bool bUsePayload
	);

	/**
	 * Hashes the attributes of the GeomMesh at the given prim path on the Stage at TimeCode
	 * If the prim path is invalid or not a GeomMesh, it will return an empty hash
	 */
	USDUTILITIES_API FString HashGeomMeshPrim(const UE::FUsdStage& Stage, const FString& PrimPath, double TimeCode);
	USDUTILITIES_API void HashGeomMeshPrim(const UE::FUsdStage& Stage, const FString& PrimPath, double TimeCode, FMD5& InOutHashState);

	/**
	 * Places in OutInstanceTransforms the UE-space instance transforms for a given point instancer prototype index.
	 * Returns whether the transforms were successfully retrieved or not.
	 */
	USDUTILITIES_API bool GetPointInstancerTransforms(
		const FUsdStageInfo& StageInfo,
		const pxr::UsdGeomPointInstancer& PointInstancer,
		const int32 ProtoIndex,
		pxr::UsdTimeCode EvalTime,
		TArray<FTransform>& OutInstanceTransforms
	);

	struct FUsdPointInstancerInstancesData
	{
		struct FUsdPrototypeInstances
		{
			FString PathString;
			TArray<FTransform> InstanceTransforms;
		};

		TArray<FUsdPrototypeInstances> Prototypes;
	};

	/** Acquires the Prototypes from the PointInstancer and the related InstanceTransforms. */
	USDUTILITIES_API void GetPointInstancerInstancesData(const pxr::UsdPrim& Prim, FUsdPointInstancerInstancesData& InstancesData);

	/** Returns true if Prim is a GeomMesh with animated attributes */
	USDUTILITIES_API bool IsAnimatedMesh(const pxr::UsdPrim& Prim);

	USDUTILITIES_API void GetAnimatedMeshTimeCodes(const UE::FUsdStage& Stage, const FString& PrimPath, int32& OutStartFrame, int32& OutEndFrame);

	/** Returns all of the Prim UsdGeomMesh-related attributes (points, faceVertexIndices, etc.) that contain timeSamples */
	USDUTILITIES_API TArray<UE::FUsdAttribute> GetAnimatedMeshAttributes(const pxr::UsdPrim& Prim);

	/** Mesh topology variance as defined by Alembic's MeshTopologyVariance */
	enum class EMeshTopologyVariance : uint8
	{
		Constant,		// unchanging topology
		Homogenous,		// connectivity is unchanging, but the positions may change
		Heterogenous	// connectivity and positions may change
	};

	/** Returns the type of topology variance UsdMesh has */
	USDUTILITIES_API EMeshTopologyVariance GetMeshTopologyVariance(const pxr::UsdGeomMesh& UsdMesh);
	USDUTILITIES_API EMeshTopologyVariance GetMeshTopologyVariance(const UE::FUsdPrim& UsdPrim);

	USDUTILITIES_API uint64 GetGprimVertexCount(const pxr::UsdGeomGprim& Gprim, double TimeCode);

	USDUTILITIES_API void AuthorIdentityTransformGprimAttributes(const pxr::UsdPrim& Gprim, bool bDefaultValues, bool bTimeSampleValues);
#endif	  // #if USE_USD_SDK
}	 // namespace UsdUtils
