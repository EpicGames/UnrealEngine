// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "CoreMinimal.h"

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class UsdPrim;
	class UsdTimeCode;
PXR_NAMESPACE_CLOSE_SCOPE

class FHairDescription;
struct FGroomAnimationInfo;

namespace UsdToUnreal
{
	/**
	 * Converts an UsdPrim groom hierarchy and appends the converted data to HairDescription
	 *
	 * @param Prim - Prim to convert
	 * @param TimeCode - The timecode at which to query the prim attributes
	 * @param ParentTransform - The prim transform to propagate to the children
	 * @param HairDescription - The hair description where the groom prim data is outputted
	 * @param AnimInfo -  Optional GroomAnimationInfo to fill when parsing the hierarchy
	 * @param bCheckGroomAttributes -  Optional Whether to check the groom-scope attributes, 
	 *                                 should only be checked (true) on the top-level prim
	 * @return true if the conversion was successful; false otherwise
	 */
	USDUTILITIES_API bool ConvertGroomHierarchy(
		const pxr::UsdPrim& Prim,
		const pxr::UsdTimeCode& TimeCode,
		const FTransform& ParentTransform,
		FHairDescription& HairDescription,
		FGroomAnimationInfo* AnimInfo = nullptr,
		bool bCheckGroomAttributes = true
	);
}

#endif	  // #if USE_USD_SDK
