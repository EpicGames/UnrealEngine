// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class UsdPrim;
	class TfToken;
PXR_NAMESPACE_CLOSE_SCOPE

#endif	  // #if USE_USD_SDK

class UActorComponent;
class ULiveLinkComponentController;

namespace UsdUtils
{
#if USE_USD_SDK
	// Returns true if Prim has the given Schema
	USDUTILITIES_API bool PrimHasSchema(const pxr::UsdPrim& Prim, const pxr::TfToken& Schema);

	// Returns true if we can apply a single-apply API schema named "SchemaName" to Prim
	USDUTILITIES_API bool CanApplySchema(pxr::UsdPrim Prim, pxr::TfToken SchemaName);

	// Apply the given Schema to the Prim. Returns true if the application was successful
	USDUTILITIES_API bool ApplySchema(const pxr::UsdPrim& Prim, const pxr::TfToken& Schema);

	// Returns true if we can remove a single-apply API schema named "SchemaName" from Prim
	USDUTILITIES_API bool CanRemoveSchema(pxr::UsdPrim Prim, pxr::TfToken SchemaName);

	// Remove the given Schema from the Prim. Returns true if was successful
	USDUTILITIES_API bool RemoveSchema(const pxr::UsdPrim& Prim, const pxr::TfToken& Schema);
#endif	  // USE_USD_SDK
}

namespace UnrealToUsd
{
#if USE_USD_SDK

	/**
	 * Converts UE component properties related to LiveLink into values for the attributes of our custom LiveLinkAPI
	 * schema.
	 * @param InComponent		The main component with data to convert
	 * @param InOutPrim			Prim with the LiveLinkAPI schema to receive the data
	 */
	USDUTILITIES_API void ConvertLiveLinkProperties(const UActorComponent* InComponent, pxr::UsdPrim& InOutPrim);

#endif	  // USE_USD_SDK
}
