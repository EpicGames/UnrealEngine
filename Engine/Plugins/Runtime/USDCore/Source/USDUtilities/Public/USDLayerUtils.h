// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "UsdWrappers/ForwardDeclarations.h"

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

class SWidget;
template<typename UsdObjectType>
class TUsdStore;

PXR_NAMESPACE_OPEN_SCOPE
	class SdfLayer;
	class UsdAttribute;
	class UsdPrim;
	class UsdStage;

	template<typename T>
	class TfRefPtr;

	using SdfLayerRefPtr = TfRefPtr<SdfLayer>;
	using UsdStageRefPtr = TfRefPtr<UsdStage>;
PXR_NAMESPACE_CLOSE_SCOPE

namespace UE
{
	class FUsdAttribute;
	class FUsdPrim;
	struct FSdfLayerOffset;
}

namespace UsdUtils
{
	enum class EBrowseFileMode
	{
		/** Browse for a file for the purpose of opening a stage with that file as the stage's root layer. */
		Open,

		/** Browse for a file for the purpose of using that file as the target of a composition arc (e.g. a sublayer, reference, payload, etc.). */
		Composition,

		/** Browse for a file for the purpose of saving or exporting a layer to that file. */
		Save
	};

	enum class EUSDZDecompressResult
	{
		/** Extraction failed due to invalid, unsafe, or unsupported archive contents. */
		Failed,

		/** Extraction succeeded, but some entries were skipped. */
		SuccessWithSkippedEntries,

		/** Extraction succeeded with all entries processed. */
		Success
	};

	enum class ECanInsertSublayerResult
	{
		Success,
		ErrorSubLayerNotFound,
		ErrorSubLayerInvalid,
		ErrorSubLayerIsParentLayer,
		ErrorCycleDetected,
	};
	USDUTILITIES_API FText ToText(ECanInsertSublayerResult Result);

	/**
	 * Checks whether we can add SubLayerIdentifier as a sublayer to ParentLayer.
	 * The index is actually not relevant.
	 */
	USDUTILITIES_API ECanInsertSublayerResult CanInsertSubLayer(const pxr::SdfLayerRefPtr& ParentLayer, const TCHAR* SubLayerIdentifier);

	/**
	 * Inserts the SubLayerFile path into ParentLayer as a sublayer
	 * @param ParentLayer - Layer to receive the new sublayer entry
	 * @param SubLayerFile - Absolute path to a usd file to use as sublayer
	 * @param Index - Zero-based index on the list of ParentLayer's sublayers list to insert the new sublayer.
					  -1 means "at the end" (the default)
	 * @param OffsetTimeCodes - Offset in USD time codes to use for the sublayer reference
	 * @param TimeCodesScale - Scale to use for the sublayer reference
	 * @return Whether the sublayer reference was added successfully or not
	 */
	USDUTILITIES_API bool InsertSubLayer(
		const pxr::SdfLayerRefPtr& ParentLayer,
		const TCHAR* SubLayerFile,
		int32 Index = -1,
		double OffsetTimeCodes = 0.0,
		double TimeCodesScale = 1.0
	);

#if WITH_EDITOR
	/** Opens a file dialog to open or save a USD file. The returned file path will always be absolute */
	USDUTILITIES_API TOptional<FString> BrowseUsdFile(EBrowseFileMode Mode);
#endif	  // #if WITH_EDITOR

	/**
	 * Converts the file path from being absolute or relative to engine binary, into being relative to the current project's directory.
	 * It will only do this if the file is actually within the project's directory (or within its folder tree). Otherwise it will return an absolute
	 * path
	 */
	USDUTILITIES_API FString MakePathRelativeToProjectDir(const FString& Path);

	/**
	* Returns true if the given path is a zip archive, by checking the first four 'magic' bytes. This prevents a usd zip archive with a 
	* usda or usdc suffix from eliding our unzip handling, which is required for correctness and zip safety/exploit reasons.
	*/

	USDUTILITIES_API bool IsZipArchive(const FString& InUSDZFilePath);
	/**
	 * If InFilePathIntoUSDZArchive is a path to something inside an USDZ file, like "C:/MyFiles/scene.usdz[0/texture.png]",
	 * this will put "C:/MyFiles/scene.usdz" on OutUSDZFilePath, and "0/texture.png" on OutInnerAssetPath and return true.
	 * Returns false when it can't split the provided path in that way.
	 */
	USDUTILITIES_API bool SplitUSDZAssetPath(const FString& InFilePathIntoUSDZArchive, FString& OutUSDZFilePath, FString& OutInnerAssetPath);

	/**
	 * If InUSDZFilePath points at an actual USDZ file, this function will decompress the archive and put its contents inside InOutputDir,
	 * returning Success or SuccessWithSkippedEntries if succeeding. In that case OutDecompressedRootLayer will also be set to the
	 * path of the root layer of the decompressed USDZ archive. Note that for safety reasons, InOutputDir must not exist prior to 
	 * calling this function. The directory will be created once the zip file contents have been validated, and will be removed if
	 * an error is encountered during decompression.
	 */
	USDUTILITIES_API EUSDZDecompressResult DecompressUSDZFile(const FString& InUSDZFilePath, const FString& InOutputDir, FString* OutDecompressedRootLayer = nullptr);

	/** Creates a new layer with a default prim */
	USDUTILITIES_API TUsdStore<pxr::SdfLayerRefPtr> CreateNewLayer(
		TUsdStore<pxr::UsdStageRefPtr> UsdStage,
		const TUsdStore<pxr::SdfLayerRefPtr>& ParentLayer,
		const TCHAR* LayerFilePath
	);

	/** Finds which layer introduced the prim in the stage local layer stack */
	USDUTILITIES_API UE::FSdfLayer FindLayerForPrim(const pxr::UsdPrim& Prim);

	/** Finds the strongest layer contributing to an attribute */
	USDUTILITIES_API UE::FSdfLayer FindLayerForAttribute(const pxr::UsdAttribute& Attribute, double TimeCode);

	/** Finds the strongest layer that contributes to a set of attributes. Assumes these attributes are on the same stage */
	UE_DEPRECATED(5.7, "Use UsdUtils::GetStrongestAttribute() and then UsdUtils::FindLayerForAttribute()")
	USDUTILITIES_API UE::FSdfLayer FindLayerForAttributes(
		const TArray<UE::FUsdAttribute>& Attributes,
		double TimeCode,
		bool bIncludeSessionLayers = true
	);

	/** Finds the layer for a sublayer path of a given root layer */
	USDUTILITIES_API UE::FSdfLayer FindLayerForSubLayerPath(const UE::FSdfLayer& RootLayer, const FStringView& SubLayerPath);

	/** Sets the layer offset for the strongest reference or payload in this prim composition arcs */
	USDUTILITIES_API bool SetRefOrPayloadLayerOffset(pxr::UsdPrim& Prim, const UE::FSdfLayerOffset& LayerOffset);

	/** Finds the layer offset that converts the Attribute local times to stage times */
	USDUTILITIES_API UE::FSdfLayerOffset GetLayerToStageOffset(const pxr::UsdAttribute& Attribute);

	/** Finds the full offset for a prim with respect to the composed stage */
	USDUTILITIES_API UE::FSdfLayerOffset GetPrimToStageOffset(const UE::FUsdPrim& Prim);

	/** Makes sure that the layer start and end timecodes include StartTimeCode and EndTimeCode */
	USDUTILITIES_API void AddTimeCodeRangeToLayer(const pxr::SdfLayerRefPtr& Layer, double StartTimeCode, double EndTimeCode);

	/** Makes Path relative to the file path of Layer. Conversion happens in-place. */
	USDUTILITIES_API void MakePathRelativeToLayer(const UE::FSdfLayer& Layer, FString& Path);

	/** Loads and returns the session sublayer that is used for storing persistent UE state, which can be saved to disk (e.g. metadata for whether an
	 * attribute is muted or not) */
	USDUTILITIES_API UE::FSdfLayer GetUEPersistentStateSublayer(const UE::FUsdStage& Stage, bool bCreateIfNeeded = true);

	/** Loads and returns the anonymous session sublayer that is used for storing transient UE session state, and won't be saved to disk (e.g. the
	 * opinion that actually mutes the attribute) */
	USDUTILITIES_API UE::FSdfLayer GetUESessionStateSublayer(const UE::FUsdStage& Stage, bool bCreateIfNeeded = true);

	/** Uses FindOrOpen to return the layer with the given identifier if possible. If the identifier is for an anonymous layer, it will search via
	 * display name instead */
	USDUTILITIES_API UE::FSdfLayer FindLayerForIdentifier(const TCHAR* Identifier, const UE::FUsdStage& Stage);

	/** Returns true if Layer is a session layer within Stage's layer stack */
	USDUTILITIES_API bool IsSessionLayerWithinStage(const pxr::SdfLayerRefPtr& Layer, const pxr::UsdStageRefPtr& Stage);

	/**
	 * Uses the USD SDK's pxr::UsdUtilsFlattenLayerStack to essentially flatten all the sublayer composition into a single layer, preserving other
	 * composition arcs like references and payloads
	 */
	USDUTILITIES_API UE::FSdfLayer FlattenLayerStack(const pxr::UsdStageRefPtr& Stage);

	/**
	 * Finds all fields that have asset paths as values (e.g. texture references) in LayerToConvert and (if they're relative paths)
	 * updates them to be absolute with respect to AnchorLayer. LayerToConvert and AnchorLayer can be the same layer, but they don't have to
	 */
	USDUTILITIES_API void ConvertAssetRelativePathsToAbsolute(UE::FSdfLayer& LayerToConvert, const UE::FSdfLayer& AnchorLayer);

	USDUTILITIES_API int32 GetSdfLayerNumFrames(const pxr::SdfLayerRefPtr& Layer);
}	 // namespace UsdUtils

#endif	  // #if USE_USD_SDK
