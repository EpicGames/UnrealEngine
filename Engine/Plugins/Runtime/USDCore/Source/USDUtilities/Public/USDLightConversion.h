// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "CoreMinimal.h"

#include "USDConversionUtils.h"
#include "UsdWrappers/ForwardDeclarations.h"

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class UsdLuxDiskLight;
	class UsdLuxDistantLight;
	class UsdLuxDomeLight;
	class UsdLuxLightAPI;
	class UsdLuxRectLight;
	class UsdLuxShapingAPI;
	class UsdLuxSphereLight;

	class UsdAttribute;
	class UsdPrim;
	class UsdStage;
	template<typename T>
	class TfRefPtr;

	using UsdStageRefPtr = TfRefPtr<UsdStage>;
PXR_NAMESPACE_CLOSE_SCOPE

class UUsdAssetCache2;
class UUsdAssetCache3;
class UDirectionalLightComponent;
class ULightComponentBase;
class UPointLightComponent;
class URectLightComponent;
class USkyLightComponent;
class USpotLightComponent;
class UUsdAssetCache;
enum class ELightUnits : uint8;
struct FUsdStageInfo;

/**
 * Converts UsdLux light attributes to the corresponding ULightComponent.
 *
 * Corresponding UsdLux light schema to Unreal component:
 *
 *	UsdLuxLightAPI		->	ULightComponent
 *	UsdLuxDistantLight	->	UDirectionalLightComponent
 *	UsdLuxRectLight		->	URectLightComponent
 *	UsdLuxDiskLight		->	URectLightComponent
 *	UsdLuxSphereLight	->	UPointLightComponent
 *	UsdLuxDomeLight 	->	USkyLightComponent
 *	UsdLuxShapingAPI 	->	USpotLightComponent
 */
namespace UsdToUnreal
{
	USDUTILITIES_API bool ConvertLight(
		const pxr::UsdPrim& Prim,
		ULightComponentBase& LightComponent,
		double UsdTimeCode = UsdUtils::GetDefaultTimeCode()
	);
	USDUTILITIES_API bool ConvertDistantLight(
		const pxr::UsdPrim& Prim,
		UDirectionalLightComponent& LightComponent,
		double UsdTimeCode = UsdUtils::GetDefaultTimeCode()
	);
	USDUTILITIES_API bool ConvertRectLight(
		const pxr::UsdPrim& Prim,
		URectLightComponent& LightComponent,
		double UsdTimeCode = UsdUtils::GetDefaultTimeCode()
	);
	USDUTILITIES_API bool ConvertDiskLight(
		const pxr::UsdPrim& Prim,
		URectLightComponent& LightComponent,
		double UsdTimeCode = UsdUtils::GetDefaultTimeCode()
	);
	USDUTILITIES_API bool ConvertSphereLight(
		const pxr::UsdPrim& Prim,
		UPointLightComponent& LightComponent,
		double UsdTimeCode = UsdUtils::GetDefaultTimeCode()
	);
	USDUTILITIES_API bool ConvertLuxShapingAPI(
		const pxr::UsdPrim& Prim,
		USpotLightComponent& LightComponent,
		double UsdTimeCode = UsdUtils::GetDefaultTimeCode()
	);

	/** Applies the USD exposure, essentially just returning UsdIntensity * 2.0^(UsdExposure) */
	USDUTILITIES_API float ConvertLightIntensity(float UsdIntensity, float UsdExposure);

	/**
	 * Receives InConeAngle (degrees) and InConeSoftness ([0,1] range) USD spot light property values, and produces
	 * the corresponding Unreal outer and inner cone angles (also degrees) for a spot light component.
	 *
	 * Softness 0 corresponds to a hard transition where the outer and inner angle the same: Max brightness from center until the cone angle, and zero after it
	 * Softness 0.5 corresponds to an inner angle with half the size of the outer angle: Max brightness until the inner angle, and it lerps to zero until the outer angle
	 * Softness 1 corresponds to zero inner angle, so the brightness lerps from max at the center all the way to zero on the outer cone angle
	 */
	USDUTILITIES_API void ConvertSpotLightConeAngles(float InConeAngle, float InConeSoftness, float& OutOuterConeAngle, float& OutInnerConeAngle);

	UE_DEPRECATED(5.8, "This has been renamed to ConvertLightIntensity.")
	USDUTILITIES_API float ConvertLightIntensityAttr(float UsdIntensity, float UsdExposure);
	UE_DEPRECATED(5.8, "No longer used as this implicitly hard-coded the intensity units. Compute the solid angle and area with the other utils functions and use ConvertIntensityFromNits.")
	USDUTILITIES_API float ConvertDistantLightIntensityAttr(float UsdIntensity, float UsdExposure);
	UE_DEPRECATED(5.8, "No longer used as this implicitly hard-coded the intensity units. Compute the solid angle and area with the other utils functions and use ConvertIntensityFromNits.")
	USDUTILITIES_API float ConvertRectLightIntensityAttr(float UsdIntensity, float UsdExposure, float UsdWidth, float UsdHeight, const FUsdStageInfo& StageInfo);
	UE_DEPRECATED(5.8, "No longer used as this implicitly hard-coded the intensity units. Compute the solid angle and area with the other utils functions and use ConvertIntensityFromNits.")
	USDUTILITIES_API float ConvertDiskLightIntensityAttr(float UsdIntensity, float UsdExposure, float UsdRadius, const FUsdStageInfo& StageInfo);
	UE_DEPRECATED(5.8, "No longer used as this implicitly hard-coded the intensity units. Compute the solid angle and area with the other utils functions and use ConvertIntensityFromNits.")
	USDUTILITIES_API float ConvertSphereLightIntensityAttr(float UsdIntensity, float UsdExposure, float UsdRadius, const FUsdStageInfo& StageInfo);
	UE_DEPRECATED(5.8, "No longer used as this implicitly hard-coded the intensity units. Compute the solid angle and area with the other utils functions and use ConvertIntensityFromNits.")
	USDUTILITIES_API float ConvertLuxShapingAPIIntensityAttr(float UsdIntensity, float UsdExposure, float UsdRadius, float UsdConeAngle, float UsdConeSoftness, const FUsdStageInfo& StageInfo);
	UE_DEPRECATED(5.8, "Use ConvertSpotLightConeAngles().")
	USDUTILITIES_API float ConvertConeAngleSoftnessAttr(float UsdConeAngle, float UsdConeSoftness, float& OutInnerConeAngle);
}

namespace UnrealToUsd
{
	USDUTILITIES_API bool ConvertLightComponent(
		const ULightComponentBase& LightComponent,
		pxr::UsdPrim& Prim,
		double UsdTimeCode = UsdUtils::GetDefaultTimeCode()
	);
	USDUTILITIES_API bool ConvertDirectionalLightComponent(
		const UDirectionalLightComponent& LightComponent,
		pxr::UsdPrim& Prim,
		double UsdTimeCode = UsdUtils::GetDefaultTimeCode()
	);
	USDUTILITIES_API bool ConvertRectLightComponent(
		const URectLightComponent& LightComponent,
		pxr::UsdPrim& Prim,
		double UsdTimeCode = UsdUtils::GetDefaultTimeCode()
	);
	USDUTILITIES_API bool ConvertPointLightComponent(
		const UPointLightComponent& LightComponent,
		pxr::UsdPrim& Prim,
		double UsdTimeCode = UsdUtils::GetDefaultTimeCode()
	);
	USDUTILITIES_API bool ConvertSkyLightComponent(
		const USkyLightComponent& LightComponent,
		pxr::UsdPrim& Prim,
		double UsdTimeCode = UsdUtils::GetDefaultTimeCode()
	);
	USDUTILITIES_API bool ConvertSpotLightComponent(
		const USpotLightComponent& LightComponent,
		pxr::UsdPrim& Prim,
		double UsdTimeCode = UsdUtils::GetDefaultTimeCode()
	);

	/**
	 * Produces the USD cone angle (degrees) and softness ([0,1] range) values for a spot light that corresponds to an unreal outer and inner cone angles (also degrees).
	 * This is the inverse of UsdToUnreal::ConvertSpotLightConeAngles().
	 */
	USDUTILITIES_API void ConvertSpotLightConeAngles(float InOuterConeAngle, float InInnerConeAngle, float& OutConeAngle, float& OutConeSoftness);

	UE_DEPRECATED(5.8, "No longer used as it didn't really do anything.")
	USDUTILITIES_API float ConvertLightIntensityProperty(float Intensity);
	UE_DEPRECATED(5.8, "Compute the solid angle and area with the other utils functions and use ConvertIntensityToNits().")
	USDUTILITIES_API float ConvertRectLightIntensityProperty(float Intensity, float Width, float Height, const FUsdStageInfo& StageInfo, ELightUnits SourceUnits);
	UE_DEPRECATED(5.8, "Compute the solid angle and area with the other utils functions and use ConvertIntensityToNits().")
	USDUTILITIES_API float ConvertRectLightIntensityProperty(float Intensity, float Radius, const FUsdStageInfo& StageInfo, ELightUnits SourceUnits);
	UE_DEPRECATED(5.8, "Compute the solid angle and area with the other utils functions and use ConvertIntensityToNits().")
	USDUTILITIES_API float ConvertPointLightIntensityProperty(float Intensity, float SourceRadius, const FUsdStageInfo& StageInfo, ELightUnits SourceUnits);
	UE_DEPRECATED(5.8, "Compute the solid angle and area with the other utils functions and use ConvertIntensityToNits().")
	USDUTILITIES_API float ConvertSpotLightIntensityProperty(float Intensity, float OuterConeAngle, float InnerConeAngle, float SourceRadius, const FUsdStageInfo& StageInfo, ELightUnits SourceUnits);
	UE_DEPRECATED(5.8, "Use ConvertSpotLightConeAngles().")
	USDUTILITIES_API float ConvertOuterConeAngleProperty(float OuterConeAngle);
	UE_DEPRECATED(5.8, "Use ConvertSpotLightConeAngles().")
	USDUTILITIES_API float ConvertInnerConeAngleProperty(float InnerConeAngle, float OuterConeAngle);
}

namespace UsdUtils
{
	/**
	 * Returns the side length of a square that possesses the same area as a circle with DiskLightRadius.
	 * Output units are the same as the input units.
	 */
	 USDUTILITIES_API float GetRectLightSideFromDiskLight(float DiskLightRadius);

	/**
	 * Returns the radius of a circle that posseses the same area as a rectangle with sides RectLightWidth and RectLightHeight 
	 * Output units are the same as the input units.
	 */
	USDUTILITIES_API float GetDiskLightRadiusFromRectLight(float RectLightWidth, float RectLightHeight);

	/**
	 * Returns the values that should be used as "surface area" for unreal lights of different types.
	 * Output units are the square of the input units.
	 */
	USDUTILITIES_API float GetRectLightArea(float Width, float Height);
	USDUTILITIES_API float GetDiskLightArea(float Radius);
	USDUTILITIES_API float GetPointLightArea(float Radius);
	USDUTILITIES_API float GetSpotLightArea(float Radius);

	/** Returns the values that should be used as "solid angle" for unreal lights of different types, in steradians. */
	USDUTILITIES_API float GetRectLightSolidAngle();
	USDUTILITIES_API float GetDiskLightSolidAngle();
	USDUTILITIES_API float GetPointLightSolidAngle();
	USDUTILITIES_API float GetSpotLightSolidAngle(float OuterConeAngle, float InnerConeAngle);

	/** For a light with the provided SolidAngle and surface AreaInSqMeters, converts the Intensity value in the provided SourceUnits into Intensity in Nits */
	USDUTILITIES_API float ConvertIntensityToNits(float Intensity, float SolidAngle, float AreaInSqMeters, ELightUnits SourceUnits);

	/** For a light with the provided SolidAngle and surface AreaInSqMeters, converts the Intensity value in Nits into the provided DestUnits */
	USDUTILITIES_API float ConvertIntensityFromNits(float Intensity, float SolidAngle, float AreaInSqMeters, ELightUnits DestUnits);

	/**
	 * Reads the value of Attribute at the given time code. If the attribute has no authored opinion
	 * and its name starts with "inputs:", strips the prefix and tries the unprefixed attribute.
	 * This handles backwards compatibility with pre-21.05 USD scenes.
	 *
	 * If bOutIsAuthored is non-null, sets it to true if the value came from an authored opinion
	 * (either on the original attribute or the fallback), false if it's only a schema default.
	 */
	template<typename T>
	USDUTILITIES_API T GetLightAttrValueWithInputsFallback(
		const pxr::UsdAttribute& Attribute,
		double UsdTimeCode = UsdUtils::GetDefaultTimeCode(),
		bool* bOutIsAuthored = nullptr
	);
}

#endif	  // #if USE_USD_SDK
