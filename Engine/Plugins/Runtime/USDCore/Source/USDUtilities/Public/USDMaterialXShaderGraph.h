// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"

#if USE_USD_SDK && ENABLE_USD_MATERIALX
#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "pxr/base/gf/vec2f.h"
#include "pxr/base/gf/vec3f.h"
#include "pxr/base/gf/vec4f.h"
#include "USDIncludesEnd.h"

#include "MaterialXFormat/Util.h"

#include <unordered_map>
#include <unordered_set>

PXR_NAMESPACE_OPEN_SCOPE
	class UsdPrim;
	struct UsdShadeConnectionSourceInfo;
	class UsdShadeInput;
	class UsdShadeMaterial;
	class UsdShadeNodeGraph;
	class UsdShadeShader;
PXR_NAMESPACE_CLOSE_SCOPE

#endif // USE_USD_SDK && ENABLE_USD_MATERIALX

/**
 * Create a MaterialX shader graph from a UsdPrim, all <geompropvalue> nodes are either converted to an <image> or a proper <geomcolor>
 * Just get the Document in order to search for the surface material, by just looking for the name of the Prim it will return the correct surface material.
 */
class FUsdMaterialXShaderGraph
{
public:

	struct FGeomProp
	{
		FString Name;
		bool bTangentSpace = false;
	};

#if USE_USD_SDK && ENABLE_USD_MATERIALX

	/** 
	 * @param UsdShadeMaterialPrim - The prim used to generate the shader graph, should be a valid pxr::UsdShadeMaterial
	 * @param RenderContext - Which render context output to read from the UsdShadeMaterial
	 */
	USDUTILITIES_API FUsdMaterialXShaderGraph(const pxr::UsdPrim& MaterialPrim, const TCHAR* RenderContext);

	USDUTILITIES_API MaterialX::DocumentPtr GetDocument() const;

	/** Return the names of the <geompropvalue> nodes which have converted to <image> node*/
	USDUTILITIES_API const TArray<FGeomProp>& GetGeomPropValueNames() const;

private:

	/** Create either a <surfacematerial> or a <volumematerial> if possible. */
	void CreateMaterial(const pxr::UsdShadeMaterial& UsdShadeMaterial, const TCHAR* RenderContext);

	bool ComputeShaderGraph(MaterialX::NodePtr Node, const pxr::UsdShadeShader& Shader, MaterialX::GraphElementPtr Graph);

	bool IsGeomColor(const pxr::UsdShadeShader& GeomPropValueShader);

	void SetInputValue(MaterialX::InputPtr Input, const pxr::UsdShadeInput& UsdInput);

	/** Set the interfaceName of the input if any */
	bool SetInterfaceName(MaterialX::InputPtr Input, const pxr::UsdShadeConnectionSourceInfo& Source, MaterialX::GraphElementPtr Graph);

	/**
	 * Get the MaterialX node from a UsdShadeConnectionSourceInfo
	 * @param Source - upstream source in the Shader Graph, usually the connected node to an input
	 * @param Graph - either the Document or a nodegraph
	 * @param bIsNodeAlreadyExist - return true if the node was already present in the shader graph, false otherwise
	 * @return The corresponding MaterialX node in the shader graph, if it doesn't exist it creates one
	 */
	MaterialX::NodePtr GetNode(const pxr::UsdShadeConnectionSourceInfo& Source, MaterialX::GraphElementPtr Graph, bool& bIsNodeAlreadyExist);

	MaterialX::InputPtr GetInput(MaterialX::NodePtr Node, const pxr::UsdShadeInput& UsdInput);

	MaterialX::NodeDefPtr GetNodeDef(const pxr::UsdShadeConnectionSourceInfo& Source, bool& bIsGeomPropValue);

	MaterialX::NodeGraphPtr GetNodeGraph(const pxr::UsdShadeConnectionSourceInfo& Source, MaterialX::GraphElementPtr Graph);
	/**
	 * Given an input/output type, retrieve the MaterialX associated type
	 * @param Name - The name of the input/output
	 * @param InOutType - The type of the input/output, if it's valid it will now be the MaterialX type
	 * @return true if there is a valid match between USD and MaterialX or if it's already a MaterialX type, false otherwise
	 */
	bool GetMatchingType(const std::string& Name, std::string& InOutType);

	bool IsTangentSpaceInput(const pxr::UsdShadeInput& UsdInput) const;

	void GetInterfaceNameInputs(const pxr::UsdShadeNodeGraph& UsdShadeGraph, MaterialX::GraphElementPtr Graph);

private:

	std::unordered_set<std::string> MaterialXTypes;

	std::unordered_map<std::string, std::string> UsdToMaterialXTypes;

	// Names of the inputs from a <surfaceshader> that requires to be in tangent space for the baking of the primvars later on
	std::unordered_set<std::string> TangentSpaceInputs;

	// MaterialX interfacenames relatives to their nodegraph (the Document if no nodegraph), some inputs use them instead of a value or a direct connection to a node
	std::unordered_map<MaterialX::GraphElementPtr, std::unordered_set<std::string>> InterfaceNames;

	MaterialX::DocumentPtr Document;

	TArray<FGeomProp> GeomPropValueNames;

	bool bTangentSpaceInput = false;

	bool bConvertGeomPropValueToSparseVolume = false;

private:

	// complex template to just retrieve the right type, basically a mapping between Usd and Mtlx for Vector types, otherwise we just
	// take the type (float, int, string)
	template<typename MtlxType>
	using FPairingUsdMtlxType_t =
		std::conditional_t<
		   std::is_same_v<MtlxType, MaterialX::Color4> || std::is_same_v<MtlxType, MaterialX::Vector4>,
		   pxr::GfVec4f,
		      std::conditional_t<
		         std::is_same_v<MtlxType, MaterialX::Color3> || std::is_same_v<MtlxType, MaterialX::Vector3>,
		         pxr::GfVec3f,
		            std::conditional_t<
		               std::is_same_v<MtlxType, MaterialX::Vector2>,
		               pxr::GfVec2f,
		   MtlxType>>>;
#endif // USE_USD_SDK && ENABLE_USD_MATERIALX
};
