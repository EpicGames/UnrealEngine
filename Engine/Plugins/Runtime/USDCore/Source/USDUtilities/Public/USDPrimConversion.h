// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "CoreMinimal.h"

#include "USDConversionUtils.h"

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class UsdAttribute;
	class UsdGeomBBoxCache;
	class UsdGeomCamera;
	class UsdGeomXformable;
	class UsdPrim;
	class UsdTimeCode;
	class UsdTyped;

	class UsdStage;
	template<typename T>
	class TfRefPtr;

	using UsdStageRefPtr = TfRefPtr<UsdStage>;
PXR_NAMESPACE_CLOSE_SCOPE

class AInstancedFoliageActor;
class UAudioComponent;
class UCineCameraComponent;
class UInstancedStaticMeshComponent;
class UHierarchicalInstancedStaticMeshComponent;
class ULevel;
class UMeshComponent;
class UMovieScene;
class UMovieScene3DTransformTrack;
class UMovieSceneAudioSection;
class UMovieSceneBoolTrack;
class UMovieSceneColorTrack;
class UMovieSceneDoubleVectorTrack;
class UMovieSceneFloatTrack;
class UMovieScenePropertyTrack;
class UMovieSceneSkeletalAnimationTrack;
class UMovieSceneTrack;
class UMovieSceneVisibilityTrack;
class USceneComponent;
class USkeletalMeshComponent;
class UUsdDrawModeComponent;
enum ERichCurveInterpMode : int;
struct FFrameRate;
struct FMovieSceneFloatChannel;
struct FMovieSceneSequenceTransform;
struct FUsdCombinedPrimMetadata;
struct FUsdPrimMetadata;
struct FUsdStageInfo;

namespace UE
{
	class FUsdAttribute;
	class FUsdGeomBBoxCache;
}

namespace UsdToUnreal
{
	/**
	 * Converts a pxr::UsdGeomXformable's attribute values into an USceneComponent's property values
	 * @param Stage - Stage that contains the prim to convert
	 * @param Schema - Prim to convert
	 * @param SceneComponent - Output component to receieve the converted data
	 * @param EvalTime - Time at which to sample the prim's attributes
	 * @param bUsePrimTransform - Whether to convert transform data or not
	 * @return Whether the conversion was successful or not.
	 */
	USDUTILITIES_API bool ConvertXformable(
		const pxr::UsdStageRefPtr& Stage,
		const pxr::UsdTyped& Schema,
		USceneComponent& SceneComponent,
		double EvalTime,
		bool bUsePrimTransform = true
	);

	/**
	 * Converts a pxr::UsdGeomXformable's transform attribute values into a UE-space FTransform
	 * @param Stage - Stage that contains the prim to convert
	 * @param Schema - Prim to convert
	 * @param OutTransform - Output transform to receieve the converted data
	 * @param EvalTime - Time at which to sample the prim's attributes
	 * @param bOutResetTransformStack - Output parameter that, when available, will be set to true if this prim uses the "!resetXformStack!" op
	 * @return Whether the conversion was successful or not.
	 */
	USDUTILITIES_API bool ConvertXformable(
		const pxr::UsdStageRefPtr& Stage,
		const pxr::UsdTyped& Schema,
		FTransform& OutTransform,
		double EvalTime,
		bool* bOutResetTransformStack = nullptr
	);

	/**
	 * Propagates the transform from Root to Leaf prims inclusively into a UE-space FTransform if they are connected by Xformables
	 * @param Stage - Stage that contains the prims to process
	 * @param Root - Prim to start the propagation from
	 * @param Leaf - Prim where to end the propagation
	 * @param EvalTime - Time at which to sample the prim's attributes
	 * @param OutTransform - The composed transform; identity if Leaf is not completely connected to Root by Xformables
	 */
	USDUTILITIES_API void PropagateTransform(
		const pxr::UsdStageRefPtr& Stage,
		const pxr::UsdPrim& Root,
		const pxr::UsdPrim& Leaf,
		double EvalTime,
		FTransform& OutTransform
	);

	USDUTILITIES_API bool ConvertGeomCamera(
		const UE::FUsdPrim& Prim,
		UCineCameraComponent& CameraComponent,
		double UsdTimeCode = UsdUtils::GetDefaultTimeCode()
	);

	/**
	 * Calls the ReaderFunc on each time sample of UsdTimeSamples in order to bake values into MovieSceneTrack.
	 * This is mostly used when reading attributes from USD into tracks for the automatically generated ULevelSequence provided with AUsdStageActors.
	 * The bounds version must use a BBoxCache: If one is not provided it will be created on-demand for this call alone.
	 */
	USDUTILITIES_API bool ConvertBoolTimeSamples(
		const UE::FUsdStage& UsdStage,
		const TArray<double>& UsdTimeSamples,
		const TFunction<bool(double)>& ReaderFunc,
		UMovieSceneBoolTrack& MovieSceneTrack,
		const FMovieSceneSequenceTransform& SequenceTransform
	);
	USDUTILITIES_API bool ConvertBoolTimeSamples(
		const UE::FUsdStage& UsdStage,
		const TArray<double>& UsdTimeSamples,
		const TFunction<bool(double)>& ReaderFunc,
		UMovieSceneVisibilityTrack& MovieSceneTrack,
		const FMovieSceneSequenceTransform& SequenceTransform
	);
	USDUTILITIES_API bool ConvertFloatTimeSamples(
		const UE::FUsdStage& UsdStage,
		const TArray<double>& UsdTimeSamples,
		const TFunction<float(double)>& ReaderFunc,
		FMovieSceneFloatChannel& FloatChannel,
		const UMovieScene& MovieSceneOuter,
		const FMovieSceneSequenceTransform& SequenceTransform,
		TOptional<ERichCurveInterpMode> InterpolationModeOverride = {}
	);
	USDUTILITIES_API bool ConvertFloatTimeSamples(
		const UE::FUsdStage& UsdStage,
		const TArray<double>& UsdTimeSamples,
		const TFunction<float(double)>& ReaderFunc,
		UMovieSceneFloatTrack& MovieSceneTrack,
		const FMovieSceneSequenceTransform& SequenceTransform,
		TOptional<ERichCurveInterpMode> InterpolationModeOverride = {}
	);
	USDUTILITIES_API bool ConvertColorTimeSamples(
		const UE::FUsdStage& UsdStage,
		const TArray<double>& UsdTimeSamples,
		const TFunction<FLinearColor(double)>& ReaderFunc,
		UMovieSceneColorTrack& MovieSceneTrack,
		const FMovieSceneSequenceTransform& SequenceTransform
	);
	USDUTILITIES_API bool ConvertTransformTimeSamples(
		const UE::FUsdStage& UsdStage,
		const TArray<double>& UsdTimeSamples,
		const TFunction<FTransform(double)>& ReaderFunc,
		UMovieScene3DTransformTrack& MovieSceneTrack,
		const FMovieSceneSequenceTransform& SequenceTransform
	);
	USDUTILITIES_API bool ConvertBoundsTimeSamples(
		const UE::FUsdPrim& InPrim,
		const TArray<double>& InUsdTimeSamples,
		const FMovieSceneSequenceTransform& InSequenceTransform,
		UMovieSceneDoubleVectorTrack& InOutMinTrack,
		UMovieSceneDoubleVectorTrack& InOutMaxTrack,
		UE::FUsdGeomBBoxCache* InOutBBoxCache = nullptr
	);

	/**
	 * Struct with lambda functions that can be used to sample an FUsdPrim's attributes at a provided UsdTimeCode and return a converted result.
	 * The target prims, stage and the actual attributes are captured into the lambda when creating it with CreatePropertyTrackReader.
	 * This is used so that the reading process can be decoupled from the baking process (ConvertXTimeSamples functions) and reused for different
	 * attributes c.f. CreatePropertyTrackReader
	 */
	struct FPropertyTrackReader
	{
		TFunction<float(double)> FloatReader;
		TFunction<bool(double)> BoolReader;
		TFunction<FLinearColor(double)> ColorReader;
		TFunction<FTransform(double)> TransformReader;
	};

	/**
	 * Creates an FPropertyTrackReader that can be used to repeatedly read and convert values of attributes that correspond to PropertyPath in UE.
	 * e.g. calling this with a RectLight prim and UnrealIdentifiers::IntensityPropertyName will return a reader with a FloatReader member that checks
	 * the intensity, exposure, width, height of Prim at each UsdTimeCode and returns the corresponding float value for Intensity.
	 * When we generate a TransformReader, if bIgnorePrimLocalTransform is true it will cause it to ignore that prim's local transform for the reader.
	 */
	USDUTILITIES_API FPropertyTrackReader
	CreatePropertyTrackReader(const UE::FUsdPrim& Prim, const FName& PropertyPath, bool bIgnorePrimLocalTransform = false);

	/**
	 * Convert a prim with UsdGeomModelAPI and an alternative drawMode (bounds, cards or origin) into property values of an
	 * UUsdDrawModeComponent at EvalTime.
	 * Must use a BBoxCache: If one is not provided it will be created on-demand for this call alone.
	 */
	USDUTILITIES_API bool ConvertDrawMode(
		const pxr::UsdPrim& Prim,
		UUsdDrawModeComponent* DrawModeComponent,
		double EvalTime = UsdUtils::GetDefaultTimeCode(),
		pxr::UsdGeomBBoxCache* BBoxCache = nullptr
	);

	/**
	 * Collects all metadata from Prim (and its subtree in case bCollectFromEntireSubtrees is true), using the provided filters,
	 * and places them into PrimMetadata
	 */
	USDUTILITIES_API bool ConvertMetadata(
		const pxr::UsdPrim& Prim,
		FUsdCombinedPrimMetadata& PrimMetadata,
		const TArray<FString>& BlockedPrefixFilters = {},
		bool bInvertFilters = false,
		bool bCollectFromEntireSubtrees = false
	);

	/**
	 * Collects all metadata from Prim (and its subtree in case bCollectFromEntireSubtrees is true), using the provided filters,
	 * and places them into AssetUserData
	 */
	USDUTILITIES_API bool ConvertMetadata(
		const pxr::UsdPrim& Prim,
		UUsdAssetUserData* AssetUserData,
		const TArray<FString>& BlockedPrefixFilters = {},
		bool bInvertFilters = false,
		bool bCollectFromEntireSubtrees = false
	);
}	 // namespace UsdToUnreal

namespace UnrealToUsd
{
	USDUTILITIES_API bool ConvertCameraComponent(
		const UCineCameraComponent& CameraComponent,
		pxr::UsdPrim& Prim,
		double UsdTimeCode = UsdUtils::GetDefaultTimeCode()
	);

	USDUTILITIES_API bool ConvertAudioComponent(
		const UAudioComponent& AudioComponent,
		pxr::UsdPrim& Prim,
		bool bFilePathOnly,
		double UsdTimeCode = UsdUtils::GetDefaultTimeCode()
	);

	USDUTILITIES_API bool ConvertAudioSection(
		const UMovieSceneAudioSection& AudioSection,
		const FMovieSceneSequenceTransform& SequenceTransform,
		pxr::UsdPrim& Prim
	);

	/**
	 * Functions that call WriterFunc on each UsdTimeSample that corresponds to the keyframes of MovieSceneTrack, writing out time samples for
	 * attributes of Prim. Mostly used to write out to USD the modified tracks from the automatically generated ULevelSequence owned by
	 * AUsdStageActors
	 */
	USDUTILITIES_API bool ConvertFloatChannel(
		const FMovieSceneFloatChannel& MovieSceneChannel,
		const UMovieScene& MovieSceneOuter,
		const FMovieSceneSequenceTransform& SequenceTransform,
		const TFunction<void(float, double)>& WriterFunc,
		UE::FUsdPrim& Prim
	);
	USDUTILITIES_API bool ConvertFloatTrack(
		const UMovieSceneFloatTrack& MovieSceneTrack,
		const FMovieSceneSequenceTransform& SequenceTransform,
		const TFunction<void(float, double)>& WriterFunc,
		UE::FUsdPrim& Prim
	);
	USDUTILITIES_API bool ConvertBoolTrack(
		const UMovieScenePropertyTrack& MovieSceneTrack,	// PropertyTrack here so it handles BoolTracks and VisibilityTracks
		const FMovieSceneSequenceTransform& SequenceTransform,
		const TFunction<void(bool, double)>& WriterFunc,
		UE::FUsdPrim& Prim
	);
	USDUTILITIES_API bool ConvertColorTrack(
		const UMovieSceneColorTrack& MovieSceneTrack,
		const FMovieSceneSequenceTransform& SequenceTransform,
		const TFunction<void(const FLinearColor&, double)>& WriterFunc,
		UE::FUsdPrim& Prim
	);
	USDUTILITIES_API bool ConvertBoundsVectorTracks(
		const UMovieSceneDoubleVectorTrack* MinTrack,
		const UMovieSceneDoubleVectorTrack* MaxTrack,
		const FMovieSceneSequenceTransform& SequenceTransform,
		const TFunction<void(const FVector&, const FVector&, double)>& WriterFunc,
		UE::FUsdPrim& Prim
	);
	USDUTILITIES_API bool Convert3DTransformTrack(
		const UMovieScene3DTransformTrack& MovieSceneTrack,
		const FMovieSceneSequenceTransform& SequenceTransform,
		const TFunction<void(const FTransform&, double)>& WriterFunc,
		UE::FUsdPrim& Prim
	);

	USDUTILITIES_API bool ConvertSceneComponent(const pxr::UsdStageRefPtr& Stage, const USceneComponent* SceneComponent, pxr::UsdPrim& UsdPrim);

	USDUTILITIES_API bool ConvertMeshComponent(const pxr::UsdStageRefPtr& Stage, const UMeshComponent* MeshComponent, pxr::UsdPrim& UsdPrim);

	USDUTILITIES_API bool ConvertHierarchicalInstancedStaticMeshComponent(
		const UHierarchicalInstancedStaticMeshComponent* HISMComponent,
		pxr::UsdPrim& UsdPrim,
		double TimeCode = UsdUtils::GetDefaultTimeCode()
	);

	USDUTILITIES_API bool ConvertInstancedStaticMeshComponent(
		const UInstancedStaticMeshComponent* ISMComponent,
		pxr::UsdPrim& UsdPrim,
		double TimeCode = UsdUtils::GetDefaultTimeCode()
	);

	/**
	 * Sets the material overrides as opinions on MeshPrim, taking care to figure out whether it needs to create LOD or subsection prims, and
	 * which ones given the chosen export LOD range.
	 * Leaving the LOD range at default of INDEX_NONE will author the material override opinions assuming all LODs of the MeshAsset were exported.
	 * MeshAsset should be an UStaticMesh, USkeletalMesh or UGeometryCache.
	 *
	 * If bCheckAssetUserData is enabled, this function will check the asset's USD Asset User Data in order to determine the mapping between
	 * material slot and prim path, which is useful when exporting multiple slots to geom subsets or LOD prims. If this is false, The overrides
	 * will be authored to standard paths used by the USDImporter plugin (i.e. LOD<N> prims when needed, UsdGeomSubsets named "Section<N>")
	 */
	USDUTILITIES_API bool ConvertMaterialOverrides(
		const UObject* MeshAsset,
		const TArray<UMaterialInterface*> MaterialOverrides,
		pxr::UsdPrim& UsdPrim,
		int32 LowestLOD = INDEX_NONE,
		int32 HighestLOD = INDEX_NONE,
		bool bCheckAssetUserData = true
	);

	/**
	 * Converts a UMovieScene3DTransformTrack to a UsdGeomXformable
	 * @param MovieSceneTrack      The track to read the time sampled transform from
	 * @param Prim                 The Xformable to write to
	 * @param SequenceTransform    The time transform to apply to the track keys to get them from Usd Stage time to track time (in other words: from
	 * main sequence to subsequence)
	 */
	USDUTILITIES_API bool ConvertXformable(
		const UMovieScene3DTransformTrack& MovieSceneTrack,
		pxr::UsdPrim& UsdPrim,
		const FMovieSceneSequenceTransform& SequenceTransform
	);
	USDUTILITIES_API bool ConvertXformable(const FTransform& RelativeTransform, pxr::UsdPrim& UsdPrim, double TimeCode);

	/**
	 * Converts a AInstancedFoliageActor to a prim containing a pxr::UsdGeomPointInstancer schema. Each foliage type should correspond to a prototype.
	 * This function only converts the protoIndices, positions, orientations and scales attributes.
	 *
	 * @param Actor				   The actor to convert data from
	 * @param Prim                 The pxr::UsdGeomPointInstancer to write to
	 * @param TimeCode			   TimeCode to write the attribute values at. Use UsdUtils::GetDefaultTimeCode() for the Default value.
	 * @param InstancesLevel	   If this is not nullptr, only foliage instances placed on components that belong to this level will be exported
	 */
	USDUTILITIES_API bool ConvertInstancedFoliageActor(
		const AInstancedFoliageActor& Actor,
		pxr::UsdPrim& UsdPrim,
		double TimeCode,
		ULevel* InstancesLevel = nullptr
	);

	// We will defer to the common UnrealToUsd::ConvertXComponent functions when baking level sequences, which will already bake
	// all of the properties of a component out at the same time.
	// We use this enum to keep track of which type of baking was already being done for a component, so that if
	// e.g. we have a track for camera aperture and another for camera focal length, we end up just baking that camera
	// only once (as only once will be enough to handle all animated properties)
	enum class EBakingType : uint8
	{
		None = 0,
		Transform = 1,
		Visibility = 2,
		Camera = 4,
		Light = 8,
		Skeletal = 16,
		Bounds = 32,
	};
	ENUM_CLASS_FLAGS(EBakingType);
	const static inline int32 NumBakingTypes = 6;

	// Contains a lambda function responsible for baking a USceneComponent on the level into a prim on an FUsdStage
	struct FComponentBaker
	{
		EBakingType BakerType;
		FString ComponentPath;	  // Used for sorting, to ensure we evaluate parents and skeletal stuff before children and attached stuff
		TFunction<void(double UsdTimeCode)> BakerFunction;
	};

	/**
	 * Creates a property baker responsible for baking Component's current state onto Prim according to its component type,
	 * if PropertyPath describes a property that we map from UE to USD for that component type.
	 *
	 * Returns whether a baker was created or not.
	 *
	 * e.g. if we provide a UCineCameraComponent and PropertyPath == "FocusSettings.ManualFocusDistance" it will create a
	 * EBakingType::Camera baker (that bakes all of the mappable UE cine camera properties at once) and return true.
	 * If we provide PropertyPath == "FocusSettings.FocusSmoothingInterpSpeed" it will return false as that is not mapped to USD.
	 */
	USDUTILITIES_API bool CreateComponentPropertyBaker(
		UE::FUsdPrim& Prim,
		const USceneComponent& Component,
		const FString& PropertyPath,
		FComponentBaker& OutBaker
	);

	/**
	 * Creates a property baker responsible for baking the joint/blend shape state of Component as a SkelAnimation,
	 * returning true if a EBakingType::Skeletal baker was created
	 */
	USDUTILITIES_API bool CreateSkeletalAnimationBaker(
		UE::FUsdPrim& SkeletonPrim,
		UE::FUsdPrim& SkelAnimation,
		USkeletalMeshComponent& Component,
		FComponentBaker& OutBaker
	);

	/**
	 * Struct with lambda functions that can be used to convert an UE property's value into the corresponding USD attribute values at the received
	 * UsdTimeCode. The target prims, stage and the actual attributes are captured into the lambda when creating it with CreatePropertyTrackWriter.
	 * This is used so that the writing process can be decoupled from the baking process (ConvertXTrack functions) and reused for different attributes
	 * c.f. CreatePropertyTrackWriter
	 */
	struct FPropertyTrackWriter
	{
		TFunction<void(float, double)> FloatWriter;
		TFunction<void(bool, double)> BoolWriter;
		TFunction<void(const FLinearColor&, double)> ColorWriter;
		TFunction<void(const FVector&, const FVector&, double)> TwoVectorWriter;	// Just used for the bounds tracks, where we use a vector for min
																					// and another for max
		TFunction<void(const FTransform&, double)> TransformWriter;
	};

	/**
	 * Creates an FPropertyTrackWriter that can be used to receive, convert and output to USD the values of UE properties contained in Track, baking
	 * frame-by-frame when needed. e.g. calling this with a RectLight prim and UnrealIdentifiers::SourceWidthPropertyName will return a writer with a
	 * FloatWriter member that writes not only to the width, but also the intensity attributes of Prim at each UsdTimeCode. Also outputs
	 * OutPropertyPathsToRefresh: In the case above containing "Intensity", which lets us know to refresh and re-read from USD the Intensity track, if
	 * we have one
	 */
	USDUTILITIES_API FPropertyTrackWriter CreatePropertyTrackWriter(
		const USceneComponent& Component,
		const UMovieScenePropertyTrack& Track,
		UE::FUsdPrim& Prim,
		TSet<FName>& OutPropertyPathsToRefresh
	);

	/**
	 * Converts the properties values from UUsdDrawModeComponent into attribute values of a prim with UsdGeomModelAPI.
	 * Note that this will even apply the schema itself to UsdPrim if it doesn't already have it, and potentially author
	 * the 'kind' metadata on it and all of its ancestors (as UsdGeomModelAPI requires the prim to be a "model" to have function)
	 */
	USDUTILITIES_API bool ConvertDrawModeComponent(
		const UUsdDrawModeComponent& DrawModeComponent,
		pxr::UsdPrim& UsdPrim,
		bool bWriteExtents = false,
		double UsdTimeCode = UsdUtils::GetDefaultTimeCode()
	);

	/**
	 * Writes out all metadata from CombinedPrimMetadata onto Prim, as actual USD metadata fields. Only fields that pass the filters
	 * will be allowed.
	 * If CombinedPrimMetadata contains metadata from multiple prim paths, this overload has some heuristics in order to try and
	 * get all of the provided metadata written out to that single prim. This means some of these metadata entries may not end up on
	 * prim at the "top level", but may instead be placed within nested metadata maps that mirror the prim path hierarchy.
	 * If this behavior is not desired, feel free to use another overload of this function for more precise control.
	 */
	USDUTILITIES_API bool ConvertMetadata(
		const FUsdCombinedPrimMetadata& CombinedPrimMetadata,
		const pxr::UsdPrim& Prim,
		const TArray<FString>& BlockedPrefixFilters = {},
		bool bInvertFilter = false
	);

	/**
	 * Writes out all metadata from PrimMetadata onto Prim, as actual USD metadata fields. Only fields that pass the filters
	 * will be allowed.
	 */
	USDUTILITIES_API bool ConvertMetadata(
		const FUsdPrimMetadata& PrimMetadata,
		const pxr::UsdPrim& Prim,
		const TArray<FString>& BlockedPrefixFilters = {},
		bool bInvertFilter = false
	);

	/**
	 * Writes out all of the collected metadata from AssetUserData onto Prim, as actual USD metadata fields. Only fields that
	 * pass the filters will be allowed.
	 * This overload will essentially just loop over all entries of AssetUserData's StageIdentifierToMetadata property and write
	 * them all out using the overload of ConvertMetadata that receives FUsdCombinedPrimMetadata.
	 */
	USDUTILITIES_API bool ConvertMetadata(
		const UUsdAssetUserData* AssetUserData,
		const pxr::UsdPrim& Prim,
		const TArray<FString>& BlockedPrefixFilters = {},
		bool bInvertFilter = false
	);
}	 // namespace UnrealToUsd

namespace UsdUtils
{
	/**
	 * Returns the attributes that in USD correspond to a property in UE.
	 * The first one is the "main" attribute, when appropriate. e.g. for a RectLight prim and UnrealIdentifiers::IntensityPropertyName,
	 * we'll receive intensity, exposure, width and height, in that order.
	 */
	USDUTILITIES_API TArray<UE::FUsdAttribute> GetAttributesForProperty(const UE::FUsdPrim& Prim, const FName& PropertyPath);

	/**
	 * Returns the names of UE properties that need to be written to, when reading an USD attribute.
	 * Example: If AttrName is "inputs:radius" and Prim is a SphereLight, this will return [UnrealIdentifiers::SourceRadiusPropertyName];
	 * 			If AttrName is "inputs:radius" and Prim is a DiskLight, this will return
	 *          [UnrealIdentifiers::SourceWidthPropertyName, UnrealIdentifiers::SourceHeightPropertyName].
	 */
	USDUTILITIES_API TArray<FName> GetPropertiesForAttribute(const UE::FUsdPrim& Prim, const FString& AttrName);

	/**
	 * Checks if the given property requires frame-by-frame baking due to non-linear attribute
	 * interactions (e.g. light intensity = inputs:intensity * 2^inputs:exposure).
	 * If so, densifies InOutTimeSamples to include every integer time code between the first and last
	 * existing sample, ensuring the non-linear combination is properly captured as linear keyframes.
	 */
	USDUTILITIES_API void DensifyTimeSamplesIfNeeded(
		const UE::FUsdPrim& Prim,
		const FName& PropertyPath,
		TArray<double>& InOutTimeSamples
	);
}

#endif	  // #if USE_USD_SDK
