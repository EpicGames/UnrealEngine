// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "CoreMinimal.h"
#include "Engine/TextureDefines.h"
#include "Materials/MaterialInterface.h"
#include "Math/TransformCalculus2D.h"
#include "Misc/TVariant.h"
#include "UObject/Package.h"
#include "UObject/WeakObjectPtr.h"

#include "UsdWrappers/SdfLayer.h"

#include "USDIncludesStart.h"
#include "pxr/base/tf/token.h"
#include "pxr/usd/usd/timeCode.h"
#include "pxr/usd/usdShade/tokens.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class UsdAttribute;
	class UsdGeomMesh;
	class UsdPrim;
	class UsdShadeMaterial;
PXR_NAMESPACE_CLOSE_SCOPE

class FSHAHash;
class UMaterial;
class UMaterialOptions;
class UTexture;
class UUsdAssetCache2;
class UUsdAssetCache3;
enum class EFlattenMaterialProperties : uint8;
enum EMaterialProperty : int;
enum TextureAddress : int;
enum TextureCompressionSettings : int;
struct FFlattenMaterial;
struct FPropertyEntry;

namespace UE
{
	class FUsdPrim;
}

namespace UsdToUnreal
{
	struct FTextureParameterValue
	{
		UTexture* Texture = nullptr;	// Only used for the ConvertMaterial overloads that receive TexturesCache

		// Parameters of the texture asset itself
		FString TextureFilePath;
		TextureGroup Group = TEXTUREGROUP_World;
		TOptional<bool> bSRGB;
		bool bIsUDIM = false;
		TextureAddress AddressX = TA_Wrap;
		TextureAddress AddressY = TA_Wrap;

		// Parameters about the texture usage
		FString Primvar;
		int32 OutputIndex = 0;
		FVector2f UVTranslation;
		float UVRotation = 0.0f;
		FScale2f UVScale;

	public:
		// Returns whether the texture should be parsed as sRGB or not, given the actually
		// authored "bSRGB" member and the fallback opinion provided by the texture group
		USDUTILITIES_API bool GetSRGBValue() const;
	};

	struct FPrimvarReaderParameterValue
	{
		FString PrimvarName;
		FVector FallbackValue;
	};

	using FParameterValue = TVariant<float, FVector, FTextureParameterValue, FPrimvarReaderParameterValue, bool, int32>;

	struct FUsdPreviewSurfaceMaterialData
	{
		TMap<FString, FParameterValue> Parameters;

		/**
		 * Describes which UV set this material will target with each primvar e.g. {'firstPrimvar': 0, 'st': 1, 'st1': 2}.
		 *
		 * We store this here because deciding this assignment involves combining and sorting all the existing primvars
		 * that the texture parameters want to read, which we do when first calling ConvertMaterial().
		 *
		 * This will later be compared with the primvar to UV index mapping we generate when parsing mesh data. If they
		 * are compatible, we'll be able to use the material directly on that mesh. Otherwise we'll need to generate a
		 * new instance of this material that assigns different primvars to each UV index (check
		 * CreatePrimvarCompatibleVersionOfMaterial).
		 */
		TMap<FString, int32> PrimvarToUVIndex;
	};

	/**
	 * Extracts material data from UsdShadeMaterial and places the results in Material. Note that since this is used for UMaterialInstanceDynamics at
	 * runtime as well, it will not set base property overrides (e.g. BlendMode) or the parent material, and will just assume that the caller handles
	 * that. Note that in order to receive the primvar to UV index mapping calculated within this function, the provided Material should have an
	 * UUsdMaterialAssetImportData object as its AssetImportData.
	 * @param UsdShadeMaterial - Shade material with the data to convert
	 * @param OutMaterial - Output parameter that will be filled with the converted data
	 * @param TexturesCache - Cache to prevent importing a texture more than once
	 * @param RenderContext - Which render context output to read from the UsdShadeMaterial
	 * @param bShareAssetsForIdenticalPrims - Whether to reuse identical textures found in the TextureCache or to create dedicated textures for each material
	 * @return Whether the conversion was successful or not.
	 */
	USDUTILITIES_API bool ConvertMaterial(
		const pxr::UsdPrim& InUsdShadeMaterialPrim,
		FUsdPreviewSurfaceMaterialData& OutMaterial,
		const TCHAR* InRenderContext = nullptr
	);
	USDUTILITIES_API bool ConvertMaterial(
		const pxr::UsdShadeMaterial& UsdShadeMaterial,
		UMaterialInstance& OutMaterial,
		UUsdAssetCache3* TexturesCache = nullptr,
		const TCHAR* RenderContext = nullptr,
		bool bShareAssetsForIdenticalPrims = true
	);
	USDUTILITIES_API bool ConvertMaterial(
		const pxr::UsdShadeMaterial& UsdShadeMaterial,
		UMaterial& OutMaterial,
		UUsdAssetCache3* TexturesCache = nullptr,
		const TCHAR* RenderContext = nullptr,
		bool bShareAssetsForIdenticalPrims = true
	);

	/**
	 * Attemps to assign the values of the surface shader inputs to the MaterialInstance parameters by matching the inputs display names to the
	 * parameters names.
	 * @param UsdShadeMaterial - Shade material with the data to convert
	 * @param MaterialInstance - Material instance on which we will set the parameter values
	 * @param TexturesCache - Cache to prevent importing a texture more than once
	 * @param RenderContext - The USD render context to use when fetching the surface shader
	 * @param ReuseIdenticalAssets - Whether to reuse identical textures found in the TextureCache or to create dedicated textures for each material
	 * @return Whether the conversion was successful or not.
	 *
	 */
	USDUTILITIES_API bool ConvertShadeInputsToParameters(
		const pxr::UsdShadeMaterial& UsdShadeMaterial,
		UMaterialInstance& MaterialInstance,
		UUsdAssetCache3* TexturesCache,
		const TCHAR* RenderContext = nullptr,
		bool bShareAssetsForIdenticalPrims = true
	);
}	 // namespace UsdToUnreal

#if WITH_EDITOR
namespace UnrealToUsd
{
	/**
	 * Bakes InMaterial into textures and constants, and configures OutUsdShadeMaterial to use the baked data.
	 * @param InMaterial - Source material to bake
	 * @param InMaterialProperties - Material properties to bake from InMaterial
	 * @param InDefaultTextureSize - Size of the baked texture to use for any material property that does not have a custom size set
	 * @param InTexturesDir - Directory where the baked textures will be placed
	 * @param OutUsdShadeMaterialPrim - UsdPrim with the UsdShadeMaterial schema that will be configured to use the baked textures and constants
	 * @param bInDecayTexturesToSinglePixel - Whether to use a single value directly on the material instead of writing out textures with that
											  have a single uniform color value for all pixels
	 * @return Whether the conversion was successful or not.
	 */
	USDUTILITIES_API bool ConvertMaterialToBakedSurface(
		const UMaterialInterface& InMaterial,
		const TArray<FPropertyEntry>& InMaterialProperties,
		const FIntPoint& InDefaultTextureSize,
		const FDirectoryPath& InTexturesDir,
		pxr::UsdPrim& OutUsdShadeMaterialPrim,
		bool bInDecayTexturesToSinglePixel = true
	);

	/**
	 * Converts a flattened material's data into textures placed at InTexturesDir, and configures OutUsdShadeMaterial to use the baked textures.
	 * Note that to avoid a potentially useless copy, InMaterial's samples will be modified in place to have 255 alpha before being exported to
	 * textures.
	 *
	 * @param MaterialName - Name of the material, used as prefix on the exported texture filenames
	 * @param InMaterial - Source material to bake
	 * @param InMaterialProperties - Object used *exclusively* to provide floating point constant values if necessary
	 * @param InTexturesDir - Directory where the baked textures will be placed
	 * @param OutUsdShadeMaterialPrim - UsdPrim with the UsdShadeMaterial schema that will be configured to use the baked textures and constants
	 * @return Whether the conversion was successful or not.
	 */
	USDUTILITIES_API bool ConvertFlattenMaterial(
		const FString& InMaterialName,
		FFlattenMaterial& InMaterial,
		const TArray<FPropertyEntry>& InMaterialProperties,
		const FDirectoryPath& InTexturesDir,
		UE::FUsdPrim& OutUsdShadeMaterialPrim
	);
}
#endif	  // WITH_EDITOR

namespace UsdUtils
{
	// Writes UnrealMaterialPathName as a material binding for MeshOrGeomSubsetPrim, either by reusing an existing UsdShadeMaterial
	// binding if it already has an 'unreal' render context output and the expected structure, or by creating a new Material prim
	// that fulfills those requirements and binding to it (optionally with strongerThanDescendants strength in that case).
	//
	// Doesn't write to the 'unrealMaterial' attribute at all, as we intend on deprecating it in the future.
	USDUTILITIES_API void AuthorUnrealMaterialBinding(
		pxr::UsdPrim& MeshOrGeomSubsetPrim,
		const FString& UnrealMaterialPathName,
		bool bStrongerThanDescendants = false
	);

	/**
	 * Similar to AuthorUnrealMaterialBinding, but instead of authoring material bindings directly to TargetMeshOrGeomSubsetPrim,
	 * it will instead author collection-based material bindings on the CollectionPrim, that instead target TargetMeshOrGeomSubsetPrim.
	 *
	 * It will try reusing existing collections and UnrealMaterials, but otherwise it will author a new collection within CollectionPrim,
	 * and a new UnrealMaterial as a sibling of CollectionPrim, referring to UnrealMaterialPathName.
	 *
	 * This is useful if CollectionPrim is an instance root, and TargetMeshOrGeomSubsetPrim is an instance proxy, for example.
	 *
	 * WARNING: In order to get collection-based bindings to work, TargetMeshOrGeomSubsetPrim must be a descendant of CollectionPrim.
	 */
	USDUTILITIES_API void AuthorUnrealCollectionBasedMaterialBinding(
		const pxr::UsdPrim& CollectionPrim,
		const pxr::UsdPrim& TargetMeshOrGeomSubsetPrim,
		const FString& UnrealMaterialPathName
	);

	/** Returns a path to an UE asset (e.g. "/Game/Assets/Red.Red") if MaterialPrim has an 'unreal' render context surface output that points at one
	 */
	USDUTILITIES_API TOptional<FString> GetUnrealSurfaceOutput(const pxr::UsdPrim& MaterialPrim);

	/**
	 * Sets which UE material asset the 'unreal' render context surface output of MaterialPrim is pointing at (creating the surface output
	 * on-demand if needed)
	 *
	 * @param MaterialPrim - pxr::UsdPrim with the pxr::UsdShadeMaterial schema to update the 'unreal' surface output of
	 * @param UnrealMaterialPathName - Path to an UE UMaterialInterface asset (e.g. "/Game/Assets/Red.Red")
	 * @return Whether we successfully set the surface output or not
	 */
	USDUTILITIES_API bool SetUnrealSurfaceOutput(pxr::UsdPrim& MaterialPrim, const FString& UnrealMaterialPathName);

	// Returns whether MaterialPrim is an actual Material, and has a surface output authored for the provided render context
	USDUTILITIES_API bool HasSurfaceOutput(const pxr::UsdPrim& MaterialPrim, const FName& RenderContext);
	USDUTILITIES_API bool HasSurfaceOutput(const pxr::UsdPrim& MaterialPrim, const TArray<FString>& RenderContexts);
	
	// Returns whether MaterialPrim is an actual Material, and has a displacement output authored for the provided render context
	USDUTILITIES_API bool HasDisplacementOutput(const pxr::UsdPrim& MaterialPrim, const FName& RenderContext);
	USDUTILITIES_API bool HasDisplacementOutput(const pxr::UsdPrim& MaterialPrim, const TArray<FString>& RenderContexts);
	
	// Returns whether MaterialPrim is an actual Material, and has a volume output authored for the provided render context
	USDUTILITIES_API bool HasVolumeOutput(const pxr::UsdPrim& MaterialPrim, const FName& RenderContext);
	USDUTILITIES_API bool HasVolumeOutput(const pxr::UsdPrim& MaterialPrim, const TArray<FString>& RenderContexts);

	/**
	 * Returns whether the material needs to be rendered with the Translucent rendering mode.
	 * This function exists because we need this information *before* we pick the right parent for a material instance and properly convert it.
	 */
	USDUTILITIES_API bool IsMaterialTranslucent(const pxr::UsdShadeMaterial& UsdShadeMaterial);
	USDUTILITIES_API bool IsMaterialTranslucent(const UsdToUnreal::FUsdPreviewSurfaceMaterialData& ConvertedMaterial);

	USDUTILITIES_API FSHAHash HashShadeMaterial(	//
		const pxr::UsdShadeMaterial& UsdShadeMaterial,
		const pxr::TfToken& RenderContext = pxr::UsdShadeTokens->universalRenderContext
	);
	USDUTILITIES_API void HashShadeMaterial(	//
		const pxr::UsdShadeMaterial& UsdShadeMaterial,
		FSHA1& InOutHash,
		const pxr::TfToken& RenderContext = pxr::UsdShadeTokens->universalRenderContext
	);

	/** Returns the resolved path from an pxr::SdfAssetPath attribute. For UDIMs path, returns the path to the 1001 tile. */
	USDUTILITIES_API FString GetResolvedAssetPath(const pxr::UsdAttribute& AssetPathAttr, pxr::UsdTimeCode TimeCode = pxr::UsdTimeCode::Default());

	/**
	 * Computes and returns the hash string for the texture at the given path.
	 * Handles regular texture asset paths as well as asset paths identifying textures inside Usdz archives.
	 * Returns an empty string if the texture could not be hashed.
	 */
	USDUTILITIES_API FString GetTextureHash(
		const FString& ResolvedTexturePath,
		bool bSRGB,
		TextureCompressionSettings CompressionSettings,
		TextureAddress AddressX,
		TextureAddress AddressY
	);

	USDUTILITIES_API UTexture* CreateTexture(
		const FString& ResolvedTexturePath,
		FName SanitizedName,
		TextureGroup Group = TEXTUREGROUP_World,
		EObjectFlags Flags = RF_Public | RF_Standalone | RF_Transactional,
		UObject* Outer = GetTransientPackage(),
		bool bForceLinear = false,
		bool bIsUDIMPath = false
	);

	/** Checks if this texture needs virtual textures and emits a warning if it is disabled for the project */
	USDUTILITIES_API void NotifyIfVirtualTexturesNeeded(UTexture* Texture);

#if WITH_EDITOR
	/** Convert between the two different types used to represent material channels to bake */
	USDUTILITIES_API EFlattenMaterialProperties MaterialPropertyToFlattenProperty(EMaterialProperty MaterialProperty);

	/** Convert between the two different types used to represent material channels to bake */
	USDUTILITIES_API EMaterialProperty FlattenPropertyToMaterialProperty(EFlattenMaterialProperties FlattenProperty);
#endif	  // WITH_EDITOR

	/** Converts channels that have the same value for every pixel into a channel that only has a single pixel with that value */
	USDUTILITIES_API void CollapseConstantChannelsToSinglePixel(FFlattenMaterial& InMaterial);

	/** Temporary function until UnrealWrappers can create attributes, just adds a custom bool attribute 'worldSpaceNormals' as true */
	USDUTILITIES_API bool MarkMaterialPrimWithWorldSpaceNormals(const UE::FUsdPrim& MaterialPrim);

	/** Sets material instance parameters whether Material is a MaterialInstanceConstant or a MaterialInstanceDynamic */
	USDUTILITIES_API void SetScalarParameterValue(UMaterialInstance& Material, const TCHAR* ParameterName, float ParameterValue);
	USDUTILITIES_API void SetVectorParameterValue(UMaterialInstance& Material, const TCHAR* ParameterName, FLinearColor ParameterValue);
	USDUTILITIES_API void SetTextureParameterValue(UMaterialInstance& Material, const TCHAR* ParameterName, UTexture* ParameterValue);
	USDUTILITIES_API void SetBoolParameterValue(UMaterialInstance& Material, const TCHAR* ParameterName, bool bParameterValue);

#if WITH_EDITOR
	/** Retrieve MaterialX file from a prim*/
	USDUTILITIES_API TArray<FString> GetMaterialXFilePaths(const pxr::UsdPrim& Prim);
#endif // WITH_EDITOR
}	 // namespace UsdUtils

#endif	  // #if USE_USD_SDK
