// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/MorphTarget.h"
#include "Animation/Skeleton.h"
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

#include "USDConversionUtils.h"
#include "USDGeomMeshConversion.h"
#include "UsdWrappers/ForwardDeclarations.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "pxr/usd/usd/timeCode.h"
#include "pxr/usd/usdShade/tokens.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class UsdGeomMesh;
	class UsdSkelAnimQuery;
	class UsdSkelBinding;
	class UsdSkelBlendShape;
	class UsdSkelCache;
	class UsdSkelRoot;
	class UsdSkelSkeleton;
	class UsdSkelSkeletonQuery;
	class UsdSkelSkinningQuery;
	template<typename T>
	class VtArray;
PXR_NAMESPACE_CLOSE_SCOPE

class FSkeletalMeshImportData;
class IMovieScenePlayer;
class UAnimSequence;
class UMovieScene;
class UMovieSceneControlRigParameterSection;
class USkeletalMesh;
class USkeletalMeshComponent;
struct FUsdStageInfo;
struct FMovieSceneInverseSequenceTransform;

namespace SkeletalMeshImportData
{
	struct FBone;
	struct FMaterial;
}

namespace UsdUtils
{
	struct FUsdPrimMaterialSlot;
	struct FUsdPrimMaterialAssignmentInfo;
}

namespace UE
{
	class FUsdSkelSkinningQuery;
}

#endif	  // #if USE_USD_SDK

namespace UsdUtils
{
	struct FUsdBlendShapeInbetween
	{
		// Name of the FUsdBlendShape/UMorphTarget that holds the morph data for this inbetween
		FString Name;

		float InbetweenWeight;

		friend FArchive& operator<<(FArchive& Ar, FUsdBlendShapeInbetween& BlendShape)
		{
			Ar << BlendShape.Name;
			Ar << BlendShape.InbetweenWeight;
			return Ar;
		}
	};

	struct FUsdBlendShape
	{
		FString Name;

		// Because Meshes need to target BlendShapes with USD relationships, and because relationships can't
		// target things inside USD variants, we get that we can never have different data for different LODs within the
		// same blend shape, like UMorphTarget does. At most, we can be *used* by different USD LOD meshes, which this member tracks.
		TSet<int32> LODIndicesThatUseThis;
		TArray<FMorphTargetDelta> Vertices;
		TArray<FUsdBlendShapeInbetween> Inbetweens;
		bool bHasAuthoredTangents = false;

		bool IsValid() const
		{
			return Vertices.Num() > 0;
		}

		friend FArchive& operator<<(FArchive& Ar, FUsdBlendShape& BlendShape)
		{
			Ar << BlendShape.Name;
			Ar << BlendShape.Inbetweens;
			return Ar;
		}
	};

	/**
	 * Maps from a full blend shape path (e.g. '/Scene/Mesh/BlendShapeName') to the parsed FUsdBlendShape struct.
	 * We need this to be case sensitive because USD paths are, and even the sample HumanFemale Skel scene has
	 * paths that differ only by case (e.g. 'JawUD' and 'JAWUD' blend shapes)
	 */
	using FBlendShapeMap = TMap<FString, FUsdBlendShape, FDefaultSetAllocator, FCaseSensitiveStringMapFuncs<FUsdBlendShape>>;

	/**
	 * We decompose inbetween blend shapes on import into separate morph targets, alongside the primary blend shape, which also becomes a morph
	 * target. This function returns the morph target weights for the primary morph target, as well as all the morph targets for inbetween shapes of
	 * 'InBlendShape', given an initial input weight value for the USD blend shape. Calculations are done following the equations at
	 * https://graphics.pixar.com/usd/docs/api/_usd_skel__schemas.html#UsdSkel_BlendShape_Inbetweens
	 *
	 * Note: This assumes that the morph targets in InBlendShape are sorted by weight.
	 */
	USDUTILITIES_API void ResolveWeightsForBlendShape(
		const FUsdBlendShape& InBlendShape,
		float InWeight,
		float& OutPrimaryWeight,
		TArray<float>& OutInbetweenWeights
	);

	USDUTILITIES_API void GetBoneTransforms(USkeletalMeshComponent* Component, TArray<FTransform>& BoneTransforms);
	USDUTILITIES_API void RefreshSkeletalMeshComponent(USkeletalMeshComponent& Component);

#if USE_USD_SDK
	/** Allows creation of a skinning query from the underlying skinned mesh and skeleton. Adapted from the USD SDK implementation */
	USDUTILITIES_API UE::FUsdSkelSkinningQuery CreateSkinningQuery(
		const pxr::UsdPrim& SkinnedMeshPrim,
		const pxr::UsdSkelSkeletonQuery& SkeletonQuery
	);

	/**
	 * Sets prim AnimationSource as the animation source for prim Prim.
	 * Applies the SkelBindingAPI to Prim. See pxr::SkelBindingAPI::GetAnimationSourceRel.
	 */
	USDUTILITIES_API void BindAnimationSource(pxr::UsdPrim& Prim, const pxr::UsdPrim& AnimationSource);

	/**
	 * Returns the SkelAnimation prim that is resolved for the first skeletal binding of SkelRootPrim, if it is a SkelRoot.
	 */
	USDUTILITIES_API UE::FUsdPrim FindFirstAnimationSource(const UE::FUsdPrim& InSkelRootPrim);

	/**
	 * Returns the SkelAnimation prim that is bound as animation source for the provided skeleton
	 */
	USDUTILITIES_API UE::FUsdPrim FindAnimationSource(const pxr::UsdPrim& InSkelRootPrim, const pxr::UsdPrim& InSkeletonPrim);

	/**
	 * Convenience function that traverses through ancestors up from SomePrim and returns the first SkelRoot prim that it finds,
	 * returning an invalid prim in case it doesn't find anything
	 */
	USDUTILITIES_API UE::FUsdPrim GetClosestParentSkelRoot(const pxr::UsdPrim& SomePrim);

	/**
	 * Convenience function to use/populate a SkelCache on a SkelRoot and retrieve the SkelBinding/SkeletonQuery for a
	 * particular InSkeletonPrim. You can retrieve the AnimQuery from the SkeletonQuery as well.
	 * A temp SkelCache will be constructed if none is provided.
	 * Returns true whether we managed to retrieve everything.
	 */
	USDUTILITIES_API bool GetSkelQueries(
		const pxr::UsdSkelRoot& InSkelRootPrim,
		const pxr::UsdSkelSkeleton& InSkeletonPrim,
		pxr::UsdSkelBinding& OutSkelBinding,
		pxr::UsdSkelSkeletonQuery& OutSkeletonQuery,
		pxr::UsdSkelCache* InOutSkelCache = nullptr
	);

	/**
	 * Converts and applies the blend shape deltas from InBlendShapePrim onto the mesh described by InOutMeshDescription.
	 * Note that we expect InOutMeshDescription and InBlendShapePrim to at least have a matching number of vertices/points.
	 * @param InOutMeshDescription - The mesh data to deform with the blend shape
	 * @param InBlendShapePrim - The UsdSkelBlendShape prim to deform the mesh with
	 * @param GeomBindTransform - USD-space transform applied to the point/normal deltas before any other transformation
	 * @param AdditionalTransform - Unreal-space transform applied to the normal deltas after all other transformations
	 * @param Weight - Scalar that multiplies the deltas before they're applied to the mesh deformation:
	 *                 0.0f means no effect, 1.0f adds the deltas at full strength, etc.
	 * @param InInbetweenName - The name of the inbetween shape of InBlendShapePrim to apply instead, if any.
	 *                          If left empty (default) the main blend shape is applied instead.
	 * @return Whether the conversion was successful or not.
	 */
	USDUTILITIES_API bool ApplyBlendShape(
		FMeshDescription& InOutMeshDescription,
		const pxr::UsdPrim& InBlendShapePrim,
		const FTransform& AdditionalTransform,
		float Weight = 1.0f,		
		const FString InInbetweenName = {}
	);
	USDUTILITIES_API bool ApplyBlendShape(
		FMeshDescription& InOutMeshDescription,
		const pxr::UsdPrim& InBlendShapePrim,
		const FMatrix& GeomBindTransform,
		const FTransform& AdditionalTransform,
		float Weight = 1.0f,		
		const FString InInbetweenName = {}
	);
#endif	  // USE_USD_SDK
}	 // namespace UsdUtils

namespace UsdToUnreal
{
	struct FUsdSkeletonData
	{
		struct FBone
		{
			FString Name;

			FTransform LocalRestTransform = FTransform::Identity;
			FTransform LocalBindTransform = FTransform::Identity;

			TArray<int32> ChildIndices;
			int32 ParentIndex = INDEX_NONE;	   // Root bones remain with INDEX_NONE
		};

		TArray<FBone> Bones;
	};

#if USE_USD_SDK && WITH_EDITOR
	/**
	 * Extracts skeleton data from UsdSkeletonQuery and places the results in ConvertedData.
	 * @param UsdSkeletonQuery - SkeletonQuery with the data to convert
	 * @param ConvertedData - Output parameter that will be filled with the converted data
	 * @param bEnsureAtLeastOneBone - Whether to ensure the resulting ConvertedData contains at least one bone (adding one if needed)
	 * @param bEnsureSingleRootBone - Whether to ensure the resulting ConvertedData has a *single* root bone
	 *                                (adding a new one and reparenting the previous root bones to it, if needed)
	 * @return Whether the conversion was successful or not.
	 */
	USDUTILITIES_API bool ConvertSkeleton(
		const pxr::UsdSkelSkeletonQuery& UsdSkeletonQuery,
		FUsdSkeletonData& ConvertedData,
		bool bEnsureAtLeastOneBone = true,
		bool bEnsureSingleRootBone = true
	);
	/**
	 * Extracts skeleton data from UsdSkeletonQuery and places the results in SkelMeshImportData.
	 * @param UsdSkeletonQuery - SkeletonQuery with the data to convert
	 * @param SkelMeshImportData - Output parameter that will be filled with the converted data
	 * @return Whether the conversion was successful or not.
	 */
	USDUTILITIES_API bool ConvertSkeleton(const pxr::UsdSkelSkeletonQuery& UsdSkeletonQuery, FSkeletalMeshImportData& SkelMeshImportData);

	/**
	 * Converts an USD blend shape into zero, one or more FUsdBlendShapes, and places them in OutBlendShapes
	 * @param UsdBlendShape - Source USD object with the blend shape data
	 * @param StageInfo - Details about the stage, required to do the coordinate conversion from the USD blend shape to morph target data
	 * @param LODIndex - LODIndex of the SkeletalMesh that will use the imported FUsdBlendShape
	 * @param AdditionalTransform - Additional affine transform to apply to the blend shape deltas and tangents (Note: No longer used, even in the
	 * deprecated overload of this function)
	 * @param PointIndexOffset - Index into the corresponding SkelMeshImportData.Points where the points corresponding to the UsdBlendShape's prim
	 * start
	 * @param UsedMorphTargetNames - Names that the newly created FUsdBlendShapes cannot have (this function will also add the names of the newly
	 * created FUsdBlendShapes to it)
	 * @param OutBlendShapes - Where the newly created blend shapes will be added to
	 * @param Options - Generic mesh conversion options to use when converting the blend shape
	 * @param GeomBindTransform - USD-space transform to be applied to the point and normal offsets before any other coordinate space conversion
	 * @return Whether the conversion was successful or not.
	 */
	USDUTILITIES_API bool ConvertBlendShape(
		const pxr::UsdSkelBlendShape& UsdBlendShape,
		const FUsdStageInfo& StageInfo,
		uint32 PointIndexOffset,
		TSet<FString>& UsedMorphTargetNames,
		UsdUtils::FBlendShapeMap& OutBlendShapes,
		const FUsdMeshConversionOptions& Options = FUsdMeshConversionOptions::DefaultOptions
	);
	USDUTILITIES_API bool ConvertBlendShape(
		const pxr::UsdSkelBlendShape& UsdBlendShape,
		const FUsdStageInfo& StageInfo,
		int32 LODIndex,
		uint32 PointIndexOffset,
		TSet<FString>& UsedMorphTargetNames,
		UsdUtils::FBlendShapeMap& OutBlendShapes,
		const FUsdMeshConversionOptions& Options,
		const pxr::GfMatrix4d* GeomBindTransform = nullptr
	);

	/**
	 * Extracts skeletal mesh data fro UsdSkinningQuery, and places the results in SkelMeshImportData.
	 *
	 * If a PrimvarToUVIndex mapping is provided in MaterialAssignments, it will be used to remap the read
	 * primvars into UV sets. Otherwise, a new PrimvarToUVIndex map will be constructed on-demand and assigned
	 * to MaterialAssignments instead.
	 *
	 * @param UsdSkinningQuery - SkinningQuery with the data to convert
	 * @param SkeletonQuery - Query object for the skeleton this mesh is bound to
	 * @param AdditionalTransform - Additional transform to apply to the vertices of the mesh (Note: No longer used, even in the deprecated overload
	 * of this function)
	 * @param SkelMeshImportData - Output parameter that will be filled with the converted data
	 * @param MaterialAssignments - Output parameter that will be filled with the material assignment data extracted from UsdSkinningQuery
	 * @param CommonOptions - Generic mesh conversion options to use for the conversion
	 * @return Whether the conversion was successful or not.
	 */
	USDUTILITIES_API bool ConvertSkinnedMesh(
		const pxr::UsdSkelSkinningQuery& UsdSkinningQuery,
		const pxr::UsdSkelSkeletonQuery& SkeletonQuery,
		FSkeletalMeshImportData& SkelMeshImportData,
		UsdUtils::FUsdPrimMaterialAssignmentInfo& MaterialAssignments,
		const FUsdMeshConversionOptions& CommonOptions = FUsdMeshConversionOptions::DefaultOptions
	);

	/**
	 * Will extract animation data from the animation source of InUsdSkeletonQuery's skeleton, and populate OutSkeletalAnimationAsset with the data.
	 * Warning: UAnimSequence must be previously set with a USkeleton generated from the skeletal data of the same UsdSkelSkeletonQuery.
	 * @param InUsdSkeletonQuery - SkinningQuery with the data to convert
	 * @param InSkinningTargets - Skinned meshes that use the skeleton of InUsdSkeletonQuery. Required to fetch the blend shape ordering of each mesh.
	 * Optional (can be nullptr to ignore)
	 * @param InBlendShapes - Converted blend shape data that will be used to interpret blend shape weights as morph target weight float curves.
	 * Optional (can be nullptr to ignore)
	 * @param InInterpretLODs - Whether we try parsing animation data from all LODs of skinning meshes that are inside LOD variant sets
	 * @param RootMotionPrim - Optional prim whose transform animation will be concatenaded into the root bone's regular joint animation
	 * @param OutSkeletalAnimationAsset - Output parameter that will be filled with the converted data
	 * @param OutStartOffsetSeconds - Optional output parameter that will be filled with the offset in seconds of when this UAnimSequence asset should
	 * be played since the start of the stage to match the intended composed animation. The baked UAnimSequence will only contain the range between the
	 * first and last joint and/or blend shape timeSamples, and so the offset is needed to properly position the animation on the stage's timeline.
	 * This is in seconds because the main use case is to use this offset when animating USkeletalMeshComponents, and those drive their UAnimSequences
	 * with seconds.
	 * @param OutScaleSeconds - Optional output parameter that will be filled with the total scale difference between the playback speed of the 
	 * baked AnimSequence and the stage
	 * @return Whether the conversion was successful or not.
	 */
	USDUTILITIES_API bool ConvertSkelAnim(
		const pxr::UsdSkelSkeletonQuery& InUsdSkeletonQuery,
		const pxr::VtArray<pxr::UsdSkelSkinningQuery>* InSkinningTargets,
		const UsdUtils::FBlendShapeMap* InBlendShapes,
		bool bInInterpretLODs,
		const pxr::UsdPrim& RootMotionPrim,
		UAnimSequence* OutSkeletalAnimationAsset,
		float* OutStartOffsetSeconds = nullptr,
		float* OutScaleSeconds = nullptr
	);

	/**
	 * Fills in InOutSkeletalMesh (and its Skeleton) with all the provided data converted from USD.
	 * @param InLODIndexToSkeletalMeshImportData - Container with the imported skeletal mesh data per LOD level
	 * @param InSkeletonBones - Bones to use for the reference skeleton (skeleton data on each LODIndexToSkeletalMeshImportData will be ignored).
	 * @param InBlendShapesByPath - Blend shapes to convert to morph targets
	 * @param InOutSkeletalMesh - SkeletalMesh to receive the converted data. It should have a valid USkeleton set.
	 * @return Whether the conversion was successful or not.
	 */
	USDUTILITIES_API bool ConvertSkeletalImportData(
		TArray<FSkeletalMeshImportData>& InLODIndexToSkeletalMeshImportData,
		const TArray<SkeletalMeshImportData::FBone>& InSkeletonBones,
		UsdUtils::FBlendShapeMap& InBlendShapesByPath,
		USkeletalMesh* InOutSkeletalMesh
	);

#endif	  // #if USE_USD_SDK && WITH_EDITOR
}	 // namespace UsdToUnreal

#if USE_USD_SDK && WITH_EDITOR
namespace UnrealToUsd
{
	/**
	 * Converts the bone data from Skeleton into UsdSkeleton.
	 * WARNING: Sometimes Skeleton->ReferenceSkeleton() has slightly different transforms than USkeletalMesh->GetRefSkeleton(), so make
	 * sure you're using the correct one for what you wish to do!
	 *
	 * @param Skeleton - Source UE data to convert
	 * @param UsdSkeleton - Previously created prim with the UsdSkelSkeleton schema that will be filled with converted data
	 * @return Whether the conversion was successful or not.
	 */
	USDUTILITIES_API bool ConvertSkeleton(const USkeleton* Skeleton, pxr::UsdSkelSkeleton& UsdSkeleton);
	USDUTILITIES_API bool ConvertSkeleton(const FReferenceSkeleton& ReferenceSkeleton, pxr::UsdSkelSkeleton& UsdSkeleton);

	// Fill out a UsdSkelAnimation's Joints attribute with data from ReferenceSkeleton, taking care to concatenate bone paths
	USDUTILITIES_API bool ConvertJointsAttribute(const FReferenceSkeleton& ReferenceSkeleton, pxr::UsdAttribute& JointsAttribute);

	/**
	 * Converts SkeletalMesh, its skeleton and morph target data into the corresponding USD objects and populates SkelRoot with them, at time TimeCode
	 * @param SkeletalMesh - Mesh with the source data. If it contains multiple LODs it will lead to the creation of LOD variant sets and variants
	 * within SkelRoot
	 * @param SkelRoot - Root prim of the output source data. Child UsdSkelSkeleton, UsdGeomMesh, and UsdSkelBlendShape will be created as children of
	 * it, containing the converted data
	 * @param TimeCode - TimeCode with which the converted data will be placed in the USD stage
	 * @param StageForMaterialAssignments - Stage to use when authoring material assignments (we use this when we want to export the mesh to a payload
	 * layer, but the material assignments to an asset layer)
	 * @param LowestMeshLOD - Lowest LOD of the UStaticMesh to export (start of the LOD range)
	 * @param HighestMeshLOD - Lowest LOD of the UStaticMesh to export (end of the LOD range)
	 * @return Whether the conversion was successful or not.
	 */
	USDUTILITIES_API bool ConvertSkeletalMesh(
		const USkeletalMesh* SkeletalMesh,
		pxr::UsdPrim& SkelRootPrim,
		const pxr::UsdTimeCode TimeCode = pxr::UsdTimeCode::Default(),
		UE::FUsdStage* StageForMaterialAssignments = nullptr,
		int32 LowestMeshLOD = 0,
		int32 HighestMeshLOD = INT32_MAX
	);

	/**
	 * Converts an AnimSequence to a UsdSkelAnimation. Includes bone transforms and blend shape weights.
	 * Keys will be baked at the stage TimeCodesPerSecond resolution.
	 * @param AnimSequence - The AnimSequence to convert
	 * @param SkelAnimPrim - Expected to be of type UsdkSkelAnimation
	 * @return Whether the conversion was successful or not.
	 */
	USDUTILITIES_API bool ConvertAnimSequence(UAnimSequence* AnimSequence, pxr::UsdPrim& SkelAnimPrim);

	/**
	 * Plays the provided Section in the background, driving its ControlRig and baking to USD the animated bones and
	 * curves end result.
	 * @param InSection - The section to bake;
	 * @param InTransform - A time transform to apply to the baked data before writing out;
	 * @param InMovieScene - The MovieScene that contains InSection (we can't access via the Outer chain from this RTTI
	 *                       module);
	 * @param IMovieScenePlayer - The player to drive the InSection with (e.g. a Sequencer instance);
	 * @param InRefSkeleton - Skeleton that describes the target Skeleton prim bone structure to use (the control rig
	 *                        may have an arbitrary bone hierarchy, but we always want to write bones compatible with
	 *                        the target UsdSkelRoot and its child prims);
	 * @param InSkelRoot - The SkelRoot that we're writing to (in some cases we need to update Mesh prims, and only
	 *                     the meshes within this SkelRoot will be updated);
	 * @param OutSkelAnimPrim - The SkelAnimation prim to receive the baked data;
	 * @param InBlendShapeMap - Optional map describing the stage blend shapes, in case baking of the rig's animation
	 *                          curves is desired;
	 * @return Whether the conversion was successful or not.
	 */
	USDUTILITIES_API bool ConvertControlRigSection(
		UMovieSceneControlRigParameterSection* InSection,
		const FMovieSceneInverseSequenceTransform& InTransform,
		UMovieScene* InMovieScene,
		IMovieScenePlayer* InPlayer,
		const FReferenceSkeleton& InRefSkeleton,
		pxr::UsdPrim& InSkelRoot,
		pxr::UsdPrim& OutSkelAnimPrim,
		const UsdUtils::FBlendShapeMap* InBlendShapeMap = nullptr
	);
}
#endif	  // #if USE_USD_SDK && WITH_EDITOR
