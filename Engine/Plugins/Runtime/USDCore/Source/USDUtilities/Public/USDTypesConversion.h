// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK
#include "UnrealUSDWrapper.h"
#include "USDStageOptions.h"

#include <string>

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class GfMatrix2d;
	class GfMatrix3d;
	class GfMatrix4d;
	class GfQuatd;
	class GfQuatf;
	class GfQuath;
	class GfVec2d;
	class GfVec2f;
	class GfVec2h;
	class GfVec2i;
	class GfVec3d;
	class GfVec3f;
	class GfVec3h;
	class GfVec3i;
	class GfVec4d;
	class GfVec4f;
	class GfVec4h;
	class GfVec4i;
	class SdfPath;
	class TfToken;

	class UsdStage;
	template<typename T>
	class TfRefPtr;

	using UsdStageRefPtr = TfRefPtr<UsdStage>;

	template<typename T>
	class VtArray;
PXR_NAMESPACE_CLOSE_SCOPE

template<typename UsdObjectType>
class TUsdStore;

struct FUsdStageInfo
{
	EUsdUpAxis UpAxis = EUsdUpAxis::ZAxis;
	double MetersPerUnit = 0.01;

	USDUTILITIES_API explicit FUsdStageInfo(const pxr::UsdStageRefPtr& Stage);
};

namespace UsdToUnreal
{
	USDUTILITIES_API FString ConvertString(const std::string& InString);
	USDUTILITIES_API FString ConvertString(const char* InString);

	USDUTILITIES_API FString ConvertPath(const pxr::SdfPath& Path);

	USDUTILITIES_API FName ConvertName(const char* InString);
	USDUTILITIES_API FName ConvertName(const std::string& InString);

	USDUTILITIES_API FString ConvertToken(const pxr::TfToken& Token);

	/** Assumes the input color is in linear space */
	USDUTILITIES_API FLinearColor ConvertColor(const pxr::GfVec3f& InValue);
	USDUTILITIES_API FLinearColor ConvertColor(const pxr::GfVec4f& InValue);

	USDUTILITIES_API FVector2D ConvertVector(const pxr::GfVec2h& InValue);
	USDUTILITIES_API FVector2D ConvertVector(const pxr::GfVec2f& InValue);
	USDUTILITIES_API FVector2D ConvertVector(const pxr::GfVec2d& InValue);
	USDUTILITIES_API FIntPoint ConvertVector(const pxr::GfVec2i& InValue);

	USDUTILITIES_API FVector ConvertVector(const pxr::GfVec3h& InValue);
	USDUTILITIES_API FVector ConvertVector(const pxr::GfVec3f& InValue);
	USDUTILITIES_API FVector ConvertVector(const pxr::GfVec3d& InValue);
	USDUTILITIES_API FIntVector ConvertVector(const pxr::GfVec3i& InValue);
	USDUTILITIES_API FVector ConvertVector(const FUsdStageInfo& StageInfo, const pxr::GfVec3h& InValue);
	USDUTILITIES_API FVector ConvertVector(const FUsdStageInfo& StageInfo, const pxr::GfVec3f& InValue);
	USDUTILITIES_API FVector ConvertVector(const FUsdStageInfo& StageInfo, const pxr::GfVec3d& InValue);

	USDUTILITIES_API FVector4 ConvertVector(const pxr::GfVec4h& InValue);
	USDUTILITIES_API FVector4 ConvertVector(const pxr::GfVec4f& InValue);
	USDUTILITIES_API FVector4 ConvertVector(const pxr::GfVec4d& InValue);
	USDUTILITIES_API FIntVector4 ConvertVector(const pxr::GfVec4i& InValue);

	USDUTILITIES_API FMatrix2D ConvertMatrix(const pxr::GfMatrix2d& Matrix);
	USDUTILITIES_API FMatrix3D ConvertMatrix(const pxr::GfMatrix3d& Matrix);
	USDUTILITIES_API FMatrix ConvertMatrix(const pxr::GfMatrix4d& Matrix);

	USDUTILITIES_API FTransform ConvertMatrix(const FUsdStageInfo& StageInfo, const pxr::GfMatrix4d& InMatrix);

	USDUTILITIES_API FQuat ConvertQuat(const pxr::GfQuath& InValue);
	USDUTILITIES_API FQuat ConvertQuat(const pxr::GfQuatf& InValue);
	USDUTILITIES_API FQuat ConvertQuat(const pxr::GfQuatd& InValue);

	/** Returns a distance in "UE units" (i.e. cm) */
	USDUTILITIES_API float ConvertDistance(const FUsdStageInfo& StageInfo, float InValue);
}	 // namespace UsdToUnreal

namespace UnrealToUsd
{
	USDUTILITIES_API TUsdStore<std::string> ConvertString(const TCHAR* InString);

	USDUTILITIES_API TUsdStore<pxr::SdfPath> ConvertPath(const TCHAR* InString);

	USDUTILITIES_API TUsdStore<std::string> ConvertName(const FName& InName);

	USDUTILITIES_API TUsdStore<pxr::TfToken> ConvertToken(const TCHAR* InString);

	/** Assumes the input color is in linear space. Returns a color in linear space */
	USDUTILITIES_API pxr::GfVec4f ConvertColor(const FLinearColor& InValue);

	/** Assumes the input color is in sRGB space. Returns a color in linear space */
	USDUTILITIES_API pxr::GfVec4f ConvertColor(const FColor& InValue);

	USDUTILITIES_API pxr::GfVec2h ConvertVectorHalf(const FVector2D& InValue);
	USDUTILITIES_API pxr::GfVec2f ConvertVectorFloat(const FVector2D& InValue);
	USDUTILITIES_API pxr::GfVec2d ConvertVectorDouble(const FVector2D& InValue);
	USDUTILITIES_API pxr::GfVec2i ConvertVectorInt(const FIntPoint& InValue);

	USDUTILITIES_API pxr::GfVec3h ConvertVectorHalf(const FVector& InValue);
	USDUTILITIES_API pxr::GfVec3f ConvertVectorFloat(const FVector& InValue);
	USDUTILITIES_API pxr::GfVec3d ConvertVectorDouble(const FVector& InValue);
	USDUTILITIES_API pxr::GfVec3i ConvertVectorInt(const FIntVector& InValue);
	USDUTILITIES_API pxr::GfVec3h ConvertVectorHalf(const FUsdStageInfo& StageInfo, const FVector& InValue);
	USDUTILITIES_API pxr::GfVec3f ConvertVectorFloat(const FUsdStageInfo& StageInfo, const FVector& InValue);
	USDUTILITIES_API pxr::GfVec3d ConvertVectorDouble(const FUsdStageInfo& StageInfo, const FVector& InValue);

	USDUTILITIES_API pxr::GfVec4h ConvertVectorHalf(const FVector4& InValue);
	USDUTILITIES_API pxr::GfVec4f ConvertVectorFloat(const FVector4& InValue);
	USDUTILITIES_API pxr::GfVec4d ConvertVectorDouble(const FVector4& InValue);
	USDUTILITIES_API pxr::GfVec4i ConvertVectorInt(const FIntVector4& InValue);

	USDUTILITIES_API pxr::GfMatrix2d ConvertMatrix(const FMatrix2D& Matrix);
	USDUTILITIES_API pxr::GfMatrix3d ConvertMatrix(const FMatrix3D& Matrix);
	USDUTILITIES_API pxr::GfMatrix4d ConvertMatrix(const FMatrix& Matrix);

	USDUTILITIES_API pxr::GfQuath ConvertQuatHalf(const FQuat& InValue);
	USDUTILITIES_API pxr::GfQuatf ConvertQuatFloat(const FQuat& InValue);
	USDUTILITIES_API pxr::GfQuatd ConvertQuatDouble(const FQuat& InValue);

	USDUTILITIES_API pxr::GfMatrix4d ConvertTransform(const FUsdStageInfo& StageInfo, const FTransform& Transform);

	/** Returns a distance in USD units (depends on metersPerUnit) */
	USDUTILITIES_API float ConvertDistance(const FUsdStageInfo& StageInfo, float InValue);

	USDUTILITIES_API TUsdStore<pxr::VtArray<pxr::GfVec3f>> ConvertBounds(const FUsdStageInfo& StageInfo, FBox Bounds);
}	 // namespace UnrealToUsd

namespace UsdUtils
{
	/**
	 * Decomposes the FMatrix into an FTransform.
	 * Identical to FTransform::SetFromMatrix, except that if a reflection is detected all axes are flipped instead
	 * of only the X axis, which keeps a uniform scaling.
	 */
	USDUTILITIES_API FTransform DecomposeWithUniformReflection(const FMatrix& InMatrix);

	USDUTILITIES_API FTransform ConvertTransformToUsdSpace(const FUsdStageInfo& StageInfo, const FTransform& TransformInUESpace);
	USDUTILITIES_API FTransform ConvertTransformToUESpace(const FUsdStageInfo& StageInfo, const FTransform& TransformInUsdSpace);

	/** Produces a transform that can be applied to a point from USD space to convert it into UE space */
	USDUTILITIES_API FTransform GetUsdToUESpaceTransform(const FUsdStageInfo& StageInfo);

	FTransform ConvertAxes(const bool bZUp, const FTransform Transform);
}
#endif	  // #if USE_USD_SDK
