// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Modules/ModuleInterface.h"
#include "USDProjectSettings.h"

class IUsdUtilitiesModule : public IModuleInterface
{
public:
	/**
	 * Delegate that is invoked when the AddReference / AddPayload functions want to display a dialog so the user
	 * can pick the desired behavior for handling referencer type names.
	 *
	 * This is needed because those functions are in RTTI=true modules, where Slate code doesn't compile, and also
	 * USDUtilities can't easily depend on any other USD module that is RTTI=false. The USDStageEditorModule should
	 * register the intended dialog here when it runs its StartupModule()
	 */
	DECLARE_DELEGATE_RetVal_TwoParams(bool /* bAccepted */, FOnReferenceHandlingDialog, const FText& /* DialogText */, EReferencerTypeHandling& /* ChosenHandling*/);
	FOnReferenceHandlingDialog OnReferenceHandlingDialog;
};
