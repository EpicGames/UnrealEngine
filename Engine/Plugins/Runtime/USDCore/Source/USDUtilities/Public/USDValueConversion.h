// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <string>

#include "CoreMinimal.h"
#include "Misc/TVariant.h"

#include "UsdWrappers/ForwardDeclarations.h"
#include "UsdWrappers/UsdAttribute.h"

#if USE_USD_SDK

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class SdfAssetPath;
	class SdfTimeCode;
	class TfToken;
	class VtValue;
PXR_NAMESPACE_CLOSE_SCOPE

#endif	  // #if USE_USD_SDK

struct FMatrix2D;
struct FMatrix3D;

namespace UE
{
	class FVtValue;
}

namespace UsdUtils
{
	/** Source: https://graphics.pixar.com/usd/docs/api/_usd__page__datatypes.html */
	enum class EUsdBasicDataTypes
	{
		None,

		Bool,
		Uchar,
		Int,
		Uint,
		Int64,
		Uint64,

		Half,
		Float,
		Double,

		Timecode,

		String,
		Token,
		Asset,

		Matrix2d,
		Matrix3d,
		Matrix4d,

		Quatd,
		Quatf,
		Quath,

		Double2,
		Float2,
		Half2,
		Int2,

		Double3,
		Float3,
		Half3,
		Int3,

		Double4,
		Float4,
		Half4,
		Int4,
	};

	// WARNING: Do not change the order of the types in this TVariant declaration, or it will break backwards compatibility with data that
	// was serialized before the change, due to how `operator<<( FArchive& Ar, FConvertedVtValueComponent& Component )` is implemented
	using FConvertedVtValueComponent = TVariant<bool, uint8, int32, uint32, int64, uint64, float, double, FString>;

	/** Represents a single non-array value held by a pxr::VtValue object, like a 'float', a 'TfToken' or a 'GfMatrix3d' */
	using FConvertedVtValueEntry = TArray<FConvertedVtValueComponent>;

	/** Corresponds to a value held by a pxr::VtValue converted into UE types. Should handle anything a VtValue can hold, including arrays of values
	 */
	struct FConvertedVtValue
	{
		TArray<FConvertedVtValueEntry> Entries;

		EUsdBasicDataTypes SourceType = EUsdBasicDataTypes::None;
		FString ExtraTypeInfo;	  // Used so that we can serialize something like SdfListOp<Token> as basically a token array, but still get the same
								  // type when converting back on the other end.
		bool bIsArrayValued = false;
		bool bIsEmpty = false;	  // Helps differentiating between empty arrays and having actually no value
	};
}	 // namespace UsdUtils

USDUTILITIES_API FArchive& operator<<(FArchive& Ar, UsdUtils::FConvertedVtValue& Struct);
USDUTILITIES_API FArchive& operator<<(FArchive& Ar, UsdUtils::FConvertedVtValueComponent& Component);

namespace UsdToUnreal
{
	USDUTILITIES_API bool ConvertValue(const UE::FVtValue& InValue, UsdUtils::FConvertedVtValue& OutValue);
}

namespace UnrealToUsd
{
	USDUTILITIES_API bool ConvertValue(const UsdUtils::FConvertedVtValue& InValue, UE::FVtValue& OutValue);
}

namespace UsdUtils
{
	/**
	 * Mostly uses USD to stringify the underlying value of the provided pxr::VtValue, but will
	 * use some custom implementation depending on the type. For string-like types in particular we will mostly
	 * use our own implementations, as they preserve quotes and delimiters in a way that we can unstringify later.
	 */
	USDUTILITIES_API FString Stringify(const UE::FVtValue& Value);
#if USE_USD_SDK
	USDUTILITIES_API FString Stringify(const pxr::VtValue& Value);
#endif	  // USE_USD_SDK

	// Converts the UE type into a string that can be set as USD metadata with the corresponding USD data type.
	//
	// We need these explicit functions here as well as the generic Stringify() because our Python/Blueprint libraries
	// need to do these operations, but they are on modules that can't manipulate the USD types directly, and we need
	// to be able to internally e.g. convert a TArray<FString> into a pxr::SdfListOp<pxr::TfToken> to stringify some of these.
	//
	// We provide Python/Blueprint libraries that can do these stringify/unstringify functions in the first place because
	// otherwise a user that tried to manipulate metadata values from Python/Blueprint would be forced to either be
	// content with just strings and not actual types, or to try unstringifying these types on their own (and it would
	// not be trivial to unstringify an e.g. matrix3d[] on Blueprint)
	USDUTILITIES_API FString StringifyAsBool(bool Value);
	USDUTILITIES_API FString StringifyAsUChar(uint8 Value);
	USDUTILITIES_API FString StringifyAsInt(int32 Value);
	USDUTILITIES_API FString StringifyAsUInt(uint32 Value);
	USDUTILITIES_API FString StringifyAsInt64(int64 Value);
	USDUTILITIES_API FString StringifyAsUInt64(uint64 Value);
	USDUTILITIES_API FString StringifyAsHalf(float Value);
	USDUTILITIES_API FString StringifyAsFloat(float Value);
	USDUTILITIES_API FString StringifyAsDouble(double Value);
	USDUTILITIES_API FString StringifyAsTimeCode(double Value);
	USDUTILITIES_API FString StringifyAsString(const FString& Value);
	USDUTILITIES_API FString StringifyAsToken(const FString& Value);
	USDUTILITIES_API FString StringifyAsAssetPath(const FString& Value);
	USDUTILITIES_API FString StringifyAsMatrix2d(const FMatrix2D& Value);
	USDUTILITIES_API FString StringifyAsMatrix3d(const FMatrix3D& Value);
	USDUTILITIES_API FString StringifyAsMatrix4d(const FMatrix& Value);
	USDUTILITIES_API FString StringifyAsQuatd(const FQuat& Value);
	USDUTILITIES_API FString StringifyAsQuatf(const FQuat& Value);
	USDUTILITIES_API FString StringifyAsQuath(const FQuat& Value);
	USDUTILITIES_API FString StringifyAsDouble2(const FVector2D& Value);
	USDUTILITIES_API FString StringifyAsFloat2(const FVector2D& Value);
	USDUTILITIES_API FString StringifyAsHalf2(const FVector2D& Value);
	USDUTILITIES_API FString StringifyAsInt2(const FIntPoint& Value);
	USDUTILITIES_API FString StringifyAsDouble3(const FVector& Value);
	USDUTILITIES_API FString StringifyAsFloat3(const FVector& Value);
	USDUTILITIES_API FString StringifyAsHalf3(const FVector& Value);
	USDUTILITIES_API FString StringifyAsInt3(const FIntVector& Value);
	USDUTILITIES_API FString StringifyAsDouble4(const FVector4& Value);
	USDUTILITIES_API FString StringifyAsFloat4(const FVector4& Value);
	USDUTILITIES_API FString StringifyAsHalf4(const FVector4& Value);
	USDUTILITIES_API FString StringifyAsInt4(const FIntVector4& Value);
	USDUTILITIES_API FString StringifyAsBoolArray(const TArray<bool>& Value);
	USDUTILITIES_API FString StringifyAsUCharArray(const TArray<uint8>& Value);
	USDUTILITIES_API FString StringifyAsIntArray(const TArray<int32>& Value);
	USDUTILITIES_API FString StringifyAsUIntArray(const TArray<uint32>& Value);
	USDUTILITIES_API FString StringifyAsInt64Array(const TArray<int64>& Value);
	USDUTILITIES_API FString StringifyAsUInt64Array(const TArray<uint64>& Value);
	USDUTILITIES_API FString StringifyAsHalfArray(const TArray<float>& Value);
	USDUTILITIES_API FString StringifyAsFloatArray(const TArray<float>& Value);
	USDUTILITIES_API FString StringifyAsDoubleArray(const TArray<double>& Value);
	USDUTILITIES_API FString StringifyAsTimeCodeArray(const TArray<double>& Value);
	USDUTILITIES_API FString StringifyAsStringArray(const TArray<FString>& Value);
	USDUTILITIES_API FString StringifyAsTokenArray(const TArray<FString>& Value);
	USDUTILITIES_API FString StringifyAsAssetPathArray(const TArray<FString>& Value);
	USDUTILITIES_API FString StringifyAsListOpTokens(const TArray<FString>& Value);
	USDUTILITIES_API FString StringifyAsMatrix2dArray(const TArray<FMatrix2D>& Value);
	USDUTILITIES_API FString StringifyAsMatrix3dArray(const TArray<FMatrix3D>& Value);
	USDUTILITIES_API FString StringifyAsMatrix4dArray(const TArray<FMatrix>& Value);
	USDUTILITIES_API FString StringifyAsQuatdArray(const TArray<FQuat>& Value);
	USDUTILITIES_API FString StringifyAsQuatfArray(const TArray<FQuat>& Value);
	USDUTILITIES_API FString StringifyAsQuathArray(const TArray<FQuat>& Value);
	USDUTILITIES_API FString StringifyAsDouble2Array(const TArray<FVector2D>& Value);
	USDUTILITIES_API FString StringifyAsFloat2Array(const TArray<FVector2D>& Value);
	USDUTILITIES_API FString StringifyAsHalf2Array(const TArray<FVector2D>& Value);
	USDUTILITIES_API FString StringifyAsInt2Array(const TArray<FIntPoint>& Value);
	USDUTILITIES_API FString StringifyAsDouble3Array(const TArray<FVector>& Value);
	USDUTILITIES_API FString StringifyAsFloat3Array(const TArray<FVector>& Value);
	USDUTILITIES_API FString StringifyAsHalf3Array(const TArray<FVector>& Value);
	USDUTILITIES_API FString StringifyAsInt3Array(const TArray<FIntVector>& Value);
	USDUTILITIES_API FString StringifyAsDouble4Array(const TArray<FVector4>& Value);
	USDUTILITIES_API FString StringifyAsFloat4Array(const TArray<FVector4>& Value);
	USDUTILITIES_API FString StringifyAsHalf4Array(const TArray<FVector4>& Value);
	USDUTILITIES_API FString StringifyAsInt4Array(const TArray<FIntVector4>& Value);

#if USE_USD_SDK
	/**
	 * Unstringifies the data in String according to what TypeName it is.
	 * The unstringified USD value is placed into a VtValue for type erasure.
	 * Example: String is "(1, 0.5, 0.3)" and TypeName "float3" means Output will contain a pxr::GfVec3f(1, 0.5, 0.3) value
	 */
	USDUTILITIES_API bool Unstringify(const FString& String, const FString& TypeName, pxr::VtValue& Output);
#endif	  // USE_USD_SDK

	// c.f. comment above StringifyAsBool
	USDUTILITIES_API bool UnstringifyAsBool(const FString& String, bool& OutValue);
	USDUTILITIES_API bool UnstringifyAsUChar(const FString& String, uint8& OutValue);
	USDUTILITIES_API bool UnstringifyAsInt(const FString& String, int32& OutValue);
	USDUTILITIES_API bool UnstringifyAsUInt(const FString& String, uint32& OutValue);
	USDUTILITIES_API bool UnstringifyAsInt64(const FString& String, int64& OutValue);
	USDUTILITIES_API bool UnstringifyAsUInt64(const FString& String, uint64& OutValue);
	USDUTILITIES_API bool UnstringifyAsHalf(const FString& String, float& OutValue);
	USDUTILITIES_API bool UnstringifyAsFloat(const FString& String, float& OutValue);
	USDUTILITIES_API bool UnstringifyAsDouble(const FString& String, double& OutValue);
	USDUTILITIES_API bool UnstringifyAsTimeCode(const FString& String, double& OutValue);
	USDUTILITIES_API bool UnstringifyAsString(const FString& String, FString& OutValue);
	USDUTILITIES_API bool UnstringifyAsToken(const FString& String, FString& OutValue);
	USDUTILITIES_API bool UnstringifyAsAssetPath(const FString& String, FString& OutValue);
	USDUTILITIES_API bool UnstringifyAsMatrix2d(const FString& String, FMatrix2D& OutValue);
	USDUTILITIES_API bool UnstringifyAsMatrix3d(const FString& String, FMatrix3D& OutValue);
	USDUTILITIES_API bool UnstringifyAsMatrix4d(const FString& String, FMatrix& OutValue);
	USDUTILITIES_API bool UnstringifyAsQuatd(const FString& String, FQuat& OutValue);
	USDUTILITIES_API bool UnstringifyAsQuatf(const FString& String, FQuat& OutValue);
	USDUTILITIES_API bool UnstringifyAsQuath(const FString& String, FQuat& OutValue);
	USDUTILITIES_API bool UnstringifyAsDouble2(const FString& String, FVector2D& OutValue);
	USDUTILITIES_API bool UnstringifyAsFloat2(const FString& String, FVector2D& OutValue);
	USDUTILITIES_API bool UnstringifyAsHalf2(const FString& String, FVector2D& OutValue);
	USDUTILITIES_API bool UnstringifyAsInt2(const FString& String, FIntPoint& OutValue);
	USDUTILITIES_API bool UnstringifyAsDouble3(const FString& String, FVector& OutValue);
	USDUTILITIES_API bool UnstringifyAsFloat3(const FString& String, FVector& OutValue);
	USDUTILITIES_API bool UnstringifyAsHalf3(const FString& String, FVector& OutValue);
	USDUTILITIES_API bool UnstringifyAsInt3(const FString& String, FIntVector& OutValue);
	USDUTILITIES_API bool UnstringifyAsDouble4(const FString& String, FVector4& OutValue);
	USDUTILITIES_API bool UnstringifyAsFloat4(const FString& String, FVector4& OutValue);
	USDUTILITIES_API bool UnstringifyAsHalf4(const FString& String, FVector4& OutValue);
	USDUTILITIES_API bool UnstringifyAsInt4(const FString& String, FIntVector4& OutValue);
	USDUTILITIES_API bool UnstringifyAsBoolArray(const FString& String, TArray<bool>& OutValue);
	USDUTILITIES_API bool UnstringifyAsUCharArray(const FString& String, TArray<uint8>& OutValue);
	USDUTILITIES_API bool UnstringifyAsIntArray(const FString& String, TArray<int32>& OutValue);
	USDUTILITIES_API bool UnstringifyAsUIntArray(const FString& String, TArray<uint32>& OutValue);
	USDUTILITIES_API bool UnstringifyAsInt64Array(const FString& String, TArray<int64>& OutValue);
	USDUTILITIES_API bool UnstringifyAsUInt64Array(const FString& String, TArray<uint64>& OutValue);
	USDUTILITIES_API bool UnstringifyAsHalfArray(const FString& String, TArray<float>& OutValue);
	USDUTILITIES_API bool UnstringifyAsFloatArray(const FString& String, TArray<float>& OutValue);
	USDUTILITIES_API bool UnstringifyAsDoubleArray(const FString& String, TArray<double>& OutValue);
	USDUTILITIES_API bool UnstringifyAsTimeCodeArray(const FString& String, TArray<double>& OutValue);
	USDUTILITIES_API bool UnstringifyAsStringArray(const FString& String, TArray<FString>& OutValue);
	USDUTILITIES_API bool UnstringifyAsTokenArray(const FString& String, TArray<FString>& OutValue);
	USDUTILITIES_API bool UnstringifyAsAssetPathArray(const FString& String, TArray<FString>& OutValue);
	USDUTILITIES_API bool UnstringifyAsListOpTokens(const FString& String, TArray<FString>& OutValue);
	USDUTILITIES_API bool UnstringifyAsMatrix2dArray(const FString& String, TArray<FMatrix2D>& OutValue);
	USDUTILITIES_API bool UnstringifyAsMatrix3dArray(const FString& String, TArray<FMatrix3D>& OutValue);
	USDUTILITIES_API bool UnstringifyAsMatrix4dArray(const FString& String, TArray<FMatrix>& OutValue);
	USDUTILITIES_API bool UnstringifyAsQuatdArray(const FString& String, TArray<FQuat>& OutValue);
	USDUTILITIES_API bool UnstringifyAsQuatfArray(const FString& String, TArray<FQuat>& OutValue);
	USDUTILITIES_API bool UnstringifyAsQuathArray(const FString& String, TArray<FQuat>& OutValue);
	USDUTILITIES_API bool UnstringifyAsDouble2Array(const FString& String, TArray<FVector2D>& OutValue);
	USDUTILITIES_API bool UnstringifyAsFloat2Array(const FString& String, TArray<FVector2D>& OutValue);
	USDUTILITIES_API bool UnstringifyAsHalf2Array(const FString& String, TArray<FVector2D>& OutValue);
	USDUTILITIES_API bool UnstringifyAsInt2Array(const FString& String, TArray<FIntPoint>& OutValue);
	USDUTILITIES_API bool UnstringifyAsDouble3Array(const FString& String, TArray<FVector>& OutValue);
	USDUTILITIES_API bool UnstringifyAsFloat3Array(const FString& String, TArray<FVector>& OutValue);
	USDUTILITIES_API bool UnstringifyAsHalf3Array(const FString& String, TArray<FVector>& OutValue);
	USDUTILITIES_API bool UnstringifyAsInt3Array(const FString& String, TArray<FIntVector>& OutValue);
	USDUTILITIES_API bool UnstringifyAsDouble4Array(const FString& String, TArray<FVector4>& OutValue);
	USDUTILITIES_API bool UnstringifyAsFloat4Array(const FString& String, TArray<FVector4>& OutValue);
	USDUTILITIES_API bool UnstringifyAsHalf4Array(const FString& String, TArray<FVector4>& OutValue);
	USDUTILITIES_API bool UnstringifyAsInt4Array(const FString& String, TArray<FIntVector4>& OutValue);

	/**
	 * Uses USD to quickly fetch the underlying value of the wrapped pxr::VtValue for fundamental data types.
	 * For complex vector/array types ConvertValue must be used instead.
	 * Template implementation must be hidden on cpp as we can't expose USD implementation on the header files.
	 */
	template<typename T>
	USDUTILITIES_API TOptional<T> GetUnderlyingValue(const UE::FVtValue& InValue);

	/**
	 * Uses USD to quickly set the underlying value of the wrapped pxr::VtValue for fundamental data types.
	 * This will not allow changing the underlying type of InValue, i.e. you can only set a float if InValue
	 * already contains a float or if its otherwise empty.
	 * For complex vector/array types ConvertValue must be used instead.
	 * Template implementation must be hidden on cpp as we can't expose USD implementation on the header files.
	 */
	template<typename T>
	USDUTILITIES_API bool SetUnderlyingValue(UE::FVtValue& InValue, const T& UnderlyingValue);

	/**
	 * Returns the name of the SdfValueTypeName object for the type of Value
	 *
	 * Note that this is the "Value type token" (e.g. will be something like 'matrix4d' or 'double2', and not 'GfMatrix4d' or 'GfVec2d')
	 * Note that this may not be the correct SdfValueTypeName for the *attribute*, it's just the implied type of the value (e.g. for an
	 * attribute with SdfValueTypeName 'normal3d', it's VtValues would have underlying type 'GfVec3d', and so this function would return 'double3')
	 */
	USDUTILITIES_API FString GetImpliedTypeName(const UE::FVtValue& Value);
}	 // namespace UsdUtils
