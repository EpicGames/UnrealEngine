// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Linq;

namespace UnrealBuildTool.Rules
{
	public class USDUtilities : ModuleRules
	{
		public USDUtilities(ReadOnlyTargetRules Target) : base(Target)
		{
			bUseRTTI = true;

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"UnrealUSDWrapper",
					"HairStrandsCore", // Objects/USDSchemaTranslator.h references FHairGroupsInterpolation
					"USDClasses", // So that consumers can also include IUsdClassesModule for the new definition of FDisplayColorMaterial
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"CinematicCamera",
					"ControlRig",
					"Engine",
					"Foliage",
					"GeometryCache", // Just so that we can fetch its AssetImportData
					"HairStrandsCore",
					"IntelTBB",
					"Landscape", // So that GetSchemaNameForComponent knows what to do with landscape proxies
					"LiveLinkComponents", // For converting LiveLinkComponentController properties to USD
					"MeshDescription",
					"MovieScene",
					"MovieSceneTracks",
					"OpenSubdiv",
					"RenderCore",
					"RHI", // So that we can use GMaxRHIFeatureLevel when force-loading textures before baking materials
					"SkeletalMeshDescription",
					"Slate",
					"SlateCore",
					"StaticMeshDescription",
				}
			);

			bool bEnableUsdMaterialX = false;

			if (Target.bBuildEditor)
			{
                bEnableExceptions = true;
                PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"DesktopPlatform", // For OpenFileDialog/SaveFileDialog
						"DeveloperSettings", // So that we can un/register the diagnostic delegate when the project settings change
						"MaterialBaking", // For the BakeMaterials function
						"MaterialEditor",
						"MeshUtilities",
						"MessageLog",
						"PropertyEditor",
						"TextureUtilitiesCommon",
						"UnrealEd",
					}
				);

				if (Target.IsInPlatformGroup(UnrealPlatformGroup.Windows) ||
					Target.IsInPlatformGroup(UnrealPlatformGroup.Unix) ||
					Target.Platform == UnrealTargetPlatform.Mac)
				{
					// Public because MaterialX headers are #include'd in
					// public USDUtilities headers.
					PublicDependencyModuleNames.AddRange(
						new string[]
						{
							"MaterialX"
						}
					);

					bEnableUsdMaterialX = true;
				}
			}

			PublicDefinitions.Add("ENABLE_USD_MATERIALX=" + (bEnableUsdMaterialX ? "1" : "0"));

			UnrealUSDWrapper.CheckAndSetupUsdSdk(Target, this);
		}
	}
}
