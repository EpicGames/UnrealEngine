// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDListener.h"

#include "USDMemory.h"

#include "UsdWrappers/ForwardDeclarations.h"
#include "UsdWrappers/SdfLayer.h"
#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdStage.h"
#include "UsdWrappers/VtValue.h"

#if USE_USD_SDK

#include "USDIncludesStart.h"
#include "pxr/base/tf/weakBase.h"
#include "pxr/usd/sdf/changeList.h"
#include "pxr/usd/sdf/notice.h"
#include "pxr/usd/sdf/path.h"
#include "pxr/usd/usd/notice.h"
#include "pxr/usd/usd/prim.h"
#include "pxr/usd/usd/stage.h"
#include "pxr/usd/usdGeom/tokens.h"
#include "USDIncludesEnd.h"
#endif	  // USE_USD_SDK

#define ENABLE_NOTICE_LOGGING 0

namespace UE::USDListener::Private
{
#if USE_USD_SDK

#if ENABLE_NOTICE_LOGGING
	void LogChangeListEntry(const pxr::SdfChangeList::Entry& Entry, int IndentLevel)
	{
		FScopedUsdAllocs Allocs;

		FString Indent;
		for (int Index = 0; Index < IndentLevel; ++Index)
		{
			Indent += TEXT("\t");
		}

		UE_LOGF(LogUsd, Log, "%lsChangeListEntry:", *Indent);

		UE_LOGF(LogUsd, Log, "%ls\tInfoChanges:", *Indent);
		for (const std::pair<pxr::TfToken, std::pair<pxr::VtValue, pxr::VtValue>>& AttributeChange : Entry.infoChanged)
		{
			const FString FieldToken = UTF8_TO_TCHAR(AttributeChange.first.GetString().c_str());

			const pxr::VtValue& OldValue = AttributeChange.second.first;
			const pxr::VtValue& NewValue = AttributeChange.second.second;
			std::string OldValueString = pxr::TfStringify(OldValue);
			std::string NewValueString = pxr::TfStringify(NewValue);

			UE_LOGF(
				LogUsd,
				Log,
				"%ls\t\t'%ls': From '%ls' to '%ls'",
				*Indent,
				*FieldToken,
				UTF8_TO_TCHAR(OldValueString.c_str()),
				UTF8_TO_TCHAR(NewValueString.c_str())
			);
		}

		UE_LOGF(LogUsd, Log, "%ls\tSubLayerChanges:", *Indent);
		static_assert(static_cast<int>(pxr::SdfChangeList::SubLayerChangeType::SubLayerAdded) == 0, "Enum values changed!");
		static_assert(static_cast<int>(pxr::SdfChangeList::SubLayerChangeType::SubLayerRemoved) == 1, "Enum values changed!");
		static_assert(static_cast<int>(pxr::SdfChangeList::SubLayerChangeType::SubLayerOffset) == 2, "Enum values changed!");
		const TCHAR* SubLayerChangeTypeStr[] = {
			TEXT("SubLayerAdded"),
			TEXT("SubLayerRemoved"),
			TEXT("SubLayerOffset"),
		};
		for (const std::pair<std::string, pxr::SdfChangeList::SubLayerChangeType>& SubLayerChange : Entry.subLayerChanges)
		{
			UE_LOGF(
				LogUsd,
				Log,
				"%ls\t\t'%ls': change type '%ls'",
				*Indent,
				UTF8_TO_TCHAR(SubLayerChange.first.c_str()),
				SubLayerChangeTypeStr[static_cast<int>(SubLayerChange.second)]
			);
		}

		UE_LOGF(LogUsd, Log, "%ls\tOldPath: '%ls'", *Indent, UTF8_TO_TCHAR(Entry.oldPath.GetString().c_str()));

		UE_LOGF(LogUsd, Log, "%ls\tOldIdentifier: '%ls'", *Indent, UTF8_TO_TCHAR(Entry.oldIdentifier.c_str()));

		UE_LOGF(LogUsd, Log, "%ls\tFlags:", *Indent);
		TArray<FString> FlagsToLog;
#define TRY_LOG_FLAG(FlagName)           \
	if (Entry.flags.##FlagName)          \
	{                                    \
		FlagsToLog.Add(TEXT(#FlagName)); \
	}

		TRY_LOG_FLAG(didChangeIdentifier);
		TRY_LOG_FLAG(didChangeResolvedPath);
		TRY_LOG_FLAG(didReplaceContent);
		TRY_LOG_FLAG(didReloadContent);
		TRY_LOG_FLAG(didReorderChildren);
		TRY_LOG_FLAG(didReorderProperties);
		TRY_LOG_FLAG(didRename);
		TRY_LOG_FLAG(didChangePrimVariantSets);
		TRY_LOG_FLAG(didChangePrimInheritPaths);
		TRY_LOG_FLAG(didChangePrimSpecializes);
		TRY_LOG_FLAG(didChangePrimReferences);
		TRY_LOG_FLAG(didChangeAttributeTimeSamples);
		TRY_LOG_FLAG(didChangeAttributeConnection);
		TRY_LOG_FLAG(didChangeRelationshipTargets);
		TRY_LOG_FLAG(didAddTarget);
		TRY_LOG_FLAG(didRemoveTarget);
		TRY_LOG_FLAG(didAddInertPrim);
		TRY_LOG_FLAG(didAddNonInertPrim);
		TRY_LOG_FLAG(didRemoveInertPrim);
		TRY_LOG_FLAG(didRemoveNonInertPrim);
		TRY_LOG_FLAG(didAddPropertyWithOnlyRequiredFields);
		TRY_LOG_FLAG(didAddProperty);
		TRY_LOG_FLAG(didRemovePropertyWithOnlyRequiredFields);
		TRY_LOG_FLAG(didRemoveProperty);

#undef TRY_LOG_FLAG
		for (const FString& Flag : FlagsToLog)
		{
			UE_LOGF(LogUsd, Log, "%ls\t\t'%ls'", *Indent, *Flag);
		}
	}

	void LogConvertedChangeListEntry(const UsdUtils::FSdfChangeListEntry& Entry, int32 IndentLevel)
	{
		FString Indent;
		for (int Index = 0; Index < IndentLevel; ++Index)
		{
			Indent += TEXT("\t");
		}

		UE_LOGF(LogUsd, Log, "%lsConverted ChangeListEntry:", *Indent);

		UE_LOGF(LogUsd, Log, "%ls\tAttribute changes:", *Indent);
		for (const UsdUtils::FAttributeChange& AttributeChange : Entry.FieldChanges)
		{
			UE_LOGF(
				LogUsd,
				Log,
				"%ls\t\tfield '%ls' typename '%ls': From '%ls' to '%ls'",
				*Indent,
				*AttributeChange.Field,
				AttributeChange.OldValue.IsEmpty() ? *AttributeChange.NewValue.GetTypeName() : *AttributeChange.OldValue.GetTypeName(),
				UTF8_TO_TCHAR(pxr::TfStringify(AttributeChange.OldValue.GetUsdValue()).c_str()),
				UTF8_TO_TCHAR(pxr::TfStringify(AttributeChange.NewValue.GetUsdValue()).c_str())
			);
		}

		UE_LOGF(LogUsd, Log, "%ls\tSubLayerChanges:", *Indent);
		static_assert(static_cast<int>(UsdUtils::ESubLayerChangeType::SubLayerAdded) == 0, "Enum values changed!");
		static_assert(static_cast<int>(UsdUtils::ESubLayerChangeType::SubLayerRemoved) == 1, "Enum values changed!");
		static_assert(static_cast<int>(UsdUtils::ESubLayerChangeType::SubLayerOffset) == 2, "Enum values changed!");
		const TCHAR* SubLayerChangeTypeStr[] = {
			TEXT("SubLayerAdded"),
			TEXT("SubLayerRemoved"),
			TEXT("SubLayerOffset"),
		};
		for (const TPair<FString, UsdUtils::ESubLayerChangeType>& SubLayerChange : Entry.SubLayerChanges)
		{
			UE_LOGF(
				LogUsd,
				Log,
				"%ls\t\t'%ls': change type '%ls'",
				*Indent,
				*SubLayerChange.Key,
				SubLayerChangeTypeStr[static_cast<int>(SubLayerChange.Value)]
			);
		}

		UE_LOGF(LogUsd, Log, "%ls\tOldPath: '%ls'", *Indent, *Entry.OldPath);

		UE_LOGF(LogUsd, Log, "%ls\tOldIdentifier: '%ls'", *Indent, *Entry.OldIdentifier);

		UE_LOGF(LogUsd, Log, "%ls\tFlags:", *Indent);
		TArray<FString> FlagsToLog;
#define TRY_LOG_FLAG(FlagName)           \
	if (Entry.Flags.##FlagName)          \
	{                                    \
		FlagsToLog.Add(TEXT(#FlagName)); \
	}

		TRY_LOG_FLAG(bDidChangeIdentifier);
		TRY_LOG_FLAG(bDidChangeResolvedPath);
		TRY_LOG_FLAG(bDidReplaceContent);
		TRY_LOG_FLAG(bDidReloadContent);
		TRY_LOG_FLAG(bDidReorderChildren);
		TRY_LOG_FLAG(bDidReorderProperties);
		TRY_LOG_FLAG(bDidRename);
		TRY_LOG_FLAG(bDidChangePrimVariantSets);
		TRY_LOG_FLAG(bDidChangePrimInheritPaths);
		TRY_LOG_FLAG(bDidChangePrimSpecializes);
		TRY_LOG_FLAG(bDidChangePrimReferences);
		TRY_LOG_FLAG(bDidChangeAttributeTimeSamples);
		TRY_LOG_FLAG(bDidChangeAttributeConnection);
		TRY_LOG_FLAG(bDidChangeRelationshipTargets);
		TRY_LOG_FLAG(bDidAddTarget);
		TRY_LOG_FLAG(bDidRemoveTarget);
		TRY_LOG_FLAG(bDidAddInertPrim);
		TRY_LOG_FLAG(bDidAddNonInertPrim);
		TRY_LOG_FLAG(bDidRemoveInertPrim);
		TRY_LOG_FLAG(bDidRemoveNonInertPrim);
		TRY_LOG_FLAG(bDidAddPropertyWithOnlyRequiredFields);
		TRY_LOG_FLAG(bDidAddProperty);
		TRY_LOG_FLAG(bDidRemovePropertyWithOnlyRequiredFields);
		TRY_LOG_FLAG(bDidRemoveProperty);

#undef TRY_LOG_FLAG
		for (const FString& Flag : FlagsToLog)
		{
			UE_LOGF(LogUsd, Log, "%ls\t\t'%ls'", *Indent, *Flag);
		}
	}

	void LogObjectsChangedPathRange(const pxr::UsdNotice::ObjectsChanged::PathRange& PathRange)
	{
		FScopedUsdAllocs Allocs;

		for (pxr::UsdNotice::ObjectsChanged::PathRange::const_iterator It = PathRange.begin(); It != PathRange.end(); ++It)
		{
			const FString FullFieldPath = UTF8_TO_TCHAR(It->GetAsString().c_str());
			UE_LOGF(LogUsd, Log, "\t\tObject '%ls'", *FullFieldPath);

			const std::vector<const pxr::SdfChangeList::Entry*>& Changes = It.base()->second;
			for (const pxr::SdfChangeList::Entry* Entry : Changes)
			{
				if (Entry)
				{
					const int32 IndentLevel = 3;
					LogChangeListEntry(*Entry, IndentLevel);
				}
				else
				{
					UE_LOGF(LogUsd, Log, "\t\t\tNullptr change");
				}
			}
		}
	}

	void LogConvertedChangesByPath(const UsdUtils::FObjectChangesByPath& Changes)
	{
		for (const TPair<FString, TArray<UsdUtils::FSdfChangeListEntry>>& Pair : Changes)
		{
			UE_LOGF(LogUsd, Log, "\t\tObject '%ls'", *Pair.Key);

			for (const UsdUtils::FSdfChangeListEntry& ChangeListEntry : Pair.Value)
			{
				const int32 IndentLevel = 3;
				LogConvertedChangeListEntry(ChangeListEntry, IndentLevel);
			}
		}
	}

	void LogNotice(const pxr::UsdNotice::ObjectsChanged& Notice, const pxr::UsdStageWeakPtr& Sender, int32 BlockCounter)
	{
		FScopedUsdAllocs Allocs;

		UE_LOGF(
			LogUsd,
			Warning,
			"pxr::UsdNotice::ObjectsChanged from sender '%ls' (blocked? %d):",
			Sender ? UTF8_TO_TCHAR(Sender->GetRootLayer()->GetIdentifier().c_str()) : TEXT(""),
			BlockCounter > 0
		);

		UE_LOGF(LogUsd, Log, "\tInfoChanges:");
		LogObjectsChangedPathRange(Notice.GetChangedInfoOnlyPaths());

		UE_LOGF(LogUsd, Log, "\tResyncChanges:");
		LogObjectsChangedPathRange(Notice.GetResyncedPaths());

		UE_LOGF(LogUsd, Log, "\tResolvedAssetPaths:");
		LogObjectsChangedPathRange(Notice.GetResolvedAssetPathsResyncedPaths());
	}

	void LogNotice(const UsdUtils::FObjectChangesByPath& ConvertedInfoChanges, const UsdUtils::FObjectChangesByPath& ConvertedResyncChanges)
	{
		UE_LOGF(LogUsd, Warning, "Converted ObjectChange notice:");

		UE_LOGF(LogUsd, Log, "\tConverted InfoChanges:");
		LogConvertedChangesByPath(ConvertedInfoChanges);
		UE_LOGF(LogUsd, Log, "\tConverted ResyncChanges:");
		LogConvertedChangesByPath(ConvertedResyncChanges);
	}

	void LogNotice(const pxr::UsdNotice::StageEditTargetChanged& Notice, const pxr::UsdStageWeakPtr& Sender)
	{
		UE_LOGF(
			LogUsd,
			Warning,
			"pxr::UsdNotice::StageEditTargetChanged from sender '%ls':",
			Sender ? UTF8_TO_TCHAR(Sender->GetRootLayer()->GetIdentifier().c_str()) : TEXT("")
		);
	}

	void LogNotice(const pxr::SdfNotice::LayersDidChange& Notice, int32 BlockCounter)
	{
		FScopedUsdAllocs Allocs;

		UE_LOGF(LogUsd, Warning, "pxr::SdfNotice::LayersDidChange (blocked? %d)", BlockCounter > 0);

		UE_LOGF(LogUsd, Log, "\tSerial number: '%d'", Notice.GetSerialNumber());

		std::vector<std::pair<pxr::SdfLayerHandle, pxr::SdfChangeList>> ChangeListVec = Notice.GetChangeListVec();
		for (const std::pair<pxr::SdfLayerHandle, pxr::SdfChangeList>& Pair : ChangeListVec)
		{
			UE_LOGF(LogUsd, Log, "\tLayer: '%ls'", UTF8_TO_TCHAR(Pair.first->GetIdentifier().c_str()));

			for (const std::pair<pxr::SdfPath, pxr::SdfChangeList::Entry>& ObjectPair : Pair.second.GetEntryList())
			{
				UE_LOGF(LogUsd, Log, "\t\tObject: '%ls'", UTF8_TO_TCHAR(ObjectPair.first.GetString().c_str()));

				const int32 IndentLevel = 3;
				LogChangeListEntry(ObjectPair.second, IndentLevel);
			}
		}
	}

	void LogNotice(const pxr::SdfNotice::LayerDirtinessChanged& Notice, int32 BlockCounter)
	{
		UE_LOGF(LogUsd, Warning, "pxr::SdfNotice::LayerDirtinessChanged (blocked? %d)", BlockCounter > 0);
	}

	void LogNotice(const UsdUtils::FLayerToSdfChangeList& ConvertedLayerToChangeList)
	{
		FScopedUsdAllocs Allocs;

		UE_LOGF(LogUsd, Warning, "Converted LayerChanges:");

		for (const TPair<UE::FSdfLayerWeak, UsdUtils::FSdfChangeList>& ConvertedPair : ConvertedLayerToChangeList)
		{
			UE_LOGF(LogUsd, Log, "\tLayer: '%ls'", *ConvertedPair.Key.GetIdentifier());

			for (const TPair<UE::FSdfPath, UsdUtils::FSdfChangeListEntry>& EntryPair : ConvertedPair.Value)
			{
				UE_LOGF(LogUsd, Log, "\tObject: '%ls'", *EntryPair.Key.GetString());

				const int32 IndentLevel = 2;
				LogConvertedChangeListEntry(EntryPair.Value, IndentLevel);
			}
		}
	}
#endif	  // ENABLE_NOTICE_LOGGING

	void ConvertSdfChangeListEntry(
		const pxr::SdfPath& ChangedObject,
		const pxr::SdfChangeList::Entry& Entry,
		UsdUtils::FSdfChangeListEntry& OutConverted
	)
	{
		using namespace pxr;

		// For most changes we'll only get one of these, but sometimes multiple changes are fired in sequence
		// (e.g. if you change framesPerSecond, it will send a notice for it but also for the matching, updated timeCodesPerSecond)
		for (const std::pair<TfToken, std::pair<VtValue, VtValue>>& AttributeChange : Entry.infoChanged)	// Note: infoChanged here is just a naming
																											// conflict, there's no "resyncChanged"
		{
			UsdUtils::FAttributeChange& ConvertedAttributeChange = OutConverted.FieldChanges.Emplace_GetRef();
			ConvertedAttributeChange.Field = UTF8_TO_TCHAR(AttributeChange.first.GetString().c_str());
			ConvertedAttributeChange.OldValue = UE::FVtValue{AttributeChange.second.first};
			ConvertedAttributeChange.NewValue = UE::FVtValue{AttributeChange.second.second};
		}

		// Some notices (like creating/removing a property) don't have any actual infoChanged entries, so we create one in here for convenience
		if (Entry.infoChanged.size() == 0
			&& (Entry.flags.didAddProperty ||							  //
				Entry.flags.didAddPropertyWithOnlyRequiredFields ||		  //
				Entry.flags.didRemoveProperty ||						  //
				Entry.flags.didRemovePropertyWithOnlyRequiredFields ||	  //
				Entry.flags.didChangeAttributeTimeSamples))
		{
			UsdUtils::FAttributeChange& ConvertedAttributeChange = OutConverted.FieldChanges.Emplace_GetRef();
			ConvertedAttributeChange.Field = Entry.flags.didChangeAttributeTimeSamples ? TEXT("timeSamples") : TEXT("default");
		}

		// These should be packed just the same, but just in case we do member by member here instead of memcopying it over
		UsdUtils::FPrimChangeFlags& ConvertedFlags = OutConverted.Flags;
		ConvertedFlags.bDidChangeIdentifier = Entry.flags.didChangeIdentifier;
		ConvertedFlags.bDidChangeResolvedPath = Entry.flags.didChangeResolvedPath;
		ConvertedFlags.bDidReplaceContent = Entry.flags.didReplaceContent;
		ConvertedFlags.bDidReloadContent = Entry.flags.didReloadContent;
		ConvertedFlags.bDidReorderChildren = Entry.flags.didReorderChildren;
		ConvertedFlags.bDidReorderProperties = Entry.flags.didReorderProperties;
		ConvertedFlags.bDidRename = Entry.flags.didRename;
		ConvertedFlags.bDidChangePrimVariantSets = Entry.flags.didChangePrimVariantSets;
		ConvertedFlags.bDidChangePrimInheritPaths = Entry.flags.didChangePrimInheritPaths;
		ConvertedFlags.bDidChangePrimSpecializes = Entry.flags.didChangePrimSpecializes;
		ConvertedFlags.bDidChangePrimReferences = Entry.flags.didChangePrimReferences;
		ConvertedFlags.bDidChangeAttributeTimeSamples = Entry.flags.didChangeAttributeTimeSamples;
		ConvertedFlags.bDidChangeAttributeConnection = Entry.flags.didChangeAttributeConnection;
		ConvertedFlags.bDidChangeRelationshipTargets = Entry.flags.didChangeRelationshipTargets;
		ConvertedFlags.bDidAddTarget = Entry.flags.didAddTarget;
		ConvertedFlags.bDidRemoveTarget = Entry.flags.didRemoveTarget;
		ConvertedFlags.bDidAddInertPrim = Entry.flags.didAddInertPrim;
		ConvertedFlags.bDidAddNonInertPrim = Entry.flags.didAddNonInertPrim;
		ConvertedFlags.bDidRemoveInertPrim = Entry.flags.didRemoveInertPrim;
		ConvertedFlags.bDidRemoveNonInertPrim = Entry.flags.didRemoveNonInertPrim;
		ConvertedFlags.bDidAddPropertyWithOnlyRequiredFields = Entry.flags.didAddPropertyWithOnlyRequiredFields;
		ConvertedFlags.bDidAddProperty = Entry.flags.didAddProperty;
		ConvertedFlags.bDidRemovePropertyWithOnlyRequiredFields = Entry.flags.didRemovePropertyWithOnlyRequiredFields;
		ConvertedFlags.bDidRemoveProperty = Entry.flags.didRemoveProperty;

		static_assert(
			static_cast<int>(UsdUtils::ESubLayerChangeType::SubLayerAdded) == static_cast<int>(pxr::SdfChangeList::SubLayerChangeType::SubLayerAdded),
			"Enum values changed!"
		);
		static_assert(
			static_cast<int>(UsdUtils::ESubLayerChangeType::SubLayerOffset)
				== static_cast<int>(pxr::SdfChangeList::SubLayerChangeType::SubLayerOffset),
			"Enum values changed!"
		);
		static_assert(
			static_cast<int>(UsdUtils::ESubLayerChangeType::SubLayerRemoved)
				== static_cast<int>(pxr::SdfChangeList::SubLayerChangeType::SubLayerRemoved),
			"Enum values changed!"
		);
		for (const std::pair<std::string, pxr::SdfChangeList::SubLayerChangeType>& SubLayerChange : Entry.subLayerChanges)
		{
			OutConverted.SubLayerChanges.Add(TPair<FString, UsdUtils::ESubLayerChangeType>(
				UTF8_TO_TCHAR(SubLayerChange.first.c_str()),
				static_cast<UsdUtils::ESubLayerChangeType>(SubLayerChange.second)
			));
		}

		OutConverted.OldPath = UTF8_TO_TCHAR(Entry.oldPath.GetString().c_str());
		OutConverted.OldIdentifier = UTF8_TO_TCHAR(Entry.oldIdentifier.c_str());
	}

	bool ConvertPathRange(const pxr::UsdNotice::ObjectsChanged::PathRange& PathRange, UsdUtils::FObjectChangesByPath& OutChanges)
	{
		using namespace pxr;

		FScopedUsdAllocs UsdAllocs;

		OutChanges.Reset();

		for (UsdNotice::ObjectsChanged::PathRange::const_iterator It = PathRange.begin(); It != PathRange.end(); ++It)
		{
			if (pxr::UsdPrim::IsPathInPrototype(It->GetAbsoluteRootOrPrimPath()))
			{
				continue;
			}

			// This may be a prim path, but also just a property path in case we're changing a property value or its metadata
			const FString ObjectPath = UTF8_TO_TCHAR(It->GetAsString().c_str());
			TArray<UsdUtils::FSdfChangeListEntry>& ConvertedChanges = OutChanges.FindOrAdd(ObjectPath);

			// Changes may be empty, but we should still pass along this overall notice because sending a root
			// resync notice with no actual change item inside is how USD signals that a layer has been added/removed/resynced
			const std::vector<const SdfChangeList::Entry*>& Changes = It.base()->second;
			for (const SdfChangeList::Entry* Entry : Changes)
			{
				UsdUtils::FSdfChangeListEntry& ConvertedEntry = ConvertedChanges.Emplace_GetRef();
				if (Entry)
				{
					ConvertSdfChangeListEntry(*It, *Entry, ConvertedEntry);
				}
			}
		}

		return true;
	}

	bool ConvertObjectsChangedNotice(
		const pxr::UsdNotice::ObjectsChanged& InNotice,
		UsdUtils::FObjectChangesByPath& OutInfoChanges,
		UsdUtils::FObjectChangesByPath& OutResyncChanges
	)
	{
		ConvertPathRange(InNotice.GetChangedInfoOnlyPaths(), OutInfoChanges);

		// If we have a root path reload, just stop right here: We will have to reload everything anyway.
		// This is handy because otherwise on full reloads USD will emit an info change with bDidReloadContent=true
		// for most prims on the stage (this could be e.g. tens of thousands of entries, which our downstream code
		// would uselessly process, sort, serialize, etc.).
		const static FString RootPath = UE::FSdfPath::AbsoluteRootPath().GetString();
		if (TArray<UsdUtils::FSdfChangeListEntry>* RootInfoChange = OutInfoChanges.Find(RootPath))
		{
			for (const UsdUtils::FSdfChangeListEntry& Entry : *RootInfoChange)
			{
				if (Entry.Flags.bDidReloadContent)
				{
					OutResyncChanges = {};
					OutResyncChanges.Add(RootPath, *RootInfoChange);

					// Careful, as this will destroy our RootInfoChange/Entry
					OutInfoChanges = {};
					return true;
				}
			}
		}

		ConvertPathRange(InNotice.GetResyncedPaths(), OutResyncChanges);

		// Upgrade targetted reload notices into resyncs (this should now only happen when reloading a reference/payload, as reloading
		// any layer on the local layer stack emits a change for the root path)
		bool bReloadedContent = false;
		for (TPair<FString, TArray<UsdUtils::FSdfChangeListEntry>>& InfoPair : OutInfoChanges)
		{
			const FString& ObjectPath = InfoPair.Key;
			TArray<UsdUtils::FSdfChangeListEntry>* AnalogueResyncChanges = nullptr;

			UE::FSdfPath UEPath{*ObjectPath};
			FString ObjectName = UEPath.GetName();

			for (TArray<UsdUtils::FSdfChangeListEntry>::TIterator ChangeIt = InfoPair.Value.CreateIterator(); ChangeIt; ++ChangeIt)
			{
				bool bUpgrade = false;

				// Upgrade info changes with content reloads into resync changes
				if (ChangeIt->Flags.bDidReloadContent)
				{
					bUpgrade = true;
					bReloadedContent = true;
				}

				// Upgrade visibility changes to resyncs because in case of mesh collapsing having one of the collapsed meshes go visible/invisible
				// should cause the regeneration of the collapsed asset. This is a bit expensive, but the asset cache will be used so its not as if
				// the mesh will be completely regenerated however
				if (!bUpgrade)
				{
					for (UsdUtils::FAttributeChange& Change : ChangeIt->FieldChanges)
					{
						if (ObjectName == UTF8_TO_TCHAR(pxr::UsdGeomTokens->visibility.GetString().c_str()))
						{
							bUpgrade = true;
							break;
						}
					}
				}

				if (bUpgrade)
				{
					if (!AnalogueResyncChanges)
					{
						AnalogueResyncChanges = &OutResyncChanges.FindOrAdd(ObjectPath);
					}

					AnalogueResyncChanges->Add(*ChangeIt);
					ChangeIt.RemoveCurrent();
				}
			}
		}

		// For now, dump info changes when handling the notices about reloading layers.
		//
		// When we reload any layer, USD will emit some notices about attributes/prims that changed, and also generic notices about those prims
		// and their ancestors having reloaded their content (via the bDidReloadContent flag).
		// We'll upgrade those latter notices to resyncs (above), so that we regenerate the assets/components for those prims, but we cannot use the
		// former info changes about attributes/prims that changed (and how they changed) at all just yet, because that doesn't carry with them the
		// respective edit target information.
		//
		// For example imagine that a prim is only authored in a sublayer/referenced layer, is modified on disk, and the stage is reloaded.
		// We'll get the notice about what modification took place, but the stage's edit target will be the root layer. If we were to track those
		// changes, whenever we undid them we'd author the old values of the prim directly on the stage's root layer, which is definitely not what
		// we want.
		//
		// TODO: Remove this and use this information in order to have a "selective resync", updating only the changed prims/attributes's
		// assets/components. It *could* be possible, but it will be complex as it will involve deducing the right edit target via composition arcs,
		// and whether new attribute/prim specs where created within this transaction, etc.
		if (bReloadedContent)
		{
			OutInfoChanges = {};
		}

		return true;
	}
#endif	  // USE_USD_SDK
}	 // namespace UE::USDListener::Private

class FUsdListenerImpl
#if USE_USD_SDK
	: public pxr::TfWeakBase
#endif	  // #if USE_USD_SDK
{
public:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FUsdListenerImpl() = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	virtual ~FUsdListenerImpl();

	FUsdListener::FOnStageEditTargetChanged OnStageEditTargetChanged;
	FUsdListener::FOnSdfLayersChanged OnSdfLayersChanged;
	FUsdListener::FOnSdfLayerDirtinessChanged OnSdfLayerDirtinessChanged;
	FUsdListener::FOnObjectsChanged OnObjectsChanged;

	FThreadSafeCounter IsBlocked;

#if USE_USD_SDK
	FUsdListenerImpl(const pxr::UsdStageRefPtr& Stage);

	void Register(const pxr::UsdStageRefPtr& Stage);

protected:
	void HandleObjectsChangedNotice(const pxr::UsdNotice::ObjectsChanged& Notice, const pxr::UsdStageWeakPtr& Sender);
	void HandleStageEditTargetChangedNotice(const pxr::UsdNotice::StageEditTargetChanged& Notice, const pxr::UsdStageWeakPtr& Sender);
	void HandleLayersChangedNotice(const pxr::SdfNotice::LayersDidChange& Notice);
	void HandleLayerDirtinessChangedNotice(const pxr::SdfNotice::LayerDirtinessChanged& Notice);

private:
	pxr::TfNotice::Key RegisteredObjectsChangedKey;
	pxr::TfNotice::Key RegisteredStageEditTargetChangedKey;
	pxr::TfNotice::Key RegisteredSdfLayersChangedKey;
	pxr::TfNotice::Key RegisteredSdfLayerDirtinessChangedKey;
#endif	  // #if USE_USD_SDK
};

FUsdListener::FUsdListener()
	: Impl(MakeUnique<FUsdListenerImpl>())
{
}

FUsdListener::FUsdListener(const UE::FUsdStage& Stage)
	: FUsdListener()
{
	Register(Stage);
}

FUsdListener::~FUsdListener() = default;

void FUsdListener::Register(const UE::FUsdStage& Stage)
{
#if USE_USD_SDK
	Impl->Register(Stage);
#endif	  // #if USE_USD_SDK
}

void FUsdListener::Block()
{
	Impl->IsBlocked.Increment();
}

void FUsdListener::Unblock()
{
	Impl->IsBlocked.Decrement();
}

bool FUsdListener::IsBlocked() const
{
	return Impl->IsBlocked.GetValue() > 0;
}

FUsdListener::FOnStageEditTargetChanged& FUsdListener::GetOnStageEditTargetChanged()
{
	return Impl->OnStageEditTargetChanged;
}

FUsdListener::FOnSdfLayersChanged& FUsdListener::GetOnSdfLayersChanged()
{
	return Impl->OnSdfLayersChanged;
}

FUsdListener::FOnSdfLayerDirtinessChanged& FUsdListener::GetOnSdfLayerDirtinessChanged()
{
	return Impl->OnSdfLayerDirtinessChanged;
}

FUsdListener::FOnObjectsChanged& FUsdListener::GetOnObjectsChanged()
{
	return Impl->OnObjectsChanged;
}

#if USE_USD_SDK
void FUsdListenerImpl::Register(const pxr::UsdStageRefPtr& Stage)
{
	FScopedUsdAllocs UsdAllocs;

	if (RegisteredObjectsChangedKey.IsValid())
	{
		pxr::TfNotice::Revoke(RegisteredObjectsChangedKey);
	}
	RegisteredObjectsChangedKey = pxr::TfNotice::Register(
		pxr::TfWeakPtr<FUsdListenerImpl>(this),
		&FUsdListenerImpl::HandleObjectsChangedNotice,
		Stage
	);

	if (RegisteredStageEditTargetChangedKey.IsValid())
	{
		pxr::TfNotice::Revoke(RegisteredStageEditTargetChangedKey);
	}
	RegisteredStageEditTargetChangedKey = pxr::TfNotice::Register(
		pxr::TfWeakPtr<FUsdListenerImpl>(this),
		&FUsdListenerImpl::HandleStageEditTargetChangedNotice,
		Stage
	);

	if (RegisteredSdfLayersChangedKey.IsValid())
	{
		pxr::TfNotice::Revoke(RegisteredSdfLayersChangedKey);
	}
	RegisteredSdfLayersChangedKey = pxr::TfNotice::Register(pxr::TfWeakPtr<FUsdListenerImpl>(this), &FUsdListenerImpl::HandleLayersChangedNotice);

	if (RegisteredSdfLayerDirtinessChangedKey.IsValid())
	{
		pxr::TfNotice::Revoke(RegisteredSdfLayerDirtinessChangedKey);
	}
	RegisteredSdfLayerDirtinessChangedKey = pxr::TfNotice::Register(
		pxr::TfWeakPtr<FUsdListenerImpl>(this),
		&FUsdListenerImpl::HandleLayerDirtinessChangedNotice
	);
}
#endif	  // #if USE_USD_SDK

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FUsdListenerImpl::~FUsdListenerImpl()
{
#if USE_USD_SDK
	FScopedUsdAllocs UsdAllocs;
	pxr::TfNotice::Revoke(RegisteredObjectsChangedKey);
	pxr::TfNotice::Revoke(RegisteredStageEditTargetChangedKey);
	pxr::TfNotice::Revoke(RegisteredSdfLayersChangedKey);
	pxr::TfNotice::Revoke(RegisteredSdfLayerDirtinessChangedKey);
#endif	  // #if USE_USD_SDK
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if USE_USD_SDK
void FUsdListenerImpl::HandleObjectsChangedNotice(const pxr::UsdNotice::ObjectsChanged& Notice, const pxr::UsdStageWeakPtr& Sender)
{
#if ENABLE_NOTICE_LOGGING
	UE::USDListener::Private::LogNotice(Notice, Sender, IsBlocked.GetValue());
#endif	  // ENABLE_NOTICE_LOGGING

	if (!OnObjectsChanged.IsBound())
	{
		return;
	}

	if (IsBlocked.GetValue() > 0)
	{
		return;
	}

	UsdUtils::FObjectChangesByPath InfoChanges;
	UsdUtils::FObjectChangesByPath ResyncChanges;
	UE::USDListener::Private::ConvertObjectsChangedNotice(Notice, InfoChanges, ResyncChanges);
	if (InfoChanges.Num() > 0 || ResyncChanges.Num() > 0)
	{
		FScopedUnrealAllocs UnrealAllocs;
		OnObjectsChanged.Broadcast(InfoChanges, ResyncChanges);
	}

#if ENABLE_NOTICE_LOGGING
	UE::USDListener::Private::LogNotice(InfoChanges, ResyncChanges);
#endif	  // ENABLE_NOTICE_LOGGING
}

void FUsdListenerImpl::HandleStageEditTargetChangedNotice(const pxr::UsdNotice::StageEditTargetChanged& Notice, const pxr::UsdStageWeakPtr& Sender)
{
#if ENABLE_NOTICE_LOGGING
	UE::USDListener::Private::LogNotice(Notice, Sender);
#endif	  // ENABLE_NOTICE_LOGGING

	FScopedUnrealAllocs UnrealAllocs;
	OnStageEditTargetChanged.Broadcast();
}

void FUsdListenerImpl::HandleLayersChangedNotice(const pxr::SdfNotice::LayersDidChange& Notice)
{
#if ENABLE_NOTICE_LOGGING
	UE::USDListener::Private::LogNotice(Notice, IsBlocked.GetValue());
#endif	  // ENABLE_NOTICE_LOGGING

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (!OnSdfLayersChanged.IsBound() || IsBlocked.GetValue() > 0)
	{
		return;
	}

	TArray<FString> LayersNames;
	UsdUtils::FLayerToSdfChangeList ConvertedLayerToChangeList;
	{
		FScopedUnrealAllocs Allocs;

		const std::vector<std::pair<pxr::SdfLayerHandle, pxr::SdfChangeList>>& UsdChangeLists = Notice.GetChangeListVec();
		ConvertedLayerToChangeList.Reserve(UsdChangeLists.size());

		for (const std::pair<pxr::SdfLayerHandle, pxr::SdfChangeList>& ChangeVecItem : UsdChangeLists)
		{
			const pxr::SdfLayerHandle& Layer = ChangeVecItem.first;
			const pxr::SdfChangeList::EntryList& ChangeList = ChangeVecItem.second.GetEntryList();

			TPair<UE::FSdfLayerWeak, UsdUtils::FSdfChangeList>& ConvertedChangeList = ConvertedLayerToChangeList.Emplace_GetRef();
			ConvertedChangeList.Key = UE::FSdfLayerWeak{Layer};
			ConvertedChangeList.Value.Reserve(ChangeList.size());

			for (const std::pair<pxr::SdfPath, pxr::SdfChangeList::Entry>& Change : ChangeList)
			{
				TPair<UE::FSdfPath, UsdUtils::FSdfChangeListEntry>& ConvertedChange = ConvertedChangeList.Value.Emplace_GetRef();
				ConvertedChange.Key = UE::FSdfPath{Change.first};
				UE::USDListener::Private::ConvertSdfChangeListEntry(Change.first, Change.second, ConvertedChange.Value);

				const pxr::SdfChangeList::Entry::_Flags& Flags = Change.second.flags;
				if (Flags.didReloadContent)
				{
					LayersNames.Add(UTF8_TO_TCHAR(ChangeVecItem.first->GetIdentifier().c_str()));
				}
			}
		}
	}

#if ENABLE_NOTICE_LOGGING
	UE::USDListener::Private::LogNotice(ConvertedLayerToChangeList);
#endif	  // ENABLE_NOTICE_LOGGING

	FScopedUnrealAllocs UnrealAllocs;
	OnSdfLayersChanged.Broadcast(ConvertedLayerToChangeList);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FUsdListenerImpl::HandleLayerDirtinessChangedNotice(const pxr::SdfNotice::LayerDirtinessChanged& Notice)
{
#if ENABLE_NOTICE_LOGGING
	UE::USDListener::Private::LogNotice(Notice, IsBlocked.GetValue());
#endif	  // ENABLE_NOTICE_LOGGING

	if (!OnSdfLayerDirtinessChanged.IsBound() || IsBlocked.GetValue() > 0)
	{
		return;
	}

	FScopedUnrealAllocs UnrealAllocs;

	OnSdfLayerDirtinessChanged.Broadcast();
}
#endif	  // #if USE_USD_SDK

FScopedBlockNotices::FScopedBlockNotices(FUsdListener& InListener)
	: Listener(InListener)
{
	Listener.Block();
}

FScopedBlockNotices::~FScopedBlockNotices()
{
	Listener.Unblock();
}
