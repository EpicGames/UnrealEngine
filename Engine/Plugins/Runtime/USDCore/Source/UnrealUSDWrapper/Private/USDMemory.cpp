// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDMemory.h"

#include "Async/Async.h"
#include "Containers/Array.h"
#include "HAL/PlatformTLS.h"
#include "HAL/TlsAutoCleanup.h"
#include "Misc/ScopeLock.h"

LLM_DEFINE_TAG(Usd);

#if USE_USD_MEMORY_MANAGER

class UNREALUSDWRAPPER_API FTlsSlot final
{
public:
	static constexpr uint32 InvalidTlsSlotIndex = FPlatformTLS::InvalidTlsSlot;

	FTlsSlot();
	~FTlsSlot();

	FTlsSlot(FTlsSlot&& Other);
	FTlsSlot& operator=(FTlsSlot&& Other);

	// Can't copy a TLS slot
	FTlsSlot(const FTlsSlot& Other) = delete;
	FTlsSlot& operator=(const FTlsSlot& Other) = delete;

	void* GetTlsValue();
	const void* GetTlsValue() const;
	void SetTlsValue(void* Value);

private:
	uint32 SlotIndex;
};

FTlsSlot::FTlsSlot()
{
	SlotIndex = FPlatformTLS::AllocTlsSlot();
}

FTlsSlot::~FTlsSlot()
{
	if (FPlatformTLS::IsValidTlsSlot(SlotIndex))
	{
		FPlatformTLS::FreeTlsSlot(SlotIndex);
	}
}

FTlsSlot::FTlsSlot(FTlsSlot&& Other)
{
	SlotIndex = Other.SlotIndex;
	Other.SlotIndex = InvalidTlsSlotIndex;
}

FTlsSlot& FTlsSlot::operator=(FTlsSlot&& Other)
{
	if (FPlatformTLS::IsValidTlsSlot(SlotIndex))
	{
		FPlatformTLS::FreeTlsSlot(SlotIndex);
	}

	SlotIndex = Other.SlotIndex;
	Other.SlotIndex = InvalidTlsSlotIndex;

	return *this;
}

void* FTlsSlot::GetTlsValue()
{
	return FPlatformTLS::GetTlsValue(SlotIndex);
}

const void* FTlsSlot::GetTlsValue() const
{
	return FPlatformTLS::GetTlsValue(SlotIndex);
}

void FTlsSlot::SetTlsValue(void* Value)
{
	FPlatformTLS::SetTlsValue(SlotIndex, Value);
}

class FActiveAllocatorsStack : private FTlsAutoCleanup
{
public:
	TArray<EUsdActiveAllocator>& operator*()
	{
		return ActiveAllocatorsStack;
	}

	const TArray<EUsdActiveAllocator>& operator*() const
	{
		return ActiveAllocatorsStack;
	}

	TArray<EUsdActiveAllocator>* operator->()
	{
		return &ActiveAllocatorsStack;
	}

	const TArray<EUsdActiveAllocator>* operator->() const
	{
		return &ActiveAllocatorsStack;
	}

private:
	TArray<EUsdActiveAllocator> ActiveAllocatorsStack;
};

TOptional<FTlsSlot> FUsdMemoryManager::ActiveAllocatorsStackTLS{};
FUsdMemoryManager::FThreadSafeSet FUsdMemoryManager::SystemAllocedPtrs{};
FCriticalSection FUsdMemoryManager::CriticalSection{};

void FUsdMemoryManager::Initialize()
{
	// ActiveAllocatorsStackTLS is lazy init because it can be needed during CDO construction which happens before the call to Initialize()
}

void FUsdMemoryManager::Shutdown()
{
	ActiveAllocatorsStackTLS.Reset();
}

void FUsdMemoryManager::ActivateAllocator(EUsdActiveAllocator Allocator)
{
	FActiveAllocatorsStack& ActiveAllocatorStack = GetActiveAllocatorsStackForThread();
	ActiveAllocatorStack->Push(Allocator);
}

bool FUsdMemoryManager::DeactivateAllocator(EUsdActiveAllocator Allocator)
{
	FActiveAllocatorsStack& ActiveAllocatorStack = GetActiveAllocatorsStackForThread();

	// Remove the topmost instance of Allocator on the stack
	if (ActiveAllocatorStack->Num() > 0)
	{
		int32 AllocatorIndex = ActiveAllocatorStack->Num() - 1;
		while (AllocatorIndex >= 0)
		{
			if ((*ActiveAllocatorStack)[AllocatorIndex] == Allocator)
			{
				ActiveAllocatorStack->RemoveAt(AllocatorIndex);
				return true;
			}

			--AllocatorIndex;
		}
	}

	ensure(false);	  // The allocator that we are trying to deactivate wasn't on the allocator stack
	return false;
}

void* FUsdMemoryManager::Malloc(SIZE_T Count)
{
	LLM_SCOPE_BYTAG(Usd);

	void* Result = nullptr;

#if USD_USES_SYSTEM_MALLOC
	if (FUsdMemoryManager::IsUsingSystemMalloc())
	{
		Result = FMemory::SystemMalloc(Count);

		if (Result)
		{
			SystemAllocedPtrs.Add(Result);
		}
	}
	else
#endif	  // #if USD_USES_SYSTEM_MALLOC
	{
		Result = FMemory::Malloc(Count);
	}

	return Result;
}

void FUsdMemoryManager::Free(void* Original)
{
#if USD_USES_SYSTEM_MALLOC
	// Because USD is multi-threaded, it might call us back to free an object after we've exited our allocator scope.
	// This can happen for inlined USD functions that call delete.
	// Skip nullptr to avoid collision between threads for something that was not inserted in the first place.
	const bool bRemoved = Original != nullptr && SystemAllocedPtrs.Remove(Original);
	if (FUsdMemoryManager::IsUsingSystemMalloc() || bRemoved)
	{
		FMemory::SystemFree(Original);
	}
	else
#endif	  // #if USD_USES_SYSTEM_MALLOC
	{
		FMemory::Free(Original);
	}
}

// IsUsingSystemMalloc is used in operator new and operator delete, never call new and delete from it.
bool FUsdMemoryManager::IsUsingSystemMalloc()
{
	if (!ActiveAllocatorsStackTLS)	  // Not yet initialized
	{
		return false;
	}

	FActiveAllocatorsStack* ActiveAllocatorStack = reinterpret_cast<FActiveAllocatorsStack*>(ActiveAllocatorsStackTLS->GetTlsValue());

	if (ActiveAllocatorStack && (*ActiveAllocatorStack)->Num() > 0)
	{
		return (*ActiveAllocatorStack)->Top() == EUsdActiveAllocator::System;
	}
	else
	{
		return false;
	}
}

FActiveAllocatorsStack& FUsdMemoryManager::GetActiveAllocatorsStackForThread()
{
	if (!ActiveAllocatorsStackTLS)
	{
		ActiveAllocatorsStackTLS.Emplace();
	}

	FActiveAllocatorsStack* ActiveAllocatorStack = reinterpret_cast<FActiveAllocatorsStack*>(ActiveAllocatorsStackTLS->GetTlsValue());

	if (!ActiveAllocatorStack)
	{
		ActiveAllocatorStack = new FActiveAllocatorsStack();
		ActiveAllocatorsStackTLS->SetTlsValue((void*)ActiveAllocatorStack);
	}

	check(ActiveAllocatorStack);

	return *ActiveAllocatorStack;
}

#endif // USE_USD_MEMORY_MANAGER
