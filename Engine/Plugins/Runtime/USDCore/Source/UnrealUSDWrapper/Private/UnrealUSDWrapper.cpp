// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnrealUSDWrapper.h"

#include "USDClassesModule.h"
#include "USDLog.h"
#include "USDMaterialUtils.h"
#include "USDMemory.h"
#include "USDProjectSettings.h"
#include "USDReferenceOptions.h"

#include "UsdWrappers/SdfLayer.h"
#include "UsdWrappers/UsdAttribute.h"
#include "UsdWrappers/UsdStage.h"

#include "CineCameraComponent.h"
#include "Components/DirectionalLightComponent.h"
#include "Components/LightComponent.h"
#include "Components/LightComponentBase.h"
#include "Components/PointLightComponent.h"
#include "Components/RectLightComponent.h"
#include "Components/SpotLightComponent.h"
#include "GameFramework/Actor.h"
#include "Interfaces/IPluginManager.h"
#include "Internationalization/Regex.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UnrealUSDWrapper)

#define LOCTEXT_NAMESPACE "UnrealUSDWrapper"

#if USE_USD_SDK

#include "USDIncludesStart.h"

#if !USE_USD_MEMORY_MANAGER
#include "pxr/base/arch/memoryOverloads.h"
#endif

#include "pxr/base/gf/rotation.h"
#include "pxr/base/plug/plugin.h"
#include "pxr/base/plug/registry.h"
#include "pxr/base/tf/diagnosticMgr.h"
#include "pxr/base/tf/errorMark.h"
#include "pxr/base/tf/getenv.h"
#include "pxr/base/tf/setenv.h"
#include "pxr/usd/ar/defaultResolver.h"
#include "pxr/usd/ar/defineResolver.h"
#include "pxr/usd/kind/registry.h"
#include "pxr/usd/sdf/fileFormat.h"
#include "pxr/usd/sdf/path.h"
#include "pxr/usd/sdf/schema.h"
#include "pxr/usd/sdf/usdFileFormat.h"
#include "pxr/usd/sdf/usdaFileFormat.h"
#include "pxr/usd/sdf/usdcFileFormat.h"
#include "pxr/usd/sdf/usdzFileFormat.h"
#include "pxr/usd/usd/attribute.h"
#include "pxr/usd/usd/common.h"
#include "pxr/usd/usd/debugCodes.h"
#include "pxr/usd/usd/modelAPI.h"
#include "pxr/usd/usd/references.h"
#include "pxr/usd/usd/relationship.h"
#include "pxr/usd/usd/schemaBase.h"
#include "pxr/usd/usd/stage.h"
#include "pxr/usd/usd/stageCacheContext.h"
#include "pxr/usd/usd/variantSets.h"
#include "pxr/usd/usdGeom/mesh.h"
#include "pxr/usd/usdGeom/metrics.h"
#include "pxr/usd/usdGeom/modelAPI.h"
#include "pxr/usd/usdGeom/tokens.h"
#include "pxr/usd/usdGeom/xformCommonAPI.h"
#include "pxr/usd/usdShade/materialBindingAPI.h"
#include "pxr/usd/usdShade/tokens.h"
#include "pxr/usd/usdUtils/stageCache.h"

#include "USDIncludesEnd.h"

using std::string;
using std::vector;

using namespace pxr;

namespace UnrealIdentifiers
{
	static const TfToken AssetPath("unrealAssetPath");

	static const TfToken ActorClass("unrealActorClass");

	static const TfToken PropertyPath("unrealPropertyPath");

	/**
	 * Identifies the LOD variant set on a primitive which means this primitive has child prims that LOD meshes
	 * named LOD0, LOD1, LOD2, etc
	 */
	const TfToken LOD("LOD");
	const TfToken LodSubtreeAPI("LodSubtreeAPI");
	const TfToken UnrealLodSubtreeLevels = TfToken("unreal:lodSubtree:levels");
	
	const TfToken SubdivisionAPI("SubdivisionAPI");
	const TfToken SubdivisionLevel = TfToken("unreal:subdivisionLevel");

	const FString StaticMeshBuildSettingsAPI(TEXT("StaticMeshBuildSettingsAPI"));
	const FString NaniteBuildSettingsAPI(TEXT("NaniteBuildSettingsAPI"));

	const TfToken MaterialAssignment = TfToken("unrealMaterial");
	const TfToken Unreal = TfToken("unreal");

	const TfToken UnrealNaniteOverride = TfToken("unrealNanite");
	const TfToken UnrealNaniteOverrideEnable = TfToken("enable");
	const TfToken UnrealNaniteOverrideDisable = TfToken("disable");

	const TfToken LiveLinkAPI = TfToken("LiveLinkAPI");
	const TfToken ControlRigAPI = TfToken("ControlRigAPI");
	const TfToken UnrealAnimBlueprintPath = TfToken("unreal:liveLink:animBlueprintPath");
	const TfToken UnrealLiveLinkSubjectName = TfToken("unreal:liveLink:subjectName");
	const TfToken UnrealLiveLinkEnabled = TfToken("unreal:liveLink:enabled");
	const TfToken UnrealControlRigPath = TfToken("unreal:controlRig:controlRigPath");
	const TfToken UnrealUseFKControlRig = TfToken("unreal:controlRig:useFKControlRig");
	const TfToken UnrealControlRigReduceKeys = TfToken("unreal:controlRig:reduceKeys");
	const TfToken UnrealControlRigReductionTolerance = TfToken("unreal:controlRig:reductionTolerance");

	const TfToken SparseVolumeTextureAPI = TfToken("SparseVolumeTextureAPI");
	const TfToken UnrealSVTMappedFields = TfToken("unreal:SVT:mappedFields");
	const TfToken UnrealSVTMappedMaterialParameters = TfToken("unreal:SVT:mappedMaterialParameters");
	const TfToken UnrealSVTMappedGridComponents = TfToken("unreal:SVT:mappedGridComponents");
	const TfToken UnrealSVTMappedAttributeChannels = TfToken("unreal:SVT:mappedAttributeChannels");
	const TfToken UnrealSVTAttributesADataType = TfToken("unreal:SVT:attributesADataType");
	const TfToken UnrealSVTAttributesBDataType = TfToken("unreal:SVT:attributesBDataType");

	const TfToken DiffuseColor = TfToken("diffuseColor");
	const TfToken EmissiveColor = TfToken("emissiveColor");
	const TfToken Metallic = TfToken("metallic");
	const TfToken Roughness = TfToken("roughness");
	const TfToken Opacity = TfToken("opacity");
	const TfToken OpacityMode = TfToken("opacityMode");
	const TfToken OpacityThreshold = TfToken("opacityThreshold");
	const TfToken Normal = TfToken("normal");
	const TfToken Specular = TfToken("specular");
	const TfToken SpecularColor = TfToken("specularColor");
	const TfToken UseSpecularWorkflow = TfToken("useSpecularWorkflow");
	const TfToken ClearCoat = TfToken("clearcoat");
	const TfToken ClearCoatRoughness = TfToken("clearcoatRoughness");
	const TfToken Anisotropy = TfToken("anisotropy");
	const TfToken Tangent = TfToken("tangent");
	const TfToken SubsurfaceColor = TfToken("subsurfaceColor");
	const TfToken Displacement = TfToken("displacement");
	const TfToken Occlusion = TfToken("occlusion");
	const TfToken Refraction = TfToken("ior");

	const TfToken Xform = TfToken("Xform");

	const TfToken Surface = TfToken("surface");
	const TfToken St = TfToken("st");
	const TfToken Varname = TfToken("varname");
	const TfToken Scale = TfToken("scale");
	const TfToken Rotation = TfToken("rotation");
	const TfToken Translation = TfToken("translation");
	const TfToken In = TfToken("in");
	const TfToken Result = TfToken("result");
	const TfToken File = TfToken("file");
	const TfToken WrapT = TfToken("wrapT");
	const TfToken WrapS = TfToken("wrapS");
	const TfToken Repeat = TfToken("repeat");
	const TfToken Mirror = TfToken("mirror");
	const TfToken Clamp = TfToken("clamp");
	const TfToken Fallback = TfToken("fallback");
	const TfToken R = TfToken("r");
	const TfToken RGB = TfToken("rgb");
	const TfToken RawColorSpaceToken = TfToken{"raw"};
	const TfToken SRGBColorSpaceToken = TfToken{"sRGB"};
	const TfToken SourceColorSpaceToken = TfToken{"sourceColorSpace"};

	const TfToken UsdPreviewSurface = TfToken("UsdPreviewSurface");
	const TfToken UsdTransform2d = TfToken("UsdTransform2d");
	const TfToken UsdPrimvarReader_float2 = TfToken("UsdPrimvarReader_float2");
	const TfToken UsdPrimvarReader_float3 = TfToken("UsdPrimvarReader_float3");
	const TfToken UsdUVTexture = TfToken("UsdUVTexture");

	const TfToken WorldSpaceNormals = TfToken("worldSpaceNormals");
	const TfToken PrimvarsNormals = TfToken("primvars:normals");
	const TfToken PrimvarsPoints = TfToken("primvars:points");

	const TfToken GroomAPI = TfToken("GroomAPI");
	const TfToken GroomBindingAPI = TfToken("GroomBindingAPI");
	const TfToken UnrealGroomToBind = TfToken("unreal:groomBinding:groom");
	const TfToken UnrealGroomReferenceMesh = TfToken("unreal:groomBinding:referenceMesh");

	const TfToken UnrealContentPath = TfToken("unreal:contentPath");
	const TfToken UnrealAssetType = TfToken("unreal:assetType");
	const TfToken UnrealExportTime = TfToken("unreal:exportTime");
	const TfToken UnrealEngineVersion = TfToken("unreal:engineVersion");

	const TfToken UnrealCollapsingAPI = TfToken("CollapsingAPI");
	const TfToken UnrealCollapsingAttr = TfToken("unreal:collapsing");
	const TfToken CollapsingAllow = TfToken("allow");
	const TfToken CollapsingDefault = TfToken("default");
	const TfToken CollapsingNever = TfToken("never");

	const TfToken NaniteAssemblyRootAPI = TfToken("NaniteAssemblyRootAPI");
	const TfToken NaniteAssemblyExternalRefAPI = TfToken("NaniteAssemblyExternalRefAPI");
	const TfToken NaniteAssemblySkelBindingAPI = TfToken("NaniteAssemblySkelBindingAPI");
	const TfToken UnrealNaniteAssemblyMeshType = TfToken("unreal:naniteAssembly:meshType");
	const TfToken UnrealNaniteAssemblyMeshAssetPath = TfToken("unreal:naniteAssembly:meshAssetPath");
	const TfToken UnrealNaniteAssemblySkeleton = TfToken("unreal:naniteAssembly:skeleton");
	const TfToken PrimvarsUnrealNaniteAssemblyBindJoints = TfToken("primvars:unreal:naniteAssembly:bindJoints");
	const TfToken PrimvarsUnrealNaniteAssemblyBindJointWeights = TfToken("primvars:unreal:naniteAssembly:bindJointWeights");
	const TfToken NaniteAssemblyStaticMesh = TfToken("staticMesh");
	const TfToken NaniteAssemblySkeletalMesh = TfToken("skeletalMesh");

}	 // namespace UnrealIdentifiers

bool IUsdPrim::IsValidPrimName(const FString& Name, FText& OutReason)
{
	if (Name.IsEmpty())
	{
		OutReason = LOCTEXT("EmptyStringInvalid", "Empty string is not a valid name!");
		return false;
	}

	const FString InvalidCharacters = TEXT("\\W");
	FRegexPattern RegexPattern(InvalidCharacters);
	FRegexMatcher RegexMatcher(RegexPattern, Name);
	if (RegexMatcher.FindNext())
	{
		OutReason = LOCTEXT("InvalidCharacter", "Can only use letters, numbers and underscore!");
		return false;
	}

	if (Name.Left(1).IsNumeric())
	{
		OutReason = LOCTEXT("InvalidFirstCharacter", "First character cannot be a number!");
		return false;
	}

	return true;
}

EUsdPurpose IUsdPrim::GetPurpose(const UsdPrim& Prim, bool bComputed)
{
	UsdGeomImageable Geom(Prim);
	if (Geom)
	{
		TfToken Purpose;
		if (bComputed)
		{
			Purpose = Geom.ComputePurpose();
		}
		else
		{
			pxr::UsdAttribute PurposeAttr = Prim.GetAttribute(pxr::UsdGeomTokens->purpose);

			pxr::VtValue Value;
			PurposeAttr.Get(&Value);

			Purpose = Value.Get<pxr::TfToken>();
		}

		if (Purpose == pxr::UsdGeomTokens->proxy)
		{
			return EUsdPurpose::Proxy;
		}
		else if (Purpose == pxr::UsdGeomTokens->render)
		{
			return EUsdPurpose::Render;
		}
		else if (Purpose == pxr::UsdGeomTokens->guide)
		{
			return EUsdPurpose::Guide;
		}
	}

	return EUsdPurpose::Default;
}

bool IUsdPrim::HasGeometryData(const UsdPrim& Prim)
{
	return UsdGeomMesh(Prim) ? true : false;
}

bool IUsdPrim::HasGeometryDataOrLODVariants(const UsdPrim& Prim)
{
	return HasGeometryData(Prim) || GetNumLODs(Prim) > 0;
}

int IUsdPrim::GetNumLODs(const UsdPrim& Prim)
{
	FScopedUsdAllocs UsdAllocs;

	// 0 indicates no variant or no lods in variant.
	int NumLODs = 0;
	if (Prim.HasVariantSets())
	{
		UsdVariantSet LODVariantSet = Prim.GetVariantSet(UnrealIdentifiers::LOD);
		if (LODVariantSet.IsValid())
		{
			vector<string> VariantNames = LODVariantSet.GetVariantNames();
			NumLODs = VariantNames.size();
		}
	}

	return NumLODs;
}

bool IUsdPrim::IsKindChildOf(const UsdPrim& Prim, const std::string& InBaseKind)
{
	TfToken BaseKind(InBaseKind);

	KindRegistry& Registry = KindRegistry::GetInstance();

	TfToken PrimKind(GetKind(Prim));

	return Registry.IsA(PrimKind, BaseKind);
}

TfToken IUsdPrim::GetKind(const pxr::UsdPrim& Prim)
{
	TfToken KindType;

	UsdModelAPI Model(Prim);
	if (Model)
	{
		Model.GetKind(&KindType);
	}
	else
	{
		// Prim is not a model, read kind directly from metadata
		Prim.GetMetadata(SdfFieldKeys->Kind, &KindType);
	}

	return KindType;
}

bool IUsdPrim::SetKind(const pxr::UsdPrim& Prim, const pxr::TfToken& Kind)
{
	UsdModelAPI Model(Prim);
	if (Model)
	{
		if (!Model.SetKind(Kind))
		{
			return Prim.SetMetadata(SdfFieldKeys->Kind, Kind);
		}

		return true;
	}

	return false;
}

bool IUsdPrim::ClearKind(const pxr::UsdPrim& Prim)
{
	return Prim.ClearMetadata(SdfFieldKeys->Kind);
}

pxr::GfMatrix4d IUsdPrim::GetLocalTransform(const pxr::UsdPrim& Prim)
{
	pxr::GfMatrix4d USDMatrix(1);

	pxr::UsdGeomXformable XForm(Prim);
	if (XForm)
	{
		// Set transform
		bool bResetXFormStack = false;
		XForm.GetLocalTransformation(&USDMatrix, &bResetXFormStack);
	}

	return USDMatrix;
}

pxr::GfMatrix4d IUsdPrim::GetLocalToWorldTransform(const pxr::UsdPrim& Prim)
{
	return GetLocalToWorldTransform(Prim, pxr::UsdTimeCode::Default().GetValue());
}

pxr::GfMatrix4d IUsdPrim::GetLocalToWorldTransform(const pxr::UsdPrim& Prim, double Time)
{
	pxr::SdfPath AbsoluteRootPath = pxr::SdfPath::AbsoluteRootPath();
	return GetLocalToWorldTransform(Prim, Time, AbsoluteRootPath);
}

pxr::GfMatrix4d IUsdPrim::GetLocalToWorldTransform(const pxr::UsdPrim& Prim, double Time, const pxr::SdfPath& AbsoluteRootPath)
{
	pxr::SdfPath PrimPath = Prim.GetPath();
	if (!Prim || PrimPath == AbsoluteRootPath)
	{
		return pxr::GfMatrix4d(1);
	}

	pxr::GfMatrix4d AccumulatedTransform(1.);
	bool bResetsXFormStack = false;
	pxr::UsdGeomXformable XFormable(Prim);
	// silently ignoring errors
	XFormable.GetLocalTransformation(&AccumulatedTransform, &bResetsXFormStack, Time);

	if (!bResetsXFormStack)
	{
		AccumulatedTransform = AccumulatedTransform * GetLocalToWorldTransform(Prim.GetParent(), Time, AbsoluteRootPath);
	}

	return AccumulatedTransform;
}

bool IUsdPrim::HasTransform(const pxr::UsdPrim& Prim)
{
	return UsdGeomXformable(Prim) ? true : false;
}

bool IUsdPrim::SetActiveLODIndex(const pxr::UsdPrim& Prim, int LODIndex)
{
	FScopedUsdAllocs UsdAllocs;

	if (Prim.HasVariantSets())
	{
		UsdVariantSet LODVariantSet = Prim.GetVariantSet(UnrealIdentifiers::LOD);
		if (LODVariantSet.IsValid())
		{
			vector<string> VariantNames = LODVariantSet.GetVariantNames();

			if (LODIndex < VariantNames.size())
			{
				return LODVariantSet.SetVariantSelection(VariantNames[LODIndex]);
			}
		}
	}

	return false;
}

EUsdGeomOrientation IUsdPrim::GetGeometryOrientation(const pxr::UsdGeomMesh& Mesh)
{
	return GetGeometryOrientation(Mesh, pxr::UsdTimeCode::Default().GetValue());
}

EUsdGeomOrientation IUsdPrim::GetGeometryOrientation(const pxr::UsdGeomMesh& Mesh, double Time)
{
	EUsdGeomOrientation GeomOrientation = EUsdGeomOrientation::RightHanded;

	if (Mesh)
	{
		UsdAttribute Orientation = Mesh.GetOrientationAttr();
		if (Orientation)
		{
			static TfToken RightHanded("rightHanded");
			static TfToken LeftHanded("leftHanded");

			TfToken OrientationValue;
			Orientation.Get(&OrientationValue, Time);

			GeomOrientation = OrientationValue == LeftHanded ? EUsdGeomOrientation::LeftHanded : EUsdGeomOrientation::RightHanded;
		}
	}

	return GeomOrientation;
}
#endif	  // USE_USD_SDK

const TCHAR* UnrealIdentifiers::LayerSavedComment = TEXT("unreal:layerSaved");
const TCHAR* UnrealIdentifiers::TwoSidedMaterialSuffix = TEXT("_TwoSided");

const TCHAR* UnrealIdentifiers::Invisible = TEXT("invisible");
const TCHAR* UnrealIdentifiers::Inherited = TEXT("inherited");
const TCHAR* UnrealIdentifiers::IdentifierPrefix = TEXT("@identifier:");
const TCHAR* UnrealIdentifiers::ExportedSkeletonPrimName = TEXT("Skel");

FName UnrealIdentifiers::TransformPropertyName = TEXT("Transform");	   // Fake FName because the transform is stored decomposed on the component
FName UnrealIdentifiers::HiddenInGamePropertyName = GET_MEMBER_NAME_CHECKED(USceneComponent, bHiddenInGame);
FName UnrealIdentifiers::HiddenPropertyName = AActor::GetHiddenPropertyName();

FName UnrealIdentifiers::AspectRatioPropertyName = GET_MEMBER_NAME_CHECKED(UCameraComponent, AspectRatio);
FName UnrealIdentifiers::OrthoWidthPropertyName = GET_MEMBER_NAME_CHECKED(UCameraComponent, OrthoWidth);
FName UnrealIdentifiers::OrthoFarClipPlanePropertyName = GET_MEMBER_NAME_CHECKED(UCameraComponent, OrthoFarClipPlane);
FName UnrealIdentifiers::OrthoNearClipPlanePropertyName = GET_MEMBER_NAME_CHECKED(UCameraComponent, OrthoNearClipPlane);

FName UnrealIdentifiers::CurrentFocalLengthPropertyName = GET_MEMBER_NAME_CHECKED(UCineCameraComponent, CurrentFocalLength);
FName UnrealIdentifiers::ManualFocusDistancePropertyName = GET_MEMBER_NAME_CHECKED(UCineCameraComponent, FocusSettings.ManualFocusDistance);
FName UnrealIdentifiers::CurrentAperturePropertyName = GET_MEMBER_NAME_CHECKED(UCineCameraComponent, CurrentAperture);
FName UnrealIdentifiers::SensorWidthPropertyName = GET_MEMBER_NAME_CHECKED(UCineCameraComponent, Filmback.SensorWidth);
FName UnrealIdentifiers::SensorHeightPropertyName = GET_MEMBER_NAME_CHECKED(UCineCameraComponent, Filmback.SensorHeight);
FName UnrealIdentifiers::SensorHorizontalOffsetPropertyName = GET_MEMBER_NAME_CHECKED(UCineCameraComponent, Filmback.SensorHorizontalOffset);
FName UnrealIdentifiers::SensorVerticalOffsetPropertyName = GET_MEMBER_NAME_CHECKED(UCineCameraComponent, Filmback.SensorVerticalOffset);
FName UnrealIdentifiers::ExposureCompensationPropertyName = GET_MEMBER_NAME_CHECKED(UCineCameraComponent, PostProcessSettings.AutoExposureBias);
FName UnrealIdentifiers::ProjectionModePropertyName = GET_MEMBER_NAME_CHECKED(UCineCameraComponent, ProjectionMode);
FName UnrealIdentifiers::CustomNearClipppingPlanePropertyName = GET_MEMBER_NAME_CHECKED(UCineCameraComponent, CustomNearClippingPlane);
FName UnrealIdentifiers::DepthOfFieldFstopPropertyName = GET_MEMBER_NAME_CHECKED(UCineCameraComponent, PostProcessSettings.DepthOfFieldFstop);
FName UnrealIdentifiers::CameraISOPropertyName = GET_MEMBER_NAME_CHECKED(UCineCameraComponent, PostProcessSettings.CameraISO);
FName UnrealIdentifiers::CameraShutterSpeedPropertyName = GET_MEMBER_NAME_CHECKED(UCineCameraComponent, PostProcessSettings.CameraShutterSpeed);

FName UnrealIdentifiers::IntensityPropertyName = GET_MEMBER_NAME_CHECKED(ULightComponentBase, Intensity);
FName UnrealIdentifiers::LightColorPropertyName = GET_MEMBER_NAME_CHECKED(ULightComponentBase, LightColor);
FName UnrealIdentifiers::UseTemperaturePropertyName = GET_MEMBER_NAME_CHECKED(ULightComponent, bUseTemperature);
FName UnrealIdentifiers::TemperaturePropertyName = GET_MEMBER_NAME_CHECKED(ULightComponent, Temperature);
FName UnrealIdentifiers::SourceWidthPropertyName = GET_MEMBER_NAME_CHECKED(URectLightComponent, SourceWidth);
FName UnrealIdentifiers::SourceHeightPropertyName = GET_MEMBER_NAME_CHECKED(URectLightComponent, SourceHeight);
FName UnrealIdentifiers::SourceRadiusPropertyName = GET_MEMBER_NAME_CHECKED(UPointLightComponent, SourceRadius);
FName UnrealIdentifiers::OuterConeAnglePropertyName = GET_MEMBER_NAME_CHECKED(USpotLightComponent, OuterConeAngle);
FName UnrealIdentifiers::InnerConeAnglePropertyName = GET_MEMBER_NAME_CHECKED(USpotLightComponent, InnerConeAngle);
FName UnrealIdentifiers::LightSourceAnglePropertyName = GET_MEMBER_NAME_CHECKED(UDirectionalLightComponent, LightSourceAngle);

FString UnrealIdentifiers::MaterialAllPurposeText = TEXT("allPurpose");
#if USE_USD_SDK
FString UnrealIdentifiers::MaterialAllPurpose = UTF8_TO_TCHAR(pxr::UsdShadeTokens->allPurpose.GetString().c_str());
FString UnrealIdentifiers::MaterialPreviewPurpose = UTF8_TO_TCHAR(pxr::UsdShadeTokens->preview.GetString().c_str());
FString UnrealIdentifiers::MaterialFullPurpose = UTF8_TO_TCHAR(pxr::UsdShadeTokens->full.GetString().c_str());
FString UnrealIdentifiers::PrimvarsDisplayColor = UTF8_TO_TCHAR(pxr::UsdGeomTokens->primvarsDisplayColor.GetString().c_str());
FString UnrealIdentifiers::PrimvarsDisplayOpacity = UTF8_TO_TCHAR(pxr::UsdGeomTokens->primvarsDisplayOpacity.GetString().c_str());
FString UnrealIdentifiers::DoubleSided = UTF8_TO_TCHAR(pxr::UsdGeomTokens->doubleSided.GetString().c_str());
FString UnrealIdentifiers::ModelDrawMode = UTF8_TO_TCHAR(pxr::UsdGeomTokens->modelDrawMode.GetString().c_str());
FString UnrealIdentifiers::ModelApplyDrawMode = UTF8_TO_TCHAR(pxr::UsdGeomTokens->modelApplyDrawMode.GetString().c_str());
FString UnrealIdentifiers::UsdNamespaceDelimiter = UTF8_TO_TCHAR(pxr::SdfPathTokens->namespaceDelimiter.GetString().c_str());
#else
FString UnrealIdentifiers::MaterialAllPurpose = TEXT("");
FString UnrealIdentifiers::MaterialPreviewPurpose = TEXT("preview");
FString UnrealIdentifiers::MaterialFullPurpose = TEXT("full");
FString UnrealIdentifiers::PrimvarsDisplayColor = TEXT("primvars:displayColor");
FString UnrealIdentifiers::PrimvarsDisplayOpacity = TEXT("primvars:displayOpacity");
FString UnrealIdentifiers::DoubleSided = TEXT("doubleSided");
FString UnrealIdentifiers::ModelDrawMode = TEXT("model:drawMode");
FString UnrealIdentifiers::ModelApplyDrawMode = TEXT("model:applyDrawMode");
FString UnrealIdentifiers::UsdNamespaceDelimiter = TEXT(":");
#endif	  // USE_USD_SDK

#if USE_USD_SDK
// This is largely for show, as the universalRenderContext is an empty string, and FNames created from empty strings equal NAME_None anyway
// i.e.: NAME_None == FName(TEXT("")) == FName(TEXT("None")) == UnrealIdentifiers::UniversalRenderContext
const FName UnrealIdentifiers::UniversalRenderContext = UTF8_TO_TCHAR(pxr::UsdShadeTokens->universalRenderContext.GetString().c_str());
#else
const FName UnrealIdentifiers::UniversalRenderContext = NAME_None;
#endif
const FName UnrealIdentifiers::UnrealRenderContext = TEXT("unreal");
const FName UnrealIdentifiers::MaterialXRenderContext = TEXT("mtlx");
const FName UnrealIdentifiers::MdlRenderContext = TEXT("mdl");

const FString UnrealIdentifiers::UniversalRenderContextDisplayString = TEXT("universal");

const FName UnrealIdentifiers::PointInstancerTag = TEXT("PointInstancer");

FUsdDelegates::FUsdImportDelegate FUsdDelegates::OnPreUsdImport;
FUsdDelegates::FUsdImportDelegate FUsdDelegates::OnPostUsdImport;

namespace UsdWrapperUtils
{
	void CheckIfForceDisabled()
	{
#if USD_FORCE_DISABLED
		UE_LOGF(
			LogUsd,
			Error,
			"The USD SDK is disabled because the executable is not forcing the ansi C allocator (you need to set 'FORCE_ANSI_ALLOCATOR=1' as a "
				 "global definition on your project *.Target.cs file). Read the comments at the end of UnrealUSDWrapper.Build.cs for more details."
		);
#endif	  // USD_FORCE_DISABLED
	}
}

FSdfTimeCode::FSdfTimeCode(double InTime)
	: Time(InTime)
{
}

FSdfTimeCode::operator double() const
{
	return Time;
}

FSdfAssetPath::FSdfAssetPath(const FString& InAssetPath, const FString& InResolvedPath)
	: AssetPath(InAssetPath)
	, ResolvedPath(InResolvedPath)
{
}

FSdfAssetPath::operator FString() const
{
	return AssetPath;
}

FQuat4h::FQuat4h(float InX, float InY, float InZ, float InW)
	: X(InX)
	, Y(InY)
	, Z(InZ)
	, W(InW)
{
}

FQuat4h::FQuat4h(const FQuat4f& InQuat)
	: X(InQuat.X)
	, Y(InQuat.Y)
	, Z(InQuat.Z)
	, W(InQuat.W)
{
}

FQuat4h::operator FQuat4f() const
{
	return FQuat4f{X, Y, Z, W};
}

FVector3h::FVector3h(float InX, float InY, float InZ)
	: X(InX)
	, Y(InY)
	, Z(InZ)
{
}

FVector3h::FVector3h(const FVector3f& InVector)
	: X(InVector.X)
	, Y(InVector.Y)
	, Z(InVector.Z)
{
}

FVector3h::operator FVector3f() const
{
	return FVector3f{X, Y, Z};
}

FVector4h::FVector4h(float InX, float InY, float InZ, float InW)
	: X(InX)
	, Y(InY)
	, Z(InZ)
	, W(InW)
{
}

FVector4h::FVector4h(const FVector4f& InVector)
	: X(InVector.X)
	, Y(InVector.Y)
	, Z(InVector.Z)
	, W(InVector.W)
{
}

FVector4h::operator FVector4f() const
{
	return FVector4f{X, Y, Z, W};
}

#if USE_USD_SDK
double UnrealUSDWrapper::GetDefaultTimeCode()
{
	return UsdTimeCode::Default().GetValue();
}
#endif	  // USE_USD_SDK

namespace UE::UnrealUSDWrapper::Private
{
	void EnsureModuleIsLoaded()
	{
		FModuleManager& Manager = FModuleManager::Get();

		const FName ModuleName = TEXT("UnrealUSDWrapper");
		if (!Manager.IsModuleLoaded(ModuleName))
		{
			UE_LOGF(
				LogUsd,
				Log,
				"Attempted to call a static function from UnrealUSDWrapper before the module is actually loaded! The module will be loaded now, but not all static functions have this check. In general, please ensure an Unreal module is loaded before calling any of its static functions, for example by calling 'FModuleManager::LoadModuleChecked<IUnrealUSDWrapperModule>(\"UnrealUSDWrapper\");' beforehand."
			);

			IModuleInterface* LoadedModule = Manager.LoadModule(ModuleName);
			if (!LoadedModule)
			{
				UE_LOGF(LogUsd, Warning, "Failed to load the UnrealUSDWrapper module!");
			}
		}
	}
};

TArray<FString> UnrealUSDWrapper::RegisterPlugins(const FString& PathToPlugInfo) TSAN_SAFE
{
	return RegisterPlugins(TArray<FString>{PathToPlugInfo});
}

TArray<FString> UnrealUSDWrapper::RegisterPlugins(const TArray<FString>& PathsToPlugInfo) TSAN_SAFE
{
	UE::UnrealUSDWrapper::Private::EnsureModuleIsLoaded();

	TArray<FString> PluginNames;

#if USE_USD_SDK
	FScopedUsdAllocs Allocs;

	std::vector<std::string> UsdPathsToPlugInfo;
	UsdPathsToPlugInfo.reserve(PathsToPlugInfo.Num());

	for (const FString& PathToPlugInfo : PathsToPlugInfo)
	{
		UsdPathsToPlugInfo.emplace_back(TCHAR_TO_UTF8(*PathToPlugInfo));
	}

	const pxr::PlugPluginPtrVector UsdPlugins = pxr::PlugRegistry::GetInstance().RegisterPlugins(UsdPathsToPlugInfo);

	for (const pxr::PlugPluginPtr& UsdPlugin : UsdPlugins)
	{
		PluginNames.Emplace(UTF8_TO_TCHAR(UsdPlugin->GetName().c_str()));
	}
#endif	  // USE_USD_SDK

	return PluginNames;
}

TArray<FString> UnrealUSDWrapper::GetAllSupportedFileFormats() TSAN_SAFE
{
	UE::UnrealUSDWrapper::Private::EnsureModuleIsLoaded();

	TArray<FString> Result;

#if USE_USD_SDK
	FScopedUsdAllocs Allocs;

	std::set<std::string> Extensions = pxr::SdfFileFormat::FindAllFileFormatExtensions();
	for (const std::string& Ext : Extensions)
	{
		// Ignore formats that don't target "usd"
		pxr::SdfFileFormatConstPtr Format = pxr::SdfFileFormat::FindByExtension(Ext, pxr::SdfUsdFileFormatTokens->Target);
		if (Format == nullptr)
		{
			continue;
		}

		Result.Emplace(UTF8_TO_TCHAR(Ext.c_str()));
	}
#endif	  // #if USE_USD_SDK

	return Result;
}

TArray<FString> UnrealUSDWrapper::GetNativeFileFormats() TSAN_SAFE
{
	UE::UnrealUSDWrapper::Private::EnsureModuleIsLoaded();

	TArray<FString> Result;

#if USE_USD_SDK
	FScopedUsdAllocs Allocs;

	const pxr::TfTokenVector NativeFormatIds{
		pxr::SdfUsdFileFormatTokens->Id,
		pxr::SdfUsdaFileFormatTokens->Id,
		pxr::SdfUsdcFileFormatTokens->Id,
		pxr::SdfUsdzFileFormatTokens->Id
	};

	std::set<std::string> FileExtensions;

	for (const pxr::TfToken& FormatId : NativeFormatIds)
	{
		const pxr::SdfFileFormatConstPtr Format = pxr::SdfFileFormat::FindById(FormatId);
		if (!Format)
		{
			continue;
		}

		for (const std::string& FileExtension : Format->GetFileExtensions())
		{
			FileExtensions.insert(FileExtension);
		}
	}

	for (const std::string& FileExtension : FileExtensions)
	{
		Result.Emplace(UTF8_TO_TCHAR(FileExtension.c_str()));
	}
#endif	  // #if USE_USD_SDK

	return Result;
}

void UnrealUSDWrapper::GetNativeFileFormats(TArray<FString>& OutTextFormats, TArray<FString>& OutPossiblyBinaryFormats) TSAN_SAFE
{
	UE::UnrealUSDWrapper::Private::EnsureModuleIsLoaded();

#if USE_USD_SDK
	FScopedUsdAllocs Allocs;

	std::set<std::string> TextExtensions;
	const pxr::SdfFileFormatConstPtr TextFormat = pxr::SdfFileFormat::FindById(pxr::SdfUsdaFileFormatTokens->Id);
	if (TextFormat)
	{
		for (const std::string& FileExtension : TextFormat->GetFileExtensions())
		{
			TextExtensions.insert(FileExtension);
		}
	}

	OutTextFormats.Reset(TextExtensions.size());
	for (const std::string& Extension : TextExtensions)
	{
		OutTextFormats.Emplace(UTF8_TO_TCHAR(Extension.c_str()));
	}

	const static std::vector<pxr::TfToken> BinaryFormatIds{
		pxr::SdfUsdFileFormatTokens->Id,
		pxr::SdfUsdcFileFormatTokens->Id,
		pxr::SdfUsdzFileFormatTokens->Id
	};

	std::set<std::string> PossiblyBinaryExtensions;
	for (const pxr::TfToken& FormatId : BinaryFormatIds)
	{
		const pxr::SdfFileFormatConstPtr Format = pxr::SdfFileFormat::FindById(FormatId);
		if (!Format)
		{
			continue;
		}

		for (const std::string& FileExtension : Format->GetFileExtensions())
		{
			PossiblyBinaryExtensions.insert(FileExtension);
		}
	}

	OutPossiblyBinaryFormats.Reset(PossiblyBinaryExtensions.size());
	for (const std::string& Extension : PossiblyBinaryExtensions)
	{
		OutPossiblyBinaryFormats.Emplace(UTF8_TO_TCHAR(Extension.c_str()));
	}
#endif	  // #if USE_USD_SDK
}

void UnrealUSDWrapper::AddUsdExportFileFormatDescriptions(TArray<FString>& OutFormatExtensions, TArray<FString>& OutFormatDescriptions)
{
	UE::UnrealUSDWrapper::Private::EnsureModuleIsLoaded();

	TArray<FString> TextExtensions;
	TArray<FString> PossiblyBinaryExtensions;
	UnrealUSDWrapper::GetNativeFileFormats(TextExtensions, PossiblyBinaryExtensions);

	for (const FString& Extension : PossiblyBinaryExtensions)
	{
		// USDZ is not supported for writing for now
		if (Extension.Equals(TEXT("usdz")))
		{
			continue;
		}

		OutFormatExtensions.Add(Extension);
		OutFormatDescriptions.Add(TEXT("Universal Scene Description binary file"));
	}

	for (const FString& Extension : TextExtensions)
	{
		OutFormatExtensions.Add(Extension);
		OutFormatDescriptions.Add(TEXT("Universal Scene Description text file"));
	}
}

void UnrealUSDWrapper::AddUsdImportFileFormatDescriptions(TArray<FString>& OutFormats)
{
	UE::UnrealUSDWrapper::Private::EnsureModuleIsLoaded();

	TArray<FString> TextExtensions;
	TArray<FString> PossiblyBinaryExtensions;
	UnrealUSDWrapper::GetNativeFileFormats(TextExtensions, PossiblyBinaryExtensions);

	// Ideally we'd generate a combined entry that allows all USD file types at once, but it seems that
	// GenerateFactoryFileExtensions and InternalGetFormatInfo on ObjectTools.cpp are really hard-coded
	// to only accept a single extension at a time... I guess the user can still just pick *.* to see
	// all file types at once

	// Add isolated entries for each supported file type
	for (const FString& Extension : PossiblyBinaryExtensions)
	{
		OutFormats.Add(FString::Printf(TEXT("%s; Universal Scene Description binary files"), *Extension));
	}
	for (const FString& Extension : TextExtensions)
	{
		OutFormats.Add(FString::Printf(TEXT("%s; Universal Scene Description text files"), *Extension));
	}
}

namespace UE::UnrealUSDWrapper::Private
{
	UE::FUsdStage OpenStageImpl(
		const TCHAR* RootIdentifier,
		const TCHAR* SessionIdentifier,
		EUsdInitialLoadSet InitialLoadSet,
		bool bUseStageCache,
		const TArray<FString>* PopulationMask,
		bool bForceReloadLayersFromDisk
	)
	{
		if (!RootIdentifier || FCString::Strlen(RootIdentifier) == 0)
		{
			return UE::FUsdStage();
		}

#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;

		pxr::SdfLayerHandleSet LoadedLayers = pxr::SdfLayer::GetLoadedLayers();
		pxr::UsdStageRefPtr Stage;

		TOptional<pxr::UsdStageCacheContext> StageCacheContext;
		if (bUseStageCache)
		{
			StageCacheContext.Emplace(pxr::UsdUtilsStageCache::Get());
		}

		pxr::UsdStagePopulationMask Mask;
		if (PopulationMask)
		{
			// The USD OpenMasked functions don't actually consult or populate the stage cache
			ensure(bUseStageCache == false);

			for (const FString& AllowedPrimPath : *PopulationMask)
			{
				Mask.Add(pxr::SdfPath{TCHAR_TO_UTF8(*AllowedPrimPath)});
			}
		}

		static_assert((int)pxr::UsdStage::InitialLoadSet::LoadAll == (int)EUsdInitialLoadSet::LoadAll);
		static_assert((int)pxr::UsdStage::InitialLoadSet::LoadNone == (int)EUsdInitialLoadSet::LoadNone);
		pxr::UsdStage::InitialLoadSet LoadSet = static_cast<pxr::UsdStage::InitialLoadSet>(InitialLoadSet);

		FString RootIdentifierStr = FString{RootIdentifier};
		FString SessionIdentifierStr = FString{SessionIdentifier};

		RootIdentifierStr.RemoveFromStart(UnrealIdentifiers::IdentifierPrefix);
		SessionIdentifierStr.RemoveFromStart(UnrealIdentifiers::IdentifierPrefix);

		pxr::SdfLayerRefPtr RootLayer = pxr::SdfLayer::FindOrOpen(TCHAR_TO_UTF8(*RootIdentifierStr));
		pxr::SdfLayerRefPtr SessionLayer = pxr::SdfLayer::FindOrOpen(TCHAR_TO_UTF8(*SessionIdentifierStr));
		if (RootLayer)
		{
			if (PopulationMask)
			{
				// We use this additional check so we don't have to replicate USD's "_CreateAnonymousSessionLayer"
				// Basically we can't pass an invalid session layer pointer here the stage will actually end up
				// with no session layer at all
				if (SessionLayer)
				{
					Stage = pxr::UsdStage::OpenMasked(RootLayer, SessionLayer, Mask, LoadSet);
				}
				else
				{
					Stage = pxr::UsdStage::OpenMasked(RootLayer, Mask, LoadSet);
				}
			}
			else
			{
				if (SessionLayer)
				{
					Stage = pxr::UsdStage::Open(RootLayer, SessionLayer, LoadSet);
				}
				else
				{
					Stage = pxr::UsdStage::Open(RootLayer, LoadSet);
				}
			}
		}

		if (bForceReloadLayersFromDisk && Stage)
		{
			// Layers are cached in the layer registry independently of the stage cache. If the layer is already in
			// the registry by the time we try to open a stage, even if we're not using a stage cache at all the
			// layer will be reused and the file will *not* be re-read, so here we manually reload them.
			// GetUsedLayers also returns layers from external references and such
			pxr::SdfLayerHandleVector StageLayers = Stage->GetUsedLayers();
			for (pxr::SdfLayerHandle StageLayer : StageLayers)
			{
				if (LoadedLayers.count(StageLayer) > 0)
				{
					StageLayer->Reload();
				}
			}
		}

		return UE::FUsdStage(Stage);
#else
		UsdWrapperUtils::CheckIfForceDisabled();
		return UE::FUsdStage();
#endif	  // #if USE_USD_SDK
	}
}	 // namespace UE::UnrealUSDWrapper::Private

UE::FUsdStage UnrealUSDWrapper::OpenStage(
	const TCHAR* Identifier,
	EUsdInitialLoadSet InitialLoadSet,
	bool bUseStageCache,
	bool bForceReloadLayersFromDisk
)
{
	const TArray<FString>* PopulationMask = nullptr;
	const TCHAR* SessionIdentifier = nullptr;
	return UE::UnrealUSDWrapper::Private::OpenStageImpl(
		Identifier,
		SessionIdentifier,
		InitialLoadSet,
		bUseStageCache,
		nullptr,
		bForceReloadLayersFromDisk
	);
}

UE::FUsdStage UnrealUSDWrapper::OpenStage(
	UE::FSdfLayer RootLayer,
	UE::FSdfLayer SessionLayer,
	EUsdInitialLoadSet InitialLoadSet,
	bool bUseStageCache,
	bool bForceReloadLayersFromDisk
)
{
	if (!RootLayer)
	{
		return {};
	}

	const TArray<FString>* PopulationMask = nullptr;
	const TCHAR* SessionIdentifier = nullptr;
	return UE::UnrealUSDWrapper::Private::OpenStageImpl(
		*RootLayer.GetIdentifier(),
		SessionLayer ? *SessionLayer.GetIdentifier() : nullptr,
		InitialLoadSet,
		bUseStageCache,
		nullptr,
		bForceReloadLayersFromDisk
	);
}

UE::FUsdStage UnrealUSDWrapper::OpenMaskedStage(
	const TCHAR* Identifier,
	EUsdInitialLoadSet InitialLoadSet,
	const TArray<FString>& PopulationMask,
	bool bForceReloadLayersFromDisk
)
{
	const bool bUseStageCache = false;
	const TCHAR* SessionIdentifier = nullptr;
	return UE::UnrealUSDWrapper::Private::OpenStageImpl(
		Identifier,
		SessionIdentifier,
		InitialLoadSet,
		bUseStageCache,
		&PopulationMask,
		bForceReloadLayersFromDisk
	);
}

UE::FUsdStage UnrealUSDWrapper::OpenMaskedStage(
	UE::FSdfLayer RootLayer,
	UE::FSdfLayer SessionLayer,
	EUsdInitialLoadSet InitialLoadSet,
	const TArray<FString>& PopulationMask,
	bool bForceReloadLayersFromDisk
)
{
	if (!RootLayer)
	{
		return {};
	}

	const bool bUseStageCache = false;
	const TCHAR* SessionIdentifier = nullptr;
	return UE::UnrealUSDWrapper::Private::OpenStageImpl(
		*RootLayer.GetIdentifier(),
		SessionLayer ? *SessionLayer.GetIdentifier() : nullptr,
		InitialLoadSet,
		bUseStageCache,
		&PopulationMask,
		bForceReloadLayersFromDisk
	);
}

UE::FUsdStage UnrealUSDWrapper::NewStage(const TCHAR* FilePath)
{
#if USE_USD_SDK
	FScopedUsdAllocs UsdAllocs;

	UE::FUsdStage UsdStage(pxr::UsdStage::CreateNew(TCHAR_TO_UTF8(FilePath)));
	if (UsdStage)
	{
		pxr::UsdGeomSetStageUpAxis(UsdStage, pxr::UsdGeomTokens->z);
	}

	return UsdStage;
#else
	UsdWrapperUtils::CheckIfForceDisabled();
	return UE::FUsdStage();
#endif	  // #if USE_USD_SDK
}

UE::FUsdStage UnrealUSDWrapper::NewStage()
{
#if USE_USD_SDK
	FScopedUsdAllocs UsdAllocs;

	UE::FUsdStage UsdStage(pxr::UsdStage::CreateInMemory());
	if (UsdStage)
	{
		pxr::UsdGeomSetStageUpAxis(UsdStage, pxr::UsdGeomTokens->z);
	}

	return UsdStage;
#else
	UsdWrapperUtils::CheckIfForceDisabled();
	return UE::FUsdStage();
#endif	  // #if USE_USD_SDK
}

UE::FUsdStage UnrealUSDWrapper::GetClipboardStage(bool bCreateIfNeeded)
{
	static UE::FUsdStage ClipboardStage;

#if USE_USD_SDK
	FScopedUsdAllocs UsdAllocs;

	if (!ClipboardStage && bCreateIfNeeded)
	{
		// Use this specific name so that we can check for it on our automated tests
		pxr::UsdStageCacheContext StageCacheContext{pxr::UsdUtilsStageCache::Get()};
		ClipboardStage = UE::FUsdStage{pxr::UsdStage::CreateInMemory(TCHAR_TO_UTF8(TEXT("unreal_clipboard_layer")))};

		pxr::UsdGeomSetStageUpAxis(ClipboardStage, pxr::UsdGeomTokens->z);
	}
#endif	  // #if USE_USD_SDK

	return ClipboardStage;
}

TArray<UE::FUsdStage> UnrealUSDWrapper::GetAllStagesFromCache()
{
	TArray<UE::FUsdStage> StagesInCache;

#if USE_USD_SDK
	FScopedUsdAllocs UsdAllocs;

	for (const pxr::UsdStageRefPtr& StageInCache : pxr::UsdUtilsStageCache::Get().GetAllStages())
	{
		StagesInCache.Emplace(StageInCache);
	}
#endif	  // #if USE_USD_SDK

	return StagesInCache;
}

void UnrealUSDWrapper::EraseStageFromCache(const UE::FUsdStage& Stage)
{
#if USE_USD_SDK
	pxr::UsdUtilsStageCache::Get().Erase(Stage);
#endif	  // #if USE_USD_SDK
}

void UnrealUSDWrapper::SetDefaultResolverDefaultSearchPath(const TArray<FDirectoryPath>& SearchPath)
{
#if USE_USD_SDK
	FScopedUsdAllocs UsdAllocs;

	std::vector<std::string> DefaultResolverSearchPath;
	DefaultResolverSearchPath.reserve(SearchPath.Num());

	for (const FDirectoryPath& Directory : SearchPath)
	{
		DefaultResolverSearchPath.push_back(TCHAR_TO_UTF8(*Directory.Path));
	}

	pxr::ArDefaultResolver::SetDefaultSearchPath(DefaultResolverSearchPath);
#endif	  // #if USE_USD_SDK
}

// Deprecated
void UnrealUSDWrapper::SetupDiagnosticDelegate()
{
}

// Deprecated
void UnrealUSDWrapper::ClearDiagnosticDelegate()
{
}

// Memory wrapper is declared in ModuleBoilerPlate.h
// Needed for FMemory::Malloc to fit the signature of std::malloc
void* FMemory_Malloc(size_t Size, size_t Alignment);
// Needed for FMemory::Free to fit the signature of std::free
void FMemory_Free(void* Data);

class FUnrealUSDWrapperModule : public IUnrealUSDWrapperModule
{
public:
	virtual void StartupModule() override TSAN_SAFE
	{
#if USE_USD_SDK
		LLM_SCOPE_BYTAG(Usd);

#if PLATFORM_WINDOWS
#if !USE_USD_MEMORY_MANAGER
		// Only Windows need to register the memory overloads when enabled in the USD SDK
		pxr::MemoryOverloads::RegisterMallocFree(FMemory_Malloc, FMemory_Free);
#endif
#endif // PLATFORM_WINDOWS
		
		// Path to USD base plugins
		FString UsdPluginsPath = FPaths::Combine(TEXT(".."), TEXT("ThirdParty"), TEXT("USD"), TEXT("UsdResources"));
		UsdPluginsPath = FPaths::ConvertRelativePathToFull(UsdPluginsPath);
#if PLATFORM_WINDOWS
#if PLATFORM_CPU_ARM_FAMILY
		UsdPluginsPath /= FPaths::Combine(TEXT("Win64"), TEXT("Arm64"), TEXT("plugins"));
#else
		UsdPluginsPath /= FPaths::Combine(TEXT("Win64"), TEXT("X64"), TEXT("plugins"));
#endif
#elif PLATFORM_LINUX
#if PLATFORM_CPU_ARM_FAMILY
		UsdPluginsPath /= FPaths::Combine(TEXT("Linux"), TEXT("Arm64"), TEXT("plugins"));
#else
		UsdPluginsPath /= FPaths::Combine(TEXT("Linux"), TEXT("X64"), TEXT("plugins"));
#endif
#elif PLATFORM_MAC
		UsdPluginsPath /= FPaths::Combine(TEXT("Mac"), TEXT("plugins"));
#endif	  // PLATFORM_WINDOWS

#ifdef USE_LIBRARIES_FROM_PLUGIN_FOLDER
		// e.g. "../../../Engine/Plugins/Runtime/USDCore/Source/ThirdParty/USD"
		FString TargetDllFolder = FPaths::Combine(
			IPluginManager::Get().FindPlugin(TEXT("USDCore"))->GetBaseDir(),
			TEXT("Source"),
			TEXT("ThirdParty"),
			TEXT("USD")
		);

#if PLATFORM_WINDOWS
		// VS2015 aligns with UnrealBuildTool's
		// Target.WindowsPlatform.GetVisualStudioCompilerVersionName()
#if PLATFORM_CPU_ARM_FAMILY
		TargetDllFolder /= FPaths::Combine(TEXT("VS2015"), TEXT("ARM64"), TEXT("bin"));
#else
		TargetDllFolder /= FPaths::Combine(TEXT("VS2015"), TEXT("x64"), TEXT("bin"));
#endif
#elif PLATFORM_LINUX
#if PLATFORM_CPU_ARM_FAMILY
		TargetDllFolder /= FPaths::Combine(TEXT("Unix"), TEXT("aarch64-unknown-linux-gnueabi"), TEXT("lib"));
#else
		TargetDllFolder /= FPaths::Combine(TEXT("Unix"), TEXT("x86_64-unknown-linux-gnu"), TEXT("lib"));
#endif
#elif PLATFORM_MAC
		TargetDllFolder /= FPaths::Combine(TEXT("Mac"), TEXT("lib"));
#endif	  // PLATFORM_WINDOWS

#else
		FString TargetDllFolder = FPlatformProcess::ExecutablePath();
		FPaths::NormalizeFilename(TargetDllFolder);
		TargetDllFolder = FPaths::GetPath(TargetDllFolder);
#endif	  // USE_LIBRARIES_FROM_PLUGIN_FOLDER

		// Path to the MaterialX standard data libraries.
		FString MaterialXStdDataLibsPath = FPaths::Combine(
			FPaths::EngineDir(),
			TEXT("Binaries"),
			TEXT("ThirdParty"),
			TEXT("MaterialX"),
			TEXT("libraries")
		);
		MaterialXStdDataLibsPath = FPaths::ConvertRelativePathToFull(MaterialXStdDataLibsPath);
		if (FPaths::DirectoryExists(MaterialXStdDataLibsPath))
		{
			FPlatformMisc::SetEnvironmentVar(TEXT("PXR_MTLX_STDLIB_SEARCH_PATHS"), *MaterialXStdDataLibsPath);
		}

		// Have to do this in USDClasses as we need the Json module, which is RTTI == false
		IUsdClassesModule::UpdatePlugInfoFiles(UsdPluginsPath, TargetDllFolder);

		UUsdProjectSettings* ProjectSettings = GetMutableDefault<UUsdProjectSettings>();

		// Combine our current plugins with any additional USD plugins the user may have set.
		TArray<FString> PluginDirectories;
		PluginDirectories.Add(UsdPluginsPath);
		for (const FDirectoryPath& Directory : ProjectSettings->AdditionalPluginDirectories)
		{
			if (!Directory.Path.IsEmpty())
			{
				// The directory chooser will try to enforce that the paths
				// in the setting are within the game content dir, but it's
				// still possible for the user to manually enter either a
				// relative or an absolute path that points elsewhere, so aim
				// to support any of these.
				FString PluginDirPath = Directory.Path;
				if (FPaths::IsRelative(PluginDirPath))
				{
					PluginDirPath = FPaths::Combine(FPaths::ProjectContentDir(), PluginDirPath);
					PluginDirPath = FPaths::ConvertRelativePathToFull(PluginDirPath);
				}
				FPaths::NormalizeDirectoryName(PluginDirPath);
				FPaths::CollapseRelativeDirectories(PluginDirPath);
				PluginDirectories.Add(PluginDirPath);
			}
		}

		{
			FScopedUsdAllocs UsdAllocs;

			std::vector<std::string> UsdPluginDirectories;
			UsdPluginDirectories.reserve(PluginDirectories.Num());

			for (const FString& Dir : PluginDirectories)
			{
				UsdPluginDirectories.push_back(TCHAR_TO_UTF8(*Dir));
			}

			PlugRegistry::GetInstance().RegisterPlugins(UsdPluginDirectories);
		}

		// Set the default search path for USD's default resolver using any path specified in the settings.
		TArray<FDirectoryPath> SearchPath = ProjectSettings->DefaultResolverSearchPath;

		FString ArSearchPathEnvVar = FPlatformMisc::GetEnvironmentVariable(TEXT("PXR_AR_DEFAULT_SEARCH_PATH"));
		if (!ArSearchPathEnvVar.IsEmpty())
		{
			TArray<FString> SearchPaths;
			ArSearchPathEnvVar.ParseIntoArray(SearchPaths, FPlatformMisc::GetPathVarDelimiter());
			SearchPath.Append(SearchPaths);
		}

		UnrealUSDWrapper::SetDefaultResolverDefaultSearchPath(SearchPath);

#endif	  // USE_USD_SDK

#if USE_USD_MEMORY_MANAGER
		FUsdMemoryManager::Initialize();
#endif

		UsdUnreal::MaterialUtils::RegisterRenderContext(UnrealIdentifiers::UniversalRenderContext);
		UsdUnreal::MaterialUtils::RegisterRenderContext(UnrealIdentifiers::UnrealRenderContext);

#if WITH_EDITOR
		// Update the supported filetype filters for reference/payload picker dialogs
		for (TFieldIterator<FProperty> PropertyIterator(UUsdReferenceOptions::StaticClass()); PropertyIterator; ++PropertyIterator)
		{
			FProperty* Property = *PropertyIterator;
			if (Property && Property->GetFName() == GET_MEMBER_NAME_CHECKED(UUsdReferenceOptions, TargetFile))
			{
				TArray<FString> SupportedExtensions = UnrealUSDWrapper::GetAllSupportedFileFormats();
				if (SupportedExtensions.Num() > 0)
				{
					// Note: Cannot have space after semicolon or else the parsing breaks on the Mac...
					FString JoinedWithSemicolon = FString::Join(SupportedExtensions, TEXT(";*."));
					FString JoinedWithComma = FString::Join(SupportedExtensions, TEXT(", *."));

					Property->SetMetaData(
						TEXT("FilePathFilter"),
						FString::Printf(TEXT("Universal Scene Description files (*.%s)|*.%s"), *JoinedWithComma, *JoinedWithSemicolon)
					);
				}
				break;
			}
		}
#endif	  // WITH_EDITOR
	}

	virtual void ShutdownModule() override
	{
		UsdUnreal::MaterialUtils::UnregisterRenderContext(UnrealIdentifiers::UniversalRenderContext);
		UsdUnreal::MaterialUtils::UnregisterRenderContext(UnrealIdentifiers::UnrealRenderContext);

#if USE_USD_MEMORY_MANAGER
		FUsdMemoryManager::Shutdown();
#endif
	}
};

IMPLEMENT_MODULE_USD(FUnrealUSDWrapperModule, UnrealUSDWrapper);

#undef LOCTEXT_NAMESPACE
