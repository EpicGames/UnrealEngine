// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdWrappers/PcpLayerStack.h"

#include "USDMemory.h"
#include "UsdWrappers/PcpLayerStackIdentifier.h"
#include "UsdWrappers/SdfLayer.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include <pxr/usd/pcp/layerStack.h>
#include <pxr/usd/pcp/layerStackIdentifier.h>
#include "USDIncludesEnd.h"
#endif	  // #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
		class FPcpLayerStackImpl
		{
		public:
			FPcpLayerStackImpl() = default;

#if defined(ENABLE_USD_DEBUG_PATH) && ENABLE_USD_DEBUG_PATH
			FPcpLayerStackIdentifier DebugIdentifier;
#endif

#if USE_USD_SDK
			explicit FPcpLayerStackImpl(const pxr::PcpLayerStackRefPtr& InPcpLayerStack)
				: PxrPcpLayerStack(InPcpLayerStack)
			{
#if ENABLE_USD_DEBUG_PATH
				DebugIdentifier = FPcpLayerStackIdentifier{(*PxrPcpLayerStack)->GetIdentifier()};
#endif
			}

			explicit FPcpLayerStackImpl(pxr::PcpLayerStackRefPtr&& InPcpLayerStack)
				: PxrPcpLayerStack(MoveTemp(InPcpLayerStack))
			{
#if ENABLE_USD_DEBUG_PATH
				DebugIdentifier = FPcpLayerStackIdentifier{(*PxrPcpLayerStack)->GetIdentifier()};
#endif
			}

			TUsdStore<pxr::PcpLayerStackRefPtr> PxrPcpLayerStack;
#endif	  // #if USE_USD_SDK
		};
	}	 // namespace Internal
}	 // namespace UE

namespace UE
{
	FPcpLayerStack::FPcpLayerStack()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FPcpLayerStackImpl>();
	}

	FPcpLayerStack::FPcpLayerStack(const FPcpLayerStack& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FPcpLayerStackImpl>(Other.Impl->PxrPcpLayerStack.Get());
#endif	  // #if USE_USD_SDK
	}

	FPcpLayerStack::FPcpLayerStack(FPcpLayerStack&& Other) = default;

	FPcpLayerStack::~FPcpLayerStack()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	FPcpLayerStack& FPcpLayerStack::operator=(const FPcpLayerStack& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FPcpLayerStackImpl>(Other.Impl->PxrPcpLayerStack.Get());
#endif	  // #if USE_USD_SDK
		return *this;
	}

	FPcpLayerStack& FPcpLayerStack::operator=(FPcpLayerStack&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MoveTemp(Other.Impl);

		return *this;
	}

	bool FPcpLayerStack::operator==(const FPcpLayerStack& Other) const
	{
#if USE_USD_SDK
		return Impl->PxrPcpLayerStack.Get() == Other.Impl->PxrPcpLayerStack.Get();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FPcpLayerStack::operator!=(const FPcpLayerStack& Other) const
	{
		return !(*this == Other);
	}

	FPcpLayerStack::operator bool() const
	{
#if USE_USD_SDK
		return static_cast<bool>(Impl->PxrPcpLayerStack);
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

#if USE_USD_SDK
	FPcpLayerStack::FPcpLayerStack(const pxr::PcpLayerStackRefPtr& InPcpLayerStack)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FPcpLayerStackImpl>(InPcpLayerStack);
	}

	FPcpLayerStack::FPcpLayerStack(pxr::PcpLayerStackRefPtr&& InPcpLayerStack)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FPcpLayerStackImpl>(MoveTemp(InPcpLayerStack));
	}

	FPcpLayerStack& FPcpLayerStack::operator=(const pxr::PcpLayerStackRefPtr& InPcpLayerStack)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FPcpLayerStackImpl>(InPcpLayerStack);
		return *this;
	}

	FPcpLayerStack& FPcpLayerStack::operator=(pxr::PcpLayerStackRefPtr&& InPcpLayerStack)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FPcpLayerStackImpl>(MoveTemp(InPcpLayerStack));
		return *this;
	}

	FPcpLayerStack::operator pxr::PcpLayerStackRefPtr&()
	{
		return Impl->PxrPcpLayerStack.Get(); 
	}

	FPcpLayerStack::operator const pxr::PcpLayerStackRefPtr&() const
	{
		return Impl->PxrPcpLayerStack.Get();
	}
#endif	  // #if USE_USD_SDK

	FPcpLayerStackIdentifier FPcpLayerStack::GetIdentifier() const
	{
		FPcpLayerStackIdentifier Result;
#if USE_USD_SDK
		Result = FPcpLayerStackIdentifier{Impl->PxrPcpLayerStack.Get()->GetIdentifier()};
#endif	  // USE_USD_SDK
		return Result;
	}

	TArray<FSdfLayer> FPcpLayerStack::GetLayers() const
	{
		TArray<FSdfLayer> Result;
#if USE_USD_SDK
		TUsdStore<std::vector<pxr::SdfLayerRefPtr>> Layers = Impl->PxrPcpLayerStack.Get()->GetLayers();
		Result.Reserve(Layers->size());

		for (const pxr::SdfLayerRefPtr& SdfLayer : *Layers)
		{
			Result.Add(FSdfLayer{SdfLayer});
		}
#endif	  // USE_USD_SDK
		return Result;
	}

	FSdfLayerOffset FPcpLayerStack::GetLayerOffsetForLayer(const FSdfLayer& Layer) const
	{
		FSdfLayerOffset Result;
#if USE_USD_SDK
		if (const pxr::SdfLayerOffset* UsdOffest = Impl->PxrPcpLayerStack.Get()->GetLayerOffsetForLayer(pxr::SdfLayerRefPtr{Layer}))
		{
			Result.Offset = UsdOffest->GetOffset();
			Result.Scale = UsdOffest->GetScale();
		}
#endif	  // USE_USD_SDK
		return Result;
	}

	FSdfLayerOffset FPcpLayerStack::GetLayerOffsetForLayer(const FSdfLayerWeak& Layer) const
	{
		FSdfLayerOffset Result;
#if USE_USD_SDK
		if (const pxr::SdfLayerOffset* UsdOffest = Impl->PxrPcpLayerStack.Get()->GetLayerOffsetForLayer(pxr::SdfLayerHandle{Layer}))
		{
			Result.Offset = UsdOffest->GetOffset();
			Result.Scale = UsdOffest->GetScale();
		}
#endif	  // USE_USD_SDK
		return Result;
	}

	bool FPcpLayerStack::HasLayer(const FSdfLayer& Layer) const
	{
#if USE_USD_SDK
		return Impl->PxrPcpLayerStack.Get()->HasLayer(pxr::SdfLayerRefPtr{Layer});
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool FPcpLayerStack::HasLayer(const FSdfLayerWeak& Layer) const
	{
#if USE_USD_SDK
		return Impl->PxrPcpLayerStack.Get()->HasLayer(pxr::SdfLayerHandle{Layer});
#else
		return false;
#endif	  // USE_USD_SDK
	}

}	 // namespace UE
