// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdWrappers/PcpLayerStackIdentifier.h"

#include "UsdWrappers/SdfLayer.h"
#include "USDMemory.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include <pxr/usd/pcp/layerStack.h>
#include "USDIncludesEnd.h"
#endif	  // #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
		class FPcpLayerStackIdentifierImpl
		{
		public:
			FPcpLayerStackIdentifierImpl() = default;

#if defined(ENABLE_USD_DEBUG_PATH) && ENABLE_USD_DEBUG_PATH
			FString DebugRootLayerIdentifier;
			FString DebugSessionLayerIdentifier;
#endif

#if USE_USD_SDK
			explicit FPcpLayerStackIdentifierImpl(const pxr::PcpLayerStackIdentifier& InPcpLayerStackIdentifier)
				: PxrPcpLayerStackIdentifier(InPcpLayerStackIdentifier)
			{
#if defined(ENABLE_USD_DEBUG_PATH) && ENABLE_USD_DEBUG_PATH
				if (PxrPcpLayerStackIdentifier->rootLayer)
				{
					DebugRootLayerIdentifier = FString{UTF8_TO_TCHAR(PxrPcpLayerStackIdentifier->rootLayer->GetIdentifier().c_str())};
				}
				if (PxrPcpLayerStackIdentifier->sessionLayer)
				{
					DebugSessionLayerIdentifier = FString{UTF8_TO_TCHAR(PxrPcpLayerStackIdentifier->sessionLayer->GetIdentifier().c_str())};
				}
#endif
			}

			explicit FPcpLayerStackIdentifierImpl(pxr::PcpLayerStackIdentifier&& InPcpLayerStackIdentifier)
				: PxrPcpLayerStackIdentifier(MoveTemp(InPcpLayerStackIdentifier))
			{
#if defined(ENABLE_USD_DEBUG_PATH) && ENABLE_USD_DEBUG_PATH
				if (PxrPcpLayerStackIdentifier->rootLayer)
				{
					DebugRootLayerIdentifier = FString{UTF8_TO_TCHAR(PxrPcpLayerStackIdentifier->rootLayer->GetIdentifier().c_str())};
				}
				if (PxrPcpLayerStackIdentifier->sessionLayer)
				{
					DebugSessionLayerIdentifier = FString{UTF8_TO_TCHAR(PxrPcpLayerStackIdentifier->sessionLayer->GetIdentifier().c_str())};
				}
#endif
			}

			TUsdStore<pxr::PcpLayerStackIdentifier> PxrPcpLayerStackIdentifier;
#endif	  // #if USE_USD_SDK
		};
	}	 // namespace Internal
}	 // namespace UE

namespace UE
{
	FPcpLayerStackIdentifier::FPcpLayerStackIdentifier()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FPcpLayerStackIdentifierImpl>();
	}

	FPcpLayerStackIdentifier::FPcpLayerStackIdentifier(const FPcpLayerStackIdentifier& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FPcpLayerStackIdentifierImpl>(Other.Impl->PxrPcpLayerStackIdentifier.Get());
#endif	  // #if USE_USD_SDK
	}

	FPcpLayerStackIdentifier::FPcpLayerStackIdentifier(FPcpLayerStackIdentifier&& Other) = default;

	FPcpLayerStackIdentifier::~FPcpLayerStackIdentifier()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	FPcpLayerStackIdentifier& FPcpLayerStackIdentifier::operator=(const FPcpLayerStackIdentifier& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FPcpLayerStackIdentifierImpl>(Other.Impl->PxrPcpLayerStackIdentifier.Get());
#endif	  // #if USE_USD_SDK
		return *this;
	}

	FPcpLayerStackIdentifier& FPcpLayerStackIdentifier::operator=(FPcpLayerStackIdentifier&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MoveTemp(Other.Impl);

		return *this;
	}

	bool FPcpLayerStackIdentifier::operator==(const FPcpLayerStackIdentifier& Other) const
	{
#if USE_USD_SDK
		return Impl->PxrPcpLayerStackIdentifier.Get() == Other.Impl->PxrPcpLayerStackIdentifier.Get();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FPcpLayerStackIdentifier::operator!=(const FPcpLayerStackIdentifier& Other) const
	{
		return !(*this == Other);
	}

	FPcpLayerStackIdentifier::operator bool() const
	{
#if USE_USD_SDK
		return static_cast<bool>(Impl->PxrPcpLayerStackIdentifier);
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

#if USE_USD_SDK
	FPcpLayerStackIdentifier::FPcpLayerStackIdentifier(const pxr::PcpLayerStackIdentifier& InPcpLayerStackIdentifier)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FPcpLayerStackIdentifierImpl>(InPcpLayerStackIdentifier);
	}

	FPcpLayerStackIdentifier::FPcpLayerStackIdentifier(pxr::PcpLayerStackIdentifier&& InPcpLayerStackIdentifier)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FPcpLayerStackIdentifierImpl>(MoveTemp(InPcpLayerStackIdentifier));
	}

	FPcpLayerStackIdentifier& FPcpLayerStackIdentifier::operator=(const pxr::PcpLayerStackIdentifier& InPcpLayerStackIdentifier)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FPcpLayerStackIdentifierImpl>(InPcpLayerStackIdentifier);
		return *this;
	}

	FPcpLayerStackIdentifier& FPcpLayerStackIdentifier::operator=(pxr::PcpLayerStackIdentifier&& InPcpLayerStackIdentifier)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FPcpLayerStackIdentifierImpl>(MoveTemp(InPcpLayerStackIdentifier));
		return *this;
	}

	FPcpLayerStackIdentifier::operator pxr::PcpLayerStackIdentifier&()
	{
		return Impl->PxrPcpLayerStackIdentifier.Get(); 
	}

	FPcpLayerStackIdentifier::operator const pxr::PcpLayerStackIdentifier&() const
	{
		return Impl->PxrPcpLayerStackIdentifier.Get();
	}
#endif	  // #if USE_USD_SDK

	FSdfLayerWeak FPcpLayerStackIdentifier::RootLayer() const
	{
		FSdfLayerWeak Result;
#if USE_USD_SDK
		Result = FSdfLayerWeak{Impl->PxrPcpLayerStackIdentifier.Get().rootLayer};
#endif	  // USE_USD_SDK
		return Result;
	}

	FSdfLayerWeak FPcpLayerStackIdentifier::SessionLayer() const
	{
		FSdfLayerWeak Result;
#if USE_USD_SDK
		Result = FSdfLayerWeak{Impl->PxrPcpLayerStackIdentifier.Get().sessionLayer};
#endif	  // USE_USD_SDK
		return Result;
	}

}	 // namespace UE
