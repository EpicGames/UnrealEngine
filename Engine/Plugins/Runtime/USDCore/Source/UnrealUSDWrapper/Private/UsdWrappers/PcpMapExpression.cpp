// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdWrappers/PcpMapExpression.h"

#include "UsdWrappers/SdfLayer.h"
#include "USDMemory.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/pcp/mapExpression.h"
#include "USDIncludesEnd.h"
#endif	  // #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
		class FPcpMapExpressionImpl
		{
		public:
			FPcpMapExpressionImpl() = default;

#if defined(ENABLE_USD_DEBUG_PATH) && ENABLE_USD_DEBUG_PATH
			FSdfLayerOffset LayerOffset;
#endif

#if USE_USD_SDK
			explicit FPcpMapExpressionImpl(const pxr::PcpMapExpression& InPcpMapExpression)
				: PxrPcpMapExpression(InPcpMapExpression)
			{
#if defined(ENABLE_USD_DEBUG_PATH) && ENABLE_USD_DEBUG_PATH && USE_USD_SDK
				const pxr::SdfLayerOffset& PcpMapToParent = PxrPcpMapExpression->GetTimeOffset();
				LayerOffset = FSdfLayerOffset{PcpMapToParent.GetOffset(), PcpMapToParent.GetScale()};
#endif
			}

			explicit FPcpMapExpressionImpl(pxr::PcpMapExpression&& InPcpMapExpression)
				: PxrPcpMapExpression(MoveTemp(InPcpMapExpression))
			{
#if defined(ENABLE_USD_DEBUG_PATH) && ENABLE_USD_DEBUG_PATH && USE_USD_SDK
				const pxr::SdfLayerOffset& PcpMapToParent = PxrPcpMapExpression->GetTimeOffset();
				LayerOffset = FSdfLayerOffset{PcpMapToParent.GetOffset(), PcpMapToParent.GetScale()};
#endif
			}

			TUsdStore<pxr::PcpMapExpression> PxrPcpMapExpression;
#endif	  // #if USE_USD_SDK
		};
	}	 // namespace Internal
}	 // namespace UE

namespace UE
{
	FPcpMapExpression::FPcpMapExpression()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FPcpMapExpressionImpl>();
	}

	FPcpMapExpression::FPcpMapExpression(const FPcpMapExpression& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FPcpMapExpressionImpl>(Other.Impl->PxrPcpMapExpression.Get());
#endif	  // #if USE_USD_SDK
	}

	FPcpMapExpression::FPcpMapExpression(FPcpMapExpression&& Other) = default;

	FPcpMapExpression::~FPcpMapExpression()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	FPcpMapExpression& FPcpMapExpression::operator=(const FPcpMapExpression& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FPcpMapExpressionImpl>(Other.Impl->PxrPcpMapExpression.Get());
#endif	  // #if USE_USD_SDK
		return *this;
	}

	FPcpMapExpression& FPcpMapExpression::operator=(FPcpMapExpression&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MoveTemp(Other.Impl);

		return *this;
	}

#if USE_USD_SDK
	FPcpMapExpression::FPcpMapExpression(const pxr::PcpMapExpression& InPcpMapExpression)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FPcpMapExpressionImpl>(InPcpMapExpression);
	}

	FPcpMapExpression::FPcpMapExpression(pxr::PcpMapExpression&& InPcpMapExpression)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FPcpMapExpressionImpl>(MoveTemp(InPcpMapExpression));
	}

	FPcpMapExpression& FPcpMapExpression::operator=(const pxr::PcpMapExpression& InPcpMapExpression)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FPcpMapExpressionImpl>(InPcpMapExpression);
		return *this;
	}

	FPcpMapExpression& FPcpMapExpression::operator=(pxr::PcpMapExpression&& InPcpMapExpression)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FPcpMapExpressionImpl>(MoveTemp(InPcpMapExpression));
		return *this;
	}

	FPcpMapExpression::operator pxr::PcpMapExpression&()
	{
		return Impl->PxrPcpMapExpression.Get();
	}

	FPcpMapExpression::operator const pxr::PcpMapExpression&() const
	{
		return Impl->PxrPcpMapExpression.Get();
	}
#endif	  // #if USE_USD_SDK

	FSdfLayerOffset FPcpMapExpression::GetTimeOffset() const
	{
		FSdfLayerOffset Result;

#if USE_USD_SDK
		const pxr::SdfLayerOffset& UsdOffset = Impl->PxrPcpMapExpression.Get().GetTimeOffset();
		Result.Offset = UsdOffset.GetOffset();
		Result.Scale = UsdOffset.GetScale();
#endif	  // #if USE_USD_SDK

		return Result;
	}
}	 // namespace UE
