// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdWrappers/PcpNodeRef.h"

#include "UsdWrappers/PcpMapExpression.h"
#include "UsdWrappers/SdfLayer.h"
#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/PcpLayerStack.h"
#include "USDMemory.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/usd/attribute.h"
#include "pxr/usd/pcp/mapExpression.h"
#include "pxr/usd/pcp/layerStack.h"
#include "USDIncludesEnd.h"
#endif	  // #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
		class FPcpNodeRefImpl
		{
		public:
			FPcpNodeRefImpl() = default;

#if USE_USD_SDK

#if ENABLE_USD_DEBUG_PATH
			FString DebugPath;
			EPcpArcType DebugArcType;
			FPcpLayerStack DebugLayerStack;
			FPcpMapExpression DebugMapToParent;
			FPcpMapExpression DebugMapToRoot;
			bool bHasSpecs = false;
			bool bCanContributeSpecs = false;
			TOptional<FPcpNodeRef> DebugParent; // Optional here otherwise our default constructor would recurse forever
#endif

			explicit FPcpNodeRefImpl(const pxr::PcpNodeRef& InPcpNodeRef)
				: PxrPcpNodeRef(InPcpNodeRef)
			{
#if ENABLE_USD_DEBUG_PATH
				SetupDebugMembers();
#endif
			}

			explicit FPcpNodeRefImpl(pxr::PcpNodeRef&& InPcpNodeRef)
				: PxrPcpNodeRef(MoveTemp(InPcpNodeRef))
			{
#if ENABLE_USD_DEBUG_PATH
				SetupDebugMembers();
#endif
			}

#if ENABLE_USD_DEBUG_PATH
			void SetupDebugMembers()
			{
				if (!PxrPcpNodeRef || !*PxrPcpNodeRef)
				{
					return;
				}

				DebugPath = FString{UTF8_TO_TCHAR(PxrPcpNodeRef->GetPath().GetString().c_str())};
				DebugArcType = static_cast<EPcpArcType>(PxrPcpNodeRef->GetArcType());
				DebugLayerStack = FPcpLayerStack{PxrPcpNodeRef->GetLayerStack()};
				DebugMapToParent = FPcpMapExpression{PxrPcpNodeRef->GetMapToParent()};
				DebugMapToRoot = FPcpMapExpression{PxrPcpNodeRef->GetMapToRoot()};
				bHasSpecs = PxrPcpNodeRef->HasSpecs();
				bCanContributeSpecs = PxrPcpNodeRef->CanContributeSpecs();
				if (pxr::PcpNodeRef ParentNode = PxrPcpNodeRef->GetParentNode())
				{
					DebugParent = FPcpNodeRef{ParentNode};
				}
			}
#endif

			TUsdStore<pxr::PcpNodeRef> PxrPcpNodeRef;
#endif	  // #if USE_USD_SDK
		};
	}	 // namespace Internal
}	 // namespace UE

namespace UE
{
	FPcpNodeRef::FPcpNodeRef()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FPcpNodeRefImpl>();
	}

	FPcpNodeRef::FPcpNodeRef(const FPcpNodeRef& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FPcpNodeRefImpl>(Other.Impl->PxrPcpNodeRef.Get());
#endif	  // #if USE_USD_SDK
	}

	FPcpNodeRef::FPcpNodeRef(FPcpNodeRef&& Other) = default;

	FPcpNodeRef::~FPcpNodeRef()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	FPcpNodeRef& FPcpNodeRef::operator=(const FPcpNodeRef& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FPcpNodeRefImpl>(Other.Impl->PxrPcpNodeRef.Get());
#endif	  // #if USE_USD_SDK
		return *this;
	}

	FPcpNodeRef& FPcpNodeRef::operator=(FPcpNodeRef&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MoveTemp(Other.Impl);

		return *this;
	}

	bool FPcpNodeRef::operator==(const FPcpNodeRef& Other) const
	{
#if USE_USD_SDK
		return Impl->PxrPcpNodeRef.Get() == Other.Impl->PxrPcpNodeRef.Get();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FPcpNodeRef::operator!=(const FPcpNodeRef& Other) const
	{
		return !(*this == Other);
	}

	FPcpNodeRef::operator bool() const
	{
#if USE_USD_SDK
		return (bool)Impl->PxrPcpNodeRef.Get();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

#if USE_USD_SDK
	FPcpNodeRef::FPcpNodeRef(const pxr::PcpNodeRef& InPcpNodeRef)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FPcpNodeRefImpl>(InPcpNodeRef);
	}

	FPcpNodeRef::FPcpNodeRef(pxr::PcpNodeRef&& InPcpNodeRef)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FPcpNodeRefImpl>(MoveTemp(InPcpNodeRef));
	}

	FPcpNodeRef& FPcpNodeRef::operator=(const pxr::PcpNodeRef& InPcpNodeRef)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FPcpNodeRefImpl>(InPcpNodeRef);
		return *this;
	}

	FPcpNodeRef& FPcpNodeRef::operator=(pxr::PcpNodeRef&& InPcpNodeRef)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FPcpNodeRefImpl>(MoveTemp(InPcpNodeRef));
		return *this;
	}

	FPcpNodeRef::operator pxr::PcpNodeRef&()
	{
		return Impl->PxrPcpNodeRef.Get();
	}

	FPcpNodeRef::operator const pxr::PcpNodeRef&() const
	{
		return Impl->PxrPcpNodeRef.Get();
	}
#endif	  // #if USE_USD_SDK

	EPcpArcType FPcpNodeRef::GetArcType() const
	{
#if USE_USD_SDK
		static_assert((int)EPcpArcType::PcpArcTypeRoot == (int)pxr::PcpArcType::PcpArcTypeRoot);
		static_assert((int)EPcpArcType::PcpArcTypeInherit == (int)pxr::PcpArcType::PcpArcTypeInherit);
		static_assert((int)EPcpArcType::PcpArcTypeVariant == (int)pxr::PcpArcType::PcpArcTypeVariant);
		static_assert((int)EPcpArcType::PcpArcTypeRelocate == (int)pxr::PcpArcType::PcpArcTypeRelocate);
		static_assert((int)EPcpArcType::PcpArcTypeReference == (int)pxr::PcpArcType::PcpArcTypeReference);
		static_assert((int)EPcpArcType::PcpArcTypePayload == (int)pxr::PcpArcType::PcpArcTypePayload);
		static_assert((int)EPcpArcType::PcpArcTypeSpecialize == (int)pxr::PcpArcType::PcpArcTypeSpecialize);
		static_assert((int)EPcpArcType::PcpNumArcTypes == (int)pxr::PcpArcType::PcpNumArcTypes);

		return static_cast<EPcpArcType>(Impl->PxrPcpNodeRef.Get().GetArcType());
#else
		return {};
#endif	  // #if USE_USD_SDK
	}

	FPcpNodeRef FPcpNodeRef::GetParentNode() const
	{
		FPcpNodeRef Result;

#if USE_USD_SDK
		Result = FPcpNodeRef{Impl->PxrPcpNodeRef.Get().GetParentNode()};
#endif	  // #if USE_USD_SDK

		return Result;
	}

	FPcpMapExpression FPcpNodeRef::GetMapToParent() const
	{
#if USE_USD_SDK
		return FPcpMapExpression{Impl->PxrPcpNodeRef.Get().GetMapToParent()};
#else
		return {};
#endif	  // #if USE_USD_SDK
	}

	FPcpMapExpression FPcpNodeRef::GetMapToRoot() const
	{
#if USE_USD_SDK
		return FPcpMapExpression{Impl->PxrPcpNodeRef.Get().GetMapToRoot()};
#else
		return {};
#endif	  // #if USE_USD_SDK
	}

	FSdfPath FPcpNodeRef::GetPath() const
	{
		FSdfPath Path;

#if USE_USD_SDK
		Path = FSdfPath{Impl->PxrPcpNodeRef.Get().GetPath()};
#endif	  // #if USE_USD_SDK

		return Path;
	}

	FPcpLayerStack FPcpNodeRef::GetLayerStack() const
	{
#if USE_USD_SDK
		return FPcpLayerStack{Impl->PxrPcpNodeRef.Get().GetLayerStack()};
#else
		return {};
#endif	  // #if USE_USD_SDK
	}
}	 // namespace UE
