// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdWrappers/PcpPrimIndex.h"

#include "UsdWrappers/PcpNodeRef.h"
#include "UsdWrappers/SdfPath.h"
#include "USDMemory.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/pcp/primIndex.h"
#include "pxr/usd/pcp/node.h"
#include "USDIncludesEnd.h"
#endif	  // #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
		class FPcpPrimIndexImpl
		{
		public:
			FPcpPrimIndexImpl() = default;

#if USE_USD_SDK

#if ENABLE_USD_DEBUG_PATH
			FString DebugPath;
#endif

			explicit FPcpPrimIndexImpl(const pxr::PcpPrimIndex& InPcpPrimIndex)
				: PxrPcpPrimIndex(InPcpPrimIndex)
			{
#if ENABLE_USD_DEBUG_PATH
				SetupDebugMembers();
#endif
			}

			explicit FPcpPrimIndexImpl(pxr::PcpPrimIndex&& InPcpPrimIndex)
				: PxrPcpPrimIndex(MoveTemp(InPcpPrimIndex))
			{
#if ENABLE_USD_DEBUG_PATH
				SetupDebugMembers();
#endif
			}

#if ENABLE_USD_DEBUG_PATH
			void SetupDebugMembers()
			{
				if (!PxrPcpPrimIndex)
				{
					return;
				}

				DebugPath = FString{UTF8_TO_TCHAR(PxrPcpPrimIndex->GetPath().GetString().c_str())};
			}
#endif

			TUsdStore<pxr::PcpPrimIndex> PxrPcpPrimIndex;
#endif	  // #if USE_USD_SDK
		};
	}	 // namespace Internal
}	 // namespace UE

namespace UE
{
	FPcpPrimIndex::FPcpPrimIndex()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FPcpPrimIndexImpl>();
	}

	FPcpPrimIndex::FPcpPrimIndex(const FPcpPrimIndex& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FPcpPrimIndexImpl>(Other.Impl->PxrPcpPrimIndex.Get());
#endif	  // #if USE_USD_SDK
	}

	FPcpPrimIndex::FPcpPrimIndex(FPcpPrimIndex&& Other) = default;

	FPcpPrimIndex::~FPcpPrimIndex()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	FPcpPrimIndex& FPcpPrimIndex::operator=(const FPcpPrimIndex& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FPcpPrimIndexImpl>(Other.Impl->PxrPcpPrimIndex.Get());
#endif	  // #if USE_USD_SDK
		return *this;
	}

	FPcpPrimIndex& FPcpPrimIndex::operator=(FPcpPrimIndex&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MoveTemp(Other.Impl);

		return *this;
	}

	FPcpPrimIndex::operator bool() const
	{
		return IsValid();
	}

#if USE_USD_SDK
	FPcpPrimIndex::FPcpPrimIndex(const pxr::PcpPrimIndex& InPcpPrimIndex)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FPcpPrimIndexImpl>(InPcpPrimIndex);
	}

	FPcpPrimIndex::FPcpPrimIndex(pxr::PcpPrimIndex&& InPcpPrimIndex)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FPcpPrimIndexImpl>(MoveTemp(InPcpPrimIndex));
	}

	FPcpPrimIndex& FPcpPrimIndex::operator=(const pxr::PcpPrimIndex& InPcpPrimIndex)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FPcpPrimIndexImpl>(InPcpPrimIndex);
		return *this;
	}

	FPcpPrimIndex& FPcpPrimIndex::operator=(pxr::PcpPrimIndex&& InPcpPrimIndex)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FPcpPrimIndexImpl>(MoveTemp(InPcpPrimIndex));
		return *this;
	}

	FPcpPrimIndex::operator pxr::PcpPrimIndex&()
	{
		return Impl->PxrPcpPrimIndex.Get();
	}

	FPcpPrimIndex::operator const pxr::PcpPrimIndex&() const
	{
		return Impl->PxrPcpPrimIndex.Get();
	}
#endif	  // #if USE_USD_SDK

	bool FPcpPrimIndex::IsValid() const
	{
#if USE_USD_SDK
		return (bool)Impl->PxrPcpPrimIndex->IsValid();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	UE::FPcpNodeRef FPcpPrimIndex::GetRootNode() const
	{
#if USE_USD_SDK
		return UE::FPcpNodeRef{Impl->PxrPcpPrimIndex->GetRootNode()};
#else
		return {};
#endif	  // #if USE_USD_SDK
	}

	FString FPcpPrimIndex::DumpToString(bool bIncludeInheritOriginInfo, bool bIncludeMaps) const
	{
#if USE_USD_SDK
		TUsdStore<std::string> UsdString = Impl->PxrPcpPrimIndex->DumpToString(bIncludeInheritOriginInfo, bIncludeMaps);
		return FString{UTF8_TO_TCHAR(UsdString->c_str())};
#else
		return {};
#endif	  // #if USE_USD_SDK
	}
}	 // namespace UE
