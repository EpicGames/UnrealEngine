// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdWrappers/SdfAttributeSpec.h"

#include "UsdWrappers/VtValue.h"

#include "USDMemory.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/usd/attribute.h"
#include "USDIncludesEnd.h"
#endif	  // #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
		class FSdfAttributeSpecImpl
		{
		public:
			FSdfAttributeSpecImpl() = default;

#if USE_USD_SDK
			explicit FSdfAttributeSpecImpl(const pxr::SdfAttributeSpecHandle& InSdfAttributeSpec)
				: PxrSdfAttributeSpec(InSdfAttributeSpec)
			{
			}

			explicit FSdfAttributeSpecImpl(pxr::SdfAttributeSpecHandle&& InSdfAttributeSpec)
				: PxrSdfAttributeSpec(MoveTemp(InSdfAttributeSpec))
			{
			}

			TUsdStore<pxr::SdfAttributeSpecHandle> PxrSdfAttributeSpec;
#endif	  // #if USE_USD_SDK
		};
	}
}

namespace UE
{
	FSdfAttributeSpec::FSdfAttributeSpec()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfAttributeSpecImpl>();
	}

	FSdfAttributeSpec::FSdfAttributeSpec(const FSdfAttributeSpec& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfAttributeSpecImpl>(Other.Impl->PxrSdfAttributeSpec.Get());
#endif	  // #if USE_USD_SDK
	}

	FSdfAttributeSpec::FSdfAttributeSpec(FSdfAttributeSpec&& Other) = default;

	FSdfAttributeSpec::~FSdfAttributeSpec()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	FSdfAttributeSpec& FSdfAttributeSpec::operator=(const FSdfAttributeSpec& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfAttributeSpecImpl>(Other.Impl->PxrSdfAttributeSpec.Get());
#endif	  // #if USE_USD_SDK
		return *this;
	}

	FSdfAttributeSpec& FSdfAttributeSpec::operator=(FSdfAttributeSpec&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MoveTemp(Other.Impl);

		return *this;
	}

	bool FSdfAttributeSpec::operator==(const FSdfAttributeSpec& Other) const
	{
#if USE_USD_SDK
		return Impl->PxrSdfAttributeSpec.Get() == Other.Impl->PxrSdfAttributeSpec.Get();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FSdfAttributeSpec::operator!=(const FSdfAttributeSpec& Other) const
	{
		return !(*this == Other);
	}

	FSdfAttributeSpec::operator bool() const
	{
#if USE_USD_SDK
		return (bool)Impl->PxrSdfAttributeSpec.Get();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

#if USE_USD_SDK
	FSdfAttributeSpec::FSdfAttributeSpec(const pxr::SdfAttributeSpecHandle& InSdfAttributeSpec)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfAttributeSpecImpl>(InSdfAttributeSpec);
	}

	FSdfAttributeSpec::FSdfAttributeSpec(pxr::SdfAttributeSpecHandle&& InSdfAttributeSpec)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfAttributeSpecImpl>(MoveTemp(InSdfAttributeSpec));
	}

	FSdfAttributeSpec& FSdfAttributeSpec::operator=(const pxr::SdfAttributeSpecHandle& InSdfAttributeSpec)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfAttributeSpecImpl>(InSdfAttributeSpec);
		return *this;
	}

	FSdfAttributeSpec& FSdfAttributeSpec::operator=(pxr::SdfAttributeSpecHandle&& InSdfAttributeSpec)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfAttributeSpecImpl>(MoveTemp(InSdfAttributeSpec));
		return *this;
	}

	FSdfAttributeSpec::operator pxr::SdfAttributeSpecHandle&()
	{
		return Impl->PxrSdfAttributeSpec.Get();
	}

	FSdfAttributeSpec::operator const pxr::SdfAttributeSpecHandle&() const
	{
		return Impl->PxrSdfAttributeSpec.Get();
	}
#endif	  // #if USE_USD_SDK

	UE::FVtValue FSdfAttributeSpec::GetDefaultValue() const
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;
		return UE::FVtValue{Impl->PxrSdfAttributeSpec.Get()->GetDefaultValue()};
#else
		return {};
#endif	  // #if USE_USD_SDK
	}

	bool FSdfAttributeSpec::SetDefaultValue(const UE::FVtValue& Value)
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;
		return Impl->PxrSdfAttributeSpec.Get()->SetDefaultValue(Value.GetUsdValue());
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	FString FSdfAttributeSpec::GetName() const
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;
		return UTF8_TO_TCHAR(Impl->PxrSdfAttributeSpec.Get()->GetName().c_str());
#else
		return {};
#endif	  // #if USE_USD_SDK
	}
}	 // namespace UE
