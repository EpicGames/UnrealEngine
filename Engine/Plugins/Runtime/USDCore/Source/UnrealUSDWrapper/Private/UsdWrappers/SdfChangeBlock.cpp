// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdWrappers/SdfChangeBlock.h"

#include "USDMemory.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/sdf/changeBlock.h"
#include "USDIncludesEnd.h"
#endif	  // #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
		class FSdfChangeBlockImpl
		{
#if USE_USD_SDK
			pxr::SdfChangeBlock SdfChangeBlock;
#endif	  // #if USE_USD_SDK
		};
	}

	FSdfChangeBlock::FSdfChangeBlock()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfChangeBlockImpl>();
	}

	FSdfChangeBlock::~FSdfChangeBlock()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}
}
