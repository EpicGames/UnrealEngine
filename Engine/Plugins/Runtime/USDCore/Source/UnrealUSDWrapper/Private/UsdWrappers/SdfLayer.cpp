// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdWrappers/SdfLayer.h"

#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/SdfPrimSpec.h"

#include "USDMemory.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/sdf/layer.h"
#include "pxr/usd/sdf/layerUtils.h"
#include "pxr/usd/sdf/primSpec.h"
#include "USDIncludesEnd.h"
#endif	  // #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
#if defined(ENABLE_USD_DEBUG_PATH) && ENABLE_USD_DEBUG_PATH
		// The only purpose of this class is so that we can cast the UsdStage impl to this type in the Unreal.natvis debug
		// visualizers, allowing us to inspect these FUsdStage and FUsdStageWeak variables in the debugger.
		//
		// Unlike the other wrapper types, we need these for FUsdStage because it is templated, and we can't express templated
		// type cast on the .natvis files very well.
		class UNREALUSDWRAPPER_API FSdfLayerDebugImpl
		{
			FString DebugIdentifier;
			TUsdStore<int> FakePointer;
		};
#endif	  // ENABLE_USD_DEBUG_PATH

		template<typename PtrType>
		class FSdfLayerImpl
		{
		public:
			FSdfLayerImpl() = default;

#if defined(ENABLE_USD_DEBUG_PATH) && ENABLE_USD_DEBUG_PATH
			FString DebugIdentifier;
#endif

			explicit FSdfLayerImpl(const PtrType& InUsdPtr)
				: UsdPtr(InUsdPtr)
			{
#if defined(ENABLE_USD_DEBUG_PATH) && ENABLE_USD_DEBUG_PATH
				if (const PtrType& Ptr = GetInner())
				{
					DebugIdentifier = FString{UTF8_TO_TCHAR(Ptr->GetIdentifier().c_str())};
				}
#endif
			}

			explicit FSdfLayerImpl(PtrType&& InUsdPtr)
				: UsdPtr(MoveTemp(InUsdPtr))
			{
#if defined(ENABLE_USD_DEBUG_PATH) && ENABLE_USD_DEBUG_PATH
				if (const PtrType& Ptr = GetInner())
				{
					DebugIdentifier = FString{UTF8_TO_TCHAR(Ptr->GetIdentifier().c_str())};
				}
#endif
			}

			PtrType& GetInner()
			{
				return UsdPtr.Get();
			}

			const PtrType& GetInner() const
			{
				return UsdPtr.Get();
			}

			TUsdStore<PtrType> UsdPtr;
		};
	}	 // namespace Internal

	template<typename PtrType>
	FSdfLayerBase<PtrType>::FSdfLayerBase()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfLayerImpl<PtrType>>();
	}

	template<typename PtrType>
	FSdfLayerBase<PtrType>::FSdfLayerBase(const FSdfLayer& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfLayerImpl<PtrType>>(Other.Impl->GetInner());
	}

	template<typename PtrType>
	FSdfLayerBase<PtrType>::FSdfLayerBase(FSdfLayer&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfLayerImpl<PtrType>>(MoveTemp(Other.Impl->GetInner()));
	}

	template<typename PtrType>
	FSdfLayerBase<PtrType>::FSdfLayerBase(const FSdfLayerWeak& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfLayerImpl<PtrType>>(Other.Impl->GetInner());
	}

	template<typename PtrType>
	FSdfLayerBase<PtrType>::FSdfLayerBase(FSdfLayerWeak&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfLayerImpl<PtrType>>(MoveTemp(Other.Impl->GetInner()));
	}

	template<typename PtrType>
	FSdfLayerBase<PtrType>& FSdfLayerBase<PtrType>::operator=(const FSdfLayer& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfLayerImpl<PtrType>>(Other.Impl->GetInner());
		return *this;
	}

	template<typename PtrType>
	FSdfLayerBase<PtrType>& FSdfLayerBase<PtrType>::operator=(FSdfLayer&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfLayerImpl<PtrType>>(MoveTemp(Other.Impl->GetInner()));
		return *this;
	}

	template<typename PtrType>
	FSdfLayerBase<PtrType>& FSdfLayerBase<PtrType>::operator=(const FSdfLayerWeak& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfLayerImpl<PtrType>>(Other.Impl->GetInner());
		return *this;
	}

	template<typename PtrType>
	FSdfLayerBase<PtrType>& FSdfLayerBase<PtrType>::operator=(FSdfLayerWeak&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfLayerImpl<PtrType>>(MoveTemp(Other.Impl->GetInner()));
		return *this;
	}

	template<typename PtrType>
	FSdfLayerBase<PtrType>::~FSdfLayerBase()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	template<typename PtrType>
	template<typename OtherPtrType>
	bool FSdfLayerBase<PtrType>::operator==(const FSdfLayerBase<OtherPtrType>& Other) const
	{
		return Impl->GetInner() == Other.Impl->GetInner();
	}

	template bool UNREALUSDWRAPPER_API FSdfLayer::operator==(const FSdfLayer& Other) const;
	template bool UNREALUSDWRAPPER_API FSdfLayer::operator==(const FSdfLayerWeak& Other) const;
	template bool UNREALUSDWRAPPER_API FSdfLayerWeak::operator==(const FSdfLayer& Other) const;
	template bool UNREALUSDWRAPPER_API FSdfLayerWeak::operator==(const FSdfLayerWeak& Other) const;

	template<typename PtrType>
	FSdfLayerBase<PtrType>::operator bool() const
	{
		return (bool)Impl->GetInner();
	}

	UNREALUSDWRAPPER_API uint32 GetTypeHash(const UE::FSdfLayerWeak& Layer)
	{
		uint32 Result = 0;
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;

		// Using TfHash will defer back to TfHashAppend for the type, which should use the implementation for
		// TfWeakPtrFacade, which is how TfWeakPtr seem to be hashed in USD.
		// We have to clamp the size_t to uint32 here but deferring to USD is likely the best approach to guarantee
		// that objects with non-colliding hashes in USD also not collide in UE.
		Result = static_cast<uint32>(pxr::TfHash()(Layer.Impl->GetInner()));
#endif	  // #if USE_USD_SDK
		return Result;
	}

#if USE_USD_SDK
	template<typename PtrType>
	FSdfLayerBase<PtrType>::FSdfLayerBase(const pxr::SdfLayerRefPtr& InSdfLayer)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfLayerImpl<PtrType>>(InSdfLayer);
	}

	template<typename PtrType>
	FSdfLayerBase<PtrType>::FSdfLayerBase(pxr::SdfLayerRefPtr&& InSdfLayer)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfLayerImpl<PtrType>>(InSdfLayer);
	}

	template<typename PtrType>
	FSdfLayerBase<PtrType>::FSdfLayerBase(const pxr::SdfLayerWeakPtr& InSdfLayer)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfLayerImpl<PtrType>>(InSdfLayer);
	}

	template<typename PtrType>
	FSdfLayerBase<PtrType>::FSdfLayerBase(pxr::SdfLayerWeakPtr&& InSdfLayer)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfLayerImpl<PtrType>>(InSdfLayer);
	}

	template<typename PtrType>
	FSdfLayerBase<PtrType>& FSdfLayerBase<PtrType>::operator=(const pxr::SdfLayerRefPtr& InSdfLayer)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfLayerImpl<PtrType>>(InSdfLayer);
		return *this;
	}

	template<typename PtrType>
	FSdfLayerBase<PtrType>& FSdfLayerBase<PtrType>::operator=(pxr::SdfLayerRefPtr&& InSdfLayer)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfLayerImpl<PtrType>>(InSdfLayer);
		return *this;
	}

	template<typename PtrType>
	FSdfLayerBase<PtrType>& FSdfLayerBase<PtrType>::operator=(const pxr::SdfLayerWeakPtr& InSdfLayer)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfLayerImpl<PtrType>>(InSdfLayer);
		return *this;
	}

	template<typename PtrType>
	FSdfLayerBase<PtrType>& FSdfLayerBase<PtrType>::operator=(pxr::SdfLayerWeakPtr&& InSdfLayer)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfLayerImpl<PtrType>>(InSdfLayer);
		return *this;
	}

	template<typename PtrType>
	FSdfLayerBase<PtrType>::operator PtrType&()
	{
		return Impl->GetInner();
	}

	template<typename PtrType>
	FSdfLayerBase<PtrType>::operator const PtrType&() const
	{
		return Impl->GetInner();
	}

	template<typename PtrType>
	FSdfLayerBase<PtrType>::operator pxr::SdfLayerRefPtr() const
	{
		return Impl->GetInner();
	}

	template<typename PtrType>
	FSdfLayerBase<PtrType>::operator pxr::SdfLayerWeakPtr() const
	{
		return Impl->GetInner();
	}
#endif	  // #if USE_USD_SDK

	template<typename PtrType>
	FString FSdfLayerBase<PtrType>::GetComment() const
	{
#if USE_USD_SDK
		if (const PtrType& Ptr = Impl->GetInner())
		{
			FScopedUsdAllocs UsdAllocs;
			return UTF8_TO_TCHAR(Ptr->GetComment().c_str());
		}
#endif	  // #if USE_USD_SDK
		return {};
	}

	template<typename PtrType>
	void FSdfLayerBase<PtrType>::SetComment(const TCHAR* Comment) const
	{
#if USE_USD_SDK
		if (const PtrType& Ptr = Impl->GetInner())
		{
			FScopedUsdAllocs UsdAllocs;
			Ptr->SetComment(TCHAR_TO_UTF8(Comment));
		}
#endif	  // #if USE_USD_SDK
	}

	template<typename PtrType>
	void FSdfLayerBase<PtrType>::TransferContent(const FSdfLayer& SourceLayer)
	{
#if USE_USD_SDK
		if (const PtrType& Ptr = Impl->GetInner())
		{
			Ptr->TransferContent(pxr::SdfLayerRefPtr{SourceLayer});
		}
#endif	  // #if USE_USD_SDK
	}

	template<typename PtrType>
	TSet<UE::FSdfLayerWeak> FSdfLayerBase<PtrType>::GetLoadedLayers()
	{
		TSet<UE::FSdfLayerWeak> Result;
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;

		pxr::SdfLayerHandleSet LoadedSdfLayers = pxr::SdfLayer::GetLoadedLayers();
		Result.Reserve(LoadedSdfLayers.size());

		for (const pxr::SdfLayerHandle& LoadedSdfLayer : LoadedSdfLayers)
		{
			Result.Add(UE::FSdfLayerWeak{LoadedSdfLayer});
		}
#endif	  // #if USE_USD_SDK
		return Result;
	}

	template<typename PtrType>
	FSdfLayer FSdfLayerBase<PtrType>::FindOrOpen(const TCHAR* Identifier, const TMap<FString, FString>& FileFormatArguments)
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		std::map<std::string, std::string> UsdFileFormatArguments;
		for (const TPair<FString, FString>& Pair : FileFormatArguments)
		{
			UsdFileFormatArguments.insert({TCHAR_TO_UTF8(*Pair.Key), TCHAR_TO_UTF8(*Pair.Value)});
		}

		return FSdfLayer(pxr::SdfLayer::FindOrOpen(TCHAR_TO_UTF8(Identifier), UsdFileFormatArguments));
#else
		return FSdfLayer();
#endif	  // #if USE_USD_SDK
	}

	template<typename PtrType>
	FSdfLayer FSdfLayerBase<PtrType>::CreateNew(const TCHAR* Identifier, const TMap<FString, FString>& FileFormatArguments)
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		std::map<std::string, std::string> UsdFileFormatArguments;
		for (const TPair<FString, FString>& Pair : FileFormatArguments)
		{
			UsdFileFormatArguments.insert({TCHAR_TO_UTF8(*Pair.Key), TCHAR_TO_UTF8(*Pair.Value)});
		}

		return FSdfLayer(pxr::SdfLayer::CreateNew(TCHAR_TO_UTF8(Identifier), UsdFileFormatArguments));
#else
		return FSdfLayer();
#endif	  // #if USE_USD_SDK
	}

	template<typename PtrType>
	TMap<FString, FString> FSdfLayerBase<PtrType>::GetFileFormatArguments() const
	{
		TMap<FString, FString> FileFormatArguments;

#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		if (const PtrType& Ptr = Impl->GetInner())
		{
			const std::map<std::string, std::string>& UsdFileFormatArguments = Ptr->GetFileFormatArguments();
			for (std::map<std::string, std::string>::const_iterator Iter = UsdFileFormatArguments.cbegin(); Iter != UsdFileFormatArguments.cend();
				 ++Iter)
			{
				FileFormatArguments.Add(UTF8_TO_TCHAR(Iter->first.c_str()), UTF8_TO_TCHAR(Iter->second.c_str()));
			}
		}
#endif	  // #if USE_USD_SDK

		return FileFormatArguments;
	}

	template<typename PtrType>
	bool FSdfLayerBase<PtrType>::Save(bool bForce) const
	{
#if USE_USD_SDK
		if (const PtrType& Ptr = Impl->GetInner())
		{
			return Ptr->Save(bForce);
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	template<typename PtrType>
	TSet<FString> FSdfLayerBase<PtrType>::GetCompositionAssetDependencies() const
	{
		TSet<FString> Results;
#if USE_USD_SDK
		if (const PtrType& Ptr = Impl->GetInner())
		{
			FScopedUsdAllocs UsdAllocs;
			const std::set<std::string> AssetDependencies = Ptr->GetCompositionAssetDependencies();

			Results.Reserve(AssetDependencies.size());
			for (const std::string& AssetDependency : AssetDependencies)
			{
				Results.Add(UTF8_TO_TCHAR(AssetDependency.c_str()));
			}
		}
#endif	  // #if USE_USD_SDK
		return Results;
	}

	template<typename PtrType>
	bool FSdfLayerBase<PtrType>::UpdateCompositionAssetDependency(const TCHAR* OldAssetPath, const TCHAR* NewAssetPath)
	{
#if USE_USD_SDK
		if (const PtrType& Ptr = Impl->GetInner())
		{
			FScopedUsdAllocs UsdAllocs;
			return Ptr->UpdateCompositionAssetDependency(
				TCHAR_TO_UTF8(OldAssetPath),
				(NewAssetPath != nullptr) ? TCHAR_TO_UTF8(NewAssetPath) : std::string()
			);
		}
#endif	  // #if USE_USD_SDK
		return false;
	}

	template<typename PtrType>
	FString FSdfLayerBase<PtrType>::GetRealPath() const
	{
#if USE_USD_SDK
		if (const PtrType& Ptr = Impl->GetInner())
		{
			FScopedUsdAllocs UsdAllocs;
			return FString(UTF8_TO_TCHAR(Ptr->GetRealPath().c_str()));
		}
#endif	  // #if USE_USD_SDK

		return FString();
	}

	template<typename PtrType>
	FString FSdfLayerBase<PtrType>::GetIdentifier() const
	{
#if USE_USD_SDK
		if (const PtrType& Ptr = Impl->GetInner())
		{
			FScopedUsdAllocs UsdAllocs;
			return FString(UTF8_TO_TCHAR(Ptr->GetIdentifier().c_str()));
		}
#endif	  // #if USE_USD_SDK

		return FString();
	}

	template<typename PtrType>
	FString FSdfLayerBase<PtrType>::GetDisplayName() const
	{
#if USE_USD_SDK
		if (const PtrType& Ptr = Impl->GetInner())
		{
			FScopedUsdAllocs UsdAllocs;
			return FString(UTF8_TO_TCHAR(Ptr->GetDisplayName().c_str()));
		}
#endif	  // #if USE_USD_SDK

		return FString();
	}

	template<typename PtrType>
	FString FSdfLayerBase<PtrType>::ComputeAbsolutePath(const FString& AssetPath) const
	{
#if USE_USD_SDK
		if (const PtrType& Ptr = Impl->GetInner())
		{
			FScopedUsdAllocs UsdAllocs;
			return FString(UTF8_TO_TCHAR(Ptr->ComputeAbsolutePath(TCHAR_TO_UTF8(*AssetPath)).c_str()));
		}
#endif	  // #if USE_USD_SDK

		return FString();
	}

	template<typename PtrType>
	bool FSdfLayerBase<PtrType>::HasStartTimeCode() const
	{
#if USE_USD_SDK
		if (const PtrType& Ptr = Impl->GetInner())
		{
			return Ptr->HasStartTimeCode();
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	template<typename PtrType>
	bool FSdfLayerBase<PtrType>::HasEndTimeCode() const
	{
#if USE_USD_SDK
		if (const PtrType& Ptr = Impl->GetInner())
		{
			return Ptr->HasEndTimeCode();
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	template<typename PtrType>
	double FSdfLayerBase<PtrType>::GetStartTimeCode() const
	{
#if USE_USD_SDK
		if (const PtrType& Ptr = Impl->GetInner())
		{
			return Ptr->GetStartTimeCode();
		}
#endif	  // #if USE_USD_SDK

		return 0.0;
	}

	template<typename PtrType>
	double FSdfLayerBase<PtrType>::GetEndTimeCode() const
	{
#if USE_USD_SDK
		if (const PtrType& Ptr = Impl->GetInner())
		{
			return Ptr->GetEndTimeCode();
		}
#endif	  // #if USE_USD_SDK

		return 0.0;
	}

	template<typename PtrType>
	void FSdfLayerBase<PtrType>::SetStartTimeCode(double TimeCode)
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			if (!Ptr->HasStartTimeCode() || !FMath::IsNearlyEqual(TimeCode, Ptr->GetStartTimeCode()))
			{
				Ptr->SetStartTimeCode(TimeCode);
			}
		}
#endif	  // #if USE_USD_SDK
	}

	template<typename PtrType>
	void FSdfLayerBase<PtrType>::SetEndTimeCode(double TimeCode)
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			if (!Ptr->HasEndTimeCode() || !FMath::IsNearlyEqual(TimeCode, Ptr->GetEndTimeCode()))
			{
				Ptr->SetEndTimeCode(TimeCode);
			}
		}
#endif	  // #if USE_USD_SDK
	}

	template<typename PtrType>
	bool FSdfLayerBase<PtrType>::HasTimeCodesPerSecond() const
	{
#if USE_USD_SDK
		if (const PtrType& Ptr = Impl->GetInner())
		{
			return Ptr->HasTimeCodesPerSecond();
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	template<typename PtrType>
	double FSdfLayerBase<PtrType>::GetTimeCodesPerSecond() const
	{
#if USE_USD_SDK
		if (const PtrType& Ptr = Impl->GetInner())
		{
			return Ptr->GetTimeCodesPerSecond();
		}
#endif	  // #if USE_USD_SDK

		return 0.0;
	}

	template<typename PtrType>
	void FSdfLayerBase<PtrType>::SetTimeCodesPerSecond(double TimeCodesPerSecond)
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			return Ptr->SetTimeCodesPerSecond(TimeCodesPerSecond);
		}
#endif	  // #if USE_USD_SDK
	}

	template<typename PtrType>
	bool FSdfLayerBase<PtrType>::HasFramesPerSecond() const
	{
#if USE_USD_SDK
		if (const PtrType& Ptr = Impl->GetInner())
		{
			return Ptr->HasFramesPerSecond();
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	template<typename PtrType>
	double FSdfLayerBase<PtrType>::GetFramesPerSecond() const
	{
#if USE_USD_SDK
		if (const PtrType& Ptr = Impl->GetInner())
		{
			return Ptr->GetFramesPerSecond();
		}
#endif	  // #if USE_USD_SDK

		return 0.0;
	}

	template<typename PtrType>
	void FSdfLayerBase<PtrType>::SetFramesPerSecond(double FramesPerSecond)
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			return Ptr->SetFramesPerSecond(FramesPerSecond);
		}
#endif	  // #if USE_USD_SDK
	}

	template<typename PtrType>
	TArray<FString> FSdfLayerBase<PtrType>::GetSubLayerPaths() const
	{
		TArray<FString> SubLayerPaths;

#if USE_USD_SDK
		if (const PtrType& Ptr = Impl->GetInner())
		{
			FScopedUsdAllocs UsdAllocs;

			for (const std::string& SubLayerPath : Ptr->GetSubLayerPaths())
			{
				const auto ConvertedPath = StringCast<TCHAR>(reinterpret_cast<const UTF8CHAR*>(SubLayerPath.c_str()));
				SubLayerPaths.Emplace(ConvertedPath.Get());
			}
		}
#endif	  // #if USE_USD_SDK

		return SubLayerPaths;
	}

	template<typename PtrType>
	void FSdfLayerBase<PtrType>::SetSubLayerPaths(const TArray<FString>& NewPaths)
	{
#if USE_USD_SDK
		if (const PtrType& Ptr = Impl->GetInner())
		{
			FScopedUsdAllocs UsdAllocs;

			std::vector<std::string> SubLayerPaths;
			SubLayerPaths.reserve(NewPaths.Num());

			for (const FString& NewPath : NewPaths)
			{
				const auto ConvertedPath = StringCast<UTF8CHAR>(*NewPath);
				std::string SubLayerPath = reinterpret_cast<const char*>(ConvertedPath.Get());
				SubLayerPaths.push_back(MoveTemp(SubLayerPath));
			}

			Ptr->SetSubLayerPaths(SubLayerPaths);
		}
#endif	  // #if USE_USD_SDK
	}

	template<typename PtrType>
	int64 FSdfLayerBase<PtrType>::GetNumSubLayerPaths() const
	{
#if USE_USD_SDK
		if (const PtrType& Ptr = Impl->GetInner())
		{
			return (int64)Ptr->GetNumSubLayerPaths();
		}
#endif	  // #if USE_USD_SDK

		return 0;
	}

	template<typename PtrType>
	void FSdfLayerBase<PtrType>::InsertSubLayerPath(const FString& Path, int32 Index)
	{
#if USE_USD_SDK
		if (const PtrType& Ptr = Impl->GetInner())
		{
			FScopedUsdAllocs UsdAllocs;

			const auto ConvertedPath = StringCast<UTF8CHAR>(*Path);
			std::string SubLayerPath = reinterpret_cast<const char*>(ConvertedPath.Get());
			Ptr->InsertSubLayerPath(MoveTemp(SubLayerPath), Index);
		}
#endif	  // #if USE_USD_SDK
	}

	template<typename PtrType>
	void FSdfLayerBase<PtrType>::RemoveSubLayerPath(int32 Index)
	{
#if USE_USD_SDK
		if (const PtrType& Ptr = Impl->GetInner())
		{
			Ptr->RemoveSubLayerPath(Index);
		}
#endif	  // #if USE_USD_SDK
	}

	template<typename PtrType>
	TArray<FSdfLayerOffset> FSdfLayerBase<PtrType>::GetSubLayerOffsets() const
	{
		TArray<FSdfLayerOffset> SubLayerOffsets;

#if USE_USD_SDK
		if (const PtrType& Ptr = Impl->GetInner())
		{
			FScopedUsdAllocs UsdAllocs;

			for (const pxr::SdfLayerOffset& SubLayerOffset : Ptr->GetSubLayerOffsets())
			{
				if (SubLayerOffset.IsValid())
				{
					SubLayerOffsets.Emplace(SubLayerOffset.GetOffset(), SubLayerOffset.GetScale());
				}
				else
				{
					SubLayerOffsets.AddDefaulted();
				}
			}
		}
#endif	  // #if USE_USD_SDK

		return SubLayerOffsets;
	}

	template<typename PtrType>
	FSdfLayerOffset FSdfLayerBase<PtrType>::GetSubLayerOffset(int32 Index) const
	{
#if USE_USD_SDK
		if (const PtrType& Ptr = Impl->GetInner())
		{
			FScopedUsdAllocs UsdAllocs;

			const pxr::SdfLayerOffset& SubLayerOffset = Ptr->GetSubLayerOffset(Index);
			if (SubLayerOffset.IsValid())
			{
				return FSdfLayerOffset(SubLayerOffset.GetOffset(), SubLayerOffset.GetScale());
			}
		}
#endif	  // #if USE_USD_SDK

		return FSdfLayerOffset{};
	}

	template<typename PtrType>
	void FSdfLayerBase<PtrType>::SetSubLayerOffset(const FSdfLayerOffset& Offset, int32 Index)
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			FScopedUsdAllocs UsdAllocs;

			pxr::SdfLayerOffset UsdLayerOffset(Offset.Offset, Offset.Scale);
			Ptr->SetSubLayerOffset(MoveTemp(UsdLayerOffset), Index);
		}
#endif	  // #if USE_USD_SDK
	}

	template<typename PtrType>
	bool FSdfLayerBase<PtrType>::HasSpec(const FSdfPath& Path) const
	{
#if USE_USD_SDK
		if (const PtrType& Ptr = Impl->GetInner())
		{
			return Ptr->HasSpec(Path);
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	template<typename PtrType>
	FSdfPrimSpec FSdfLayerBase<PtrType>::GetPseudoRoot() const
	{
#if USE_USD_SDK
		if (const PtrType& Ptr = Impl->GetInner())
		{
			return FSdfPrimSpec{Ptr->GetPseudoRoot()};
		}
#endif	  // #if USE_USD_SDK

		return FSdfPrimSpec{};
	}

	template<typename PtrType>
	FSdfPrimSpec FSdfLayerBase<PtrType>::GetPrimAtPath(const FSdfPath& Path) const
	{
#if USE_USD_SDK
		if (const PtrType& Ptr = Impl->GetInner())
		{
			return FSdfPrimSpec{Ptr->GetPrimAtPath(Path)};
		}
#endif	  // #if USE_USD_SDK

		return FSdfPrimSpec{};
	}

	template<typename PtrType>
	TSet<double> FSdfLayerBase<PtrType>::ListTimeSamplesForPath(const FSdfPath& Path) const
	{
		TSet<double> TimeSamples;

#if USE_USD_SDK
		if (const PtrType& Ptr = Impl->GetInner())
		{
			FScopedUsdAllocs UsdAllocs;

			std::set<double> UsdTimeSamples = Ptr->ListTimeSamplesForPath(Path);

			for (double UsdTimeSample : UsdTimeSamples)
			{
				TimeSamples.Add(UsdTimeSample);
			}
		}
#endif	  // #if USE_USD_SDK

		return TimeSamples;
	}

	template<typename PtrType>
	void FSdfLayerBase<PtrType>::EraseTimeSample(const FSdfPath& Path, double Time)
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			return Ptr->EraseTimeSample(Path, Time);
		}
#endif	  // #if USE_USD_SDK
	}

	template<typename PtrType>
	bool FSdfLayerBase<PtrType>::IsDirty() const
	{
#if USE_USD_SDK
		if (const PtrType& Ptr = Impl->GetInner())
		{
			return Ptr->IsDirty();
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	template<typename PtrType>
	bool FSdfLayerBase<PtrType>::IsEmpty() const
	{
#if USE_USD_SDK
		if (const PtrType& Ptr = Impl->GetInner())
		{
			return Ptr->IsEmpty();
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	template<typename PtrType>
	bool FSdfLayerBase<PtrType>::IsAnonymous() const
	{
#if USE_USD_SDK
		if (const PtrType& Ptr = Impl->GetInner())
		{
			return Ptr->IsAnonymous();
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	template<typename PtrType>
	bool FSdfLayerBase<PtrType>::Export(const TCHAR* Filename, const FString& Comment, const TMap<FString, FString>& FileFormatArguments) const
	{
#if USE_USD_SDK
		if (const PtrType& Ptr = Impl->GetInner())
		{
			FScopedUsdAllocs Allocs;

			std::map<std::string, std::string> UsdFileFormatArguments;
			for (const TPair<FString, FString>& Pair : FileFormatArguments)
			{
				UsdFileFormatArguments.insert({TCHAR_TO_UTF8(*Pair.Key), TCHAR_TO_UTF8(*Pair.Value)});
			}

			return Ptr->Export(TCHAR_TO_UTF8(Filename), TCHAR_TO_UTF8(*Comment), UsdFileFormatArguments);
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	template<typename PtrType>
	void FSdfLayerBase<PtrType>::Clear()
	{
#if USE_USD_SDK
		if (const PtrType& Ptr = Impl->GetInner())
		{
			return Ptr->Clear();
		}
#endif	  // #if USE_USD_SDK
	}

	template<typename PtrType>
	void FSdfLayerBase<PtrType>::Reload(bool bForce)
	{
#if USE_USD_SDK
		if (const PtrType& Ptr = Impl->GetInner())
		{
			Ptr->Reload(bForce);
		}
#endif	  // #if USE_USD_SDK
	}

	FString FSdfLayerUtils::SdfComputeAssetPathRelativeToLayer(const FSdfLayer& Anchor, const TCHAR* AssetPath)
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;
		return FString(UTF8_TO_TCHAR(pxr::SdfComputeAssetPathRelativeToLayer(pxr::SdfLayerRefPtr(Anchor), TCHAR_TO_UTF8(AssetPath)).c_str()));
#else
		return FString();
#endif	  // #if USE_USD_SDK
	}

	template<typename PtrType>
	bool FSdfLayerBase<PtrType>::IsMuted() const
	{
#if USE_USD_SDK
		if (const PtrType& Ptr = Impl->GetInner())
		{
			return Ptr->IsMuted();
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	template<typename PtrType>
	void FSdfLayerBase<PtrType>::SetMuted(bool bMuted)
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			return Ptr->SetMuted(bMuted);
		}
#endif	  // #if USE_USD_SDK
	}

#if USE_USD_SDK
	template class UNREALUSDWRAPPER_API FSdfLayerBase<pxr::SdfLayerRefPtr>;
	template class UNREALUSDWRAPPER_API FSdfLayerBase<pxr::SdfLayerWeakPtr>;
#else
	template class UNREALUSDWRAPPER_API FSdfLayerBase<FDummyRefPtrType>;
	template class UNREALUSDWRAPPER_API FSdfLayerBase<FDummyWeakPtrType>;
#endif	  // #if USE_USD_SDK
}	 // namespace UE
