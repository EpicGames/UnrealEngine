// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdWrappers/SdfPath.h"

#include "USDMemory.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/sdf/path.h"
#include "USDIncludesEnd.h"
#endif	  // #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
		class FSdfPathImpl
		{
		public:
			FSdfPathImpl() = default;

#if USE_USD_SDK

#if defined(ENABLE_USD_DEBUG_PATH) && ENABLE_USD_DEBUG_PATH
			FString DebugPath;
#endif
			explicit FSdfPathImpl(const pxr::SdfPath& InSdfPath)
				: PxrSdfPath(InSdfPath)
			{
#if defined(ENABLE_USD_DEBUG_PATH) && ENABLE_USD_DEBUG_PATH
				DebugPath = FString(UTF8_TO_TCHAR(PxrSdfPath.Get().GetString().c_str()));
#endif
			}

			explicit FSdfPathImpl(pxr::SdfPath&& InSdfPath)
				: PxrSdfPath(MoveTemp(InSdfPath))
			{
#if defined(ENABLE_USD_DEBUG_PATH) && ENABLE_USD_DEBUG_PATH
				DebugPath = FString(UTF8_TO_TCHAR(PxrSdfPath.Get().GetString().c_str()));
#endif
			}

			TUsdStore<pxr::SdfPath> PxrSdfPath;
#endif	  // #if USE_USD_SDK
		};
	}	 // namespace Internal

	FSdfPath::FSdfPath()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfPathImpl>();
	}

	FSdfPath::FSdfPath(const FSdfPath& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfPathImpl>(Other.Impl->PxrSdfPath.Get());
#endif	  // #if USE_USD_SDK
	}

	FSdfPath::FSdfPath(FSdfPath&& Other) = default;

	FSdfPath::FSdfPath(const TCHAR* InPath)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfPathImpl>(
			FCString::Strlen(InPath) == 0 ? pxr::SdfPath()	  // USD gives a warning if we construct with an empty string
										  : pxr::SdfPath(TCHAR_TO_UTF8(InPath))
		);
#endif	  // #if USE_USD_SDK
	}

	FSdfPath& FSdfPath::operator=(const FSdfPath& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfPathImpl>(Other.Impl->PxrSdfPath.Get());
#endif	  // #if USE_USD_SDK
		return *this;
	}

	FSdfPath& FSdfPath::operator=(FSdfPath&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MoveTemp(Other.Impl);

		return *this;
	}

	FSdfPath::~FSdfPath()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	bool FSdfPath::operator==(const FSdfPath& Other) const
	{
#if USE_USD_SDK
		return Impl->PxrSdfPath.Get() == Other.Impl->PxrSdfPath.Get();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FSdfPath::operator!=(const FSdfPath& Other) const
	{
		return !(*this == Other);
	}

	bool FSdfPath::operator<(const FSdfPath& Other) const
	{
#if USE_USD_SDK
		return Impl->PxrSdfPath.Get() < Other.Impl->PxrSdfPath.Get();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FSdfPath::operator<=(const FSdfPath& Other) const
	{
#if USE_USD_SDK
		return Impl->PxrSdfPath.Get() <= Other.Impl->PxrSdfPath.Get();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	uint32 GetTypeHash(const UE::FSdfPath& Path)
	{
		uint32 Result = 0;
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Result = static_cast<uint32>(Path.Impl->PxrSdfPath.Get().GetHash());
#endif	  // #if USE_USD_SDK
		return Result;
	}

	FArchive& operator<<(FArchive& Ar, UE::FSdfPath& Path)
	{
		if (Ar.IsSaving())
		{
			FString String = Path.GetString();
			Ar << String;
		}
		else if (Ar.IsLoading())
		{
			FString String;
			Ar << String;

			Path = UE::FSdfPath{*String};
		}

		return Ar;
	}

#if USE_USD_SDK
	FSdfPath::FSdfPath(const pxr::SdfPath& InSdfPath)
		: Impl(MakeUnique<Internal::FSdfPathImpl>(InSdfPath))
	{
	}

	FSdfPath::FSdfPath(pxr::SdfPath&& InSdfPath)
		: Impl(MakeUnique<Internal::FSdfPathImpl>(MoveTemp(InSdfPath)))
	{
	}

	FSdfPath& FSdfPath::operator=(const pxr::SdfPath& InSdfPath)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfPathImpl>(InSdfPath);
		return *this;
	}

	FSdfPath& FSdfPath::operator=(pxr::SdfPath&& InSdfPath)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfPathImpl>(MoveTemp(InSdfPath));
		return *this;
	}

	FSdfPath::operator pxr::SdfPath&()
	{
		return Impl->PxrSdfPath.Get();
	}

	FSdfPath::operator const pxr::SdfPath&() const
	{
		return Impl->PxrSdfPath.Get();
	}
#endif	  // #if USE_USD_SDK

	const FSdfPath& FSdfPath::AbsoluteRootPath()
	{
#if USE_USD_SDK
		static FSdfPath AbsoluteRootPath(pxr::SdfPath::AbsoluteRootPath());
		return AbsoluteRootPath;
#else
		static FSdfPath AbsoluteRootPath;
		return AbsoluteRootPath;
#endif	  // #if USE_USD_SDK
	}

	bool FSdfPath::IsValidPathString(const TCHAR* PathString, FString* OutErrorMessage)
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		std::string ErrorMessage;
		const bool bIsValid = pxr::SdfPath::IsValidPathString(
			TCHAR_TO_UTF8(PathString),
			OutErrorMessage ? &ErrorMessage : nullptr
		);

		if (OutErrorMessage)
		{
			*OutErrorMessage = FString(UTF8_TO_TCHAR(ErrorMessage.c_str()));
		}

		return bIsValid;
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FSdfPath::IsEmpty() const noexcept
	{
#if USE_USD_SDK
		return Impl->PxrSdfPath.Get().IsEmpty();
#else
		return true;
#endif	  // #if USE_USD_SDK
	}

	bool FSdfPath::IsAbsoluteRootPath() const
	{
#if USE_USD_SDK
		return Impl->PxrSdfPath.Get().IsAbsoluteRootPath();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FSdfPath::IsPrimPath() const
	{
#if USE_USD_SDK
		return Impl->PxrSdfPath.Get().IsPrimPath();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FSdfPath::IsAbsoluteRootOrPrimPath() const
	{
#if USE_USD_SDK
		return Impl->PxrSdfPath.Get().IsAbsoluteRootOrPrimPath();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FSdfPath::IsPrimPropertyPath() const
	{
#if USE_USD_SDK
		return Impl->PxrSdfPath.Get().IsPrimPropertyPath();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FSdfPath::IsPropertyPath() const
	{
#if USE_USD_SDK
		return Impl->PxrSdfPath.Get().IsPropertyPath();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FSdfPath::IsRelationalAttributePath() const
	{
#if USE_USD_SDK
		return Impl->PxrSdfPath.Get().IsRelationalAttributePath();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FSdfPath::IsTargetPath() const
	{
#if USE_USD_SDK
		return Impl->PxrSdfPath.Get().IsTargetPath();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	FString FSdfPath::GetName() const
	{
#if USE_USD_SDK
		return FString(UTF8_TO_TCHAR(Impl->PxrSdfPath.Get().GetName().c_str()));
#else
		return FString();
#endif	  // #if USE_USD_SDK
	}

	FString FSdfPath::GetElementString() const
	{
#if USE_USD_SDK
		return FString(UTF8_TO_TCHAR(Impl->PxrSdfPath.Get().GetElementString().c_str()));
#else
		return FString();
#endif	  // #if USE_USD_SDK
	}

	FSdfPath FSdfPath::GetAbsoluteRootOrPrimPath() const
	{
#if USE_USD_SDK
		return FSdfPath(Impl->PxrSdfPath.Get().GetAbsoluteRootOrPrimPath());
#else
		return FSdfPath();
#endif	  // #if USE_USD_SDK
	}

	FSdfPath FSdfPath::ReplaceName(const TCHAR* NewLeafName) const
	{
#if USE_USD_SDK
		return FSdfPath(Impl->PxrSdfPath.Get().ReplaceName(pxr::TfToken(TCHAR_TO_UTF8(NewLeafName))));
#else
		return FSdfPath();
#endif	  // #if USE_USD_SDK
	}

	FSdfPath FSdfPath::ReplacePrefix(const UE::FSdfPath& OldPrefix,
								     const UE::FSdfPath& NewPrefix,
			                         bool FixTargetPaths) const
	{
#if USE_USD_SDK
		return FSdfPath(Impl->PxrSdfPath.Get().ReplacePrefix(OldPrefix, NewPrefix, FixTargetPaths));
#else
		return FSdfPath();
#endif	  // #if USE_USD_SDK
	}

	FSdfPath FSdfPath::GetParentPath() const
	{
#if USE_USD_SDK
		return FSdfPath(Impl->PxrSdfPath.Get().GetParentPath());
#else
		return FSdfPath();
#endif	  // #if USE_USD_SDK
	}

	FSdfPath FSdfPath::GetPrimPath() const
	{
#if USE_USD_SDK
		return FSdfPath(Impl->PxrSdfPath.Get().GetPrimPath());
#else
		return FSdfPath();
#endif	  // #if USE_USD_SDK
	}

	FSdfPath FSdfPath::AppendPath(const UE::FSdfPath& NewRelativeSuffix) const
	{
#if USE_USD_SDK
		return FSdfPath(Impl->PxrSdfPath.Get().AppendPath(NewRelativeSuffix.Impl->PxrSdfPath.Get()));
#else
		return FSdfPath();
#endif	  // #if USE_USD_SDK
	}

	FSdfPath FSdfPath::AppendChild(const TCHAR* ChildName) const
	{
#if USE_USD_SDK
		return FSdfPath(Impl->PxrSdfPath.Get().AppendChild(pxr::TfToken(TCHAR_TO_UTF8(ChildName))));
#else
		return FSdfPath();
#endif	  // #if USE_USD_SDK
	}

	FSdfPath FSdfPath::AppendProperty(FName PropertyName) const
	{
#if USE_USD_SDK
		return FSdfPath(Impl->PxrSdfPath.Get().AppendProperty(pxr::TfToken(TCHAR_TO_UTF8(*PropertyName.ToString()))));
#else
		return FSdfPath();
#endif	  // #if USE_USD_SDK
	}

	UE::FSdfPath FSdfPath::StripAllVariantSelections() const
	{
#if USE_USD_SDK
		return UE::FSdfPath(Impl->PxrSdfPath.Get().StripAllVariantSelections());
#else
		return UE::FSdfPath();
#endif	  // #if USE_USD_SDK
	}

	FString FSdfPath::GetString() const
	{
#if USE_USD_SDK
		return FString(UTF8_TO_TCHAR(Impl->PxrSdfPath.Get().GetString().c_str()));
#else
		return FString();
#endif	  // #if USE_USD_SDK
	}

	TArray<FSdfPath> FSdfPath::GetPrefixes() const
	{
		TArray<FSdfPath> Result;
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		std::vector<pxr::SdfPath> UsdPrefixes = Impl->PxrSdfPath.Get().GetPrefixes();
		Result.Reserve(UsdPrefixes.size());

		for (const pxr::SdfPath& UsdPrefix : UsdPrefixes)
		{
			Result.Add(UE::FSdfPath{UTF8_TO_TCHAR(UsdPrefix.GetString().c_str())});
		}
#endif	  // #if USE_USD_SDK
		return Result;
	}

	int32 FSdfPath::GetPathElementCount() const
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		return static_cast<int32>(Impl->PxrSdfPath.Get().GetPathElementCount());
#else
		return -1;
#endif	  // #if USE_USD_SDK
	}

	bool FSdfPath::HasPrefix(const UE::FSdfPath& Prefix) const
	{
#if USE_USD_SDK
		return Impl->PxrSdfPath.Get().HasPrefix(Prefix);
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	FSdfPath FSdfPath::MakeAbsolutePath(const UE::FSdfPath& Anchor) const
	{
#if USE_USD_SDK
		return FSdfPath(Impl->PxrSdfPath.Get().MakeAbsolutePath(Anchor.Impl->PxrSdfPath.Get()));
#else
		return FSdfPath();
#endif	  // #if USE_USD_SDK
	}

	FSdfPath FSdfPath::MakeRelativePath(const UE::FSdfPath& Anchor) const
	{
#if USE_USD_SDK
		return FSdfPath(Impl->PxrSdfPath.Get().MakeRelativePath(Anchor.Impl->PxrSdfPath.Get()));
#else
		return FSdfPath();
#endif	  // #if USE_USD_SDK
	}
}	 // namespace UE
