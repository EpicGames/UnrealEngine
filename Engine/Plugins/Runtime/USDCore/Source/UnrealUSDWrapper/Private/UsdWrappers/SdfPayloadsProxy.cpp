// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdWrappers/SdfPayloadsProxy.h"

#include "USDMemory.h"

#include "UsdWrappers/SdfAttributeSpec.h"
#include "UsdWrappers/SdfLayer.h"
#include "UsdWrappers/SdfPath.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/sdf/listEditorProxy.h"
#include "pxr/usd/sdf/proxyPolicies.h"
#include "pxr/usd/sdf/payload.h"
#include "USDIncludesEnd.h"
#endif	  // #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
		class FSdfPayloadsProxyImpl
		{
		public:
			FSdfPayloadsProxyImpl() = default;

#if USE_USD_SDK
			explicit FSdfPayloadsProxyImpl(const pxr::SdfPayloadsProxy& InSdfPayloadsProxy)
				: PxrSdfPayloadsProxy(InSdfPayloadsProxy)
			{
			}

			explicit FSdfPayloadsProxyImpl(pxr::SdfPayloadsProxy&& InSdfPayloadsProxy)
				: PxrSdfPayloadsProxy(MoveTemp(InSdfPayloadsProxy))
			{
			}

			TUsdStore<pxr::SdfPayloadsProxy> PxrSdfPayloadsProxy;
#endif	  // #if USE_USD_SDK
		};
	}

	FSdfPayloadsProxy::FSdfPayloadsProxy()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfPayloadsProxyImpl>();
	}

	FSdfPayloadsProxy::FSdfPayloadsProxy(const FSdfPayloadsProxy& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfPayloadsProxyImpl>(Other.Impl->PxrSdfPayloadsProxy.Get());
#endif	  // #if USE_USD_SDK
	}

	FSdfPayloadsProxy::FSdfPayloadsProxy(FSdfPayloadsProxy&& Other) = default;

	FSdfPayloadsProxy::~FSdfPayloadsProxy()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	FSdfPayloadsProxy& FSdfPayloadsProxy::operator=(const FSdfPayloadsProxy& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfPayloadsProxyImpl>(Other.Impl->PxrSdfPayloadsProxy.Get());
#endif	  // #if USE_USD_SDK
		return *this;
	}

	FSdfPayloadsProxy& FSdfPayloadsProxy::operator=(FSdfPayloadsProxy&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MoveTemp(Other.Impl);

		return *this;
	}

	FSdfPayloadsProxy::operator bool() const
	{
#if USE_USD_SDK
		return (bool)Impl->PxrSdfPayloadsProxy.Get();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

#if USE_USD_SDK
	FSdfPayloadsProxy::FSdfPayloadsProxy(const pxr::SdfPayloadsProxy& InSdfPayloadsProxy)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfPayloadsProxyImpl>(InSdfPayloadsProxy);
	}

	FSdfPayloadsProxy::FSdfPayloadsProxy(pxr::SdfPayloadsProxy&& InSdfPayloadsProxy)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfPayloadsProxyImpl>(MoveTemp(InSdfPayloadsProxy));
	}

	FSdfPayloadsProxy& FSdfPayloadsProxy::operator=(const pxr::SdfPayloadsProxy& InSdfPayloadsProxy)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfPayloadsProxyImpl>(InSdfPayloadsProxy);
		return *this;
	}

	FSdfPayloadsProxy& FSdfPayloadsProxy::operator=(pxr::SdfPayloadsProxy&& InSdfPayloadsProxy)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfPayloadsProxyImpl>(MoveTemp(InSdfPayloadsProxy));
		return *this;
	}

	FSdfPayloadsProxy::operator pxr::SdfPayloadsProxy&()
	{
		return Impl->PxrSdfPayloadsProxy.Get();
	}

	FSdfPayloadsProxy::operator const pxr::SdfPayloadsProxy&() const
	{
		return Impl->PxrSdfPayloadsProxy.Get();
	}
#endif	  // #if USE_USD_SDK

	TArray<FSdfPayload> FSdfPayloadsProxy::GetAppliedItems() const
	{
		TArray<FSdfPayload> Result;
#if USE_USD_SDK
		TUsdStore<std::vector<pxr::SdfPayload>> SdfPayloads = Impl->PxrSdfPayloadsProxy->GetAppliedItems();
		Result.Reserve(SdfPayloads->size());

		for (const pxr::SdfPayload& SdfPayload : *SdfPayloads)
		{
			const pxr::SdfLayerOffset& SdfOffset = SdfPayload.GetLayerOffset();

			Result.Add(FSdfPayload{
				FString{UTF8_TO_TCHAR(SdfPayload.GetAssetPath().c_str())},
				FSdfPath{SdfPayload.GetPrimPath()},
				FSdfLayerOffset{SdfOffset.GetOffset(), SdfOffset.GetScale()}
			});
		}
#endif	  // USE_USD_SDK
		return Result;
	}
}	 // namespace UE
