// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdWrappers/SdfPrimSpec.h"

#include "USDMemory.h"

#include "UsdWrappers/SdfAttributeSpec.h"
#include "UsdWrappers/SdfLayer.h"
#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/SdfPayloadsProxy.h"
#include "UsdWrappers/SdfReferencesProxy.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/sdf/attributeSpec.h"
#include "pxr/usd/sdf/primSpec.h"
#include "pxr/usd/sdf/layer.h"
#include "USDIncludesEnd.h"
#endif	  // #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
		class FSdfPrimSpecImpl
		{
		public:
			FSdfPrimSpecImpl() = default;

#if USE_USD_SDK

#if defined(ENABLE_USD_DEBUG_PATH) && ENABLE_USD_DEBUG_PATH
			FString DebugPath;
			FString DebugLayerPath;
#endif

			explicit FSdfPrimSpecImpl(const pxr::SdfPrimSpecHandle& InSdfPrimSpec)
				: PxrSdfPrimSpec(InSdfPrimSpec)
			{
#if defined(ENABLE_USD_DEBUG_PATH) && ENABLE_USD_DEBUG_PATH
				SetupDebugMembers();
#endif
			}

			explicit FSdfPrimSpecImpl(pxr::SdfPrimSpecHandle&& InSdfPrimSpec)
				: PxrSdfPrimSpec(MoveTemp(InSdfPrimSpec))
			{
#if defined(ENABLE_USD_DEBUG_PATH) && ENABLE_USD_DEBUG_PATH
				SetupDebugMembers();
#endif
			}

#if defined(ENABLE_USD_DEBUG_PATH) && ENABLE_USD_DEBUG_PATH
			void SetupDebugMembers()
			{
				if (pxr::SdfPrimSpecHandle ValidSpec = PxrSdfPrimSpec.Get())
				{
					DebugPath = FString{UTF8_TO_TCHAR(ValidSpec->GetPath().GetString().c_str())};

					if (pxr::SdfLayerHandle ValidLayer = ValidSpec->GetLayer())
					{
						DebugLayerPath = FString{UTF8_TO_TCHAR(ValidLayer->GetIdentifier().c_str())};
					}
				}
			}
#endif

			TUsdStore<pxr::SdfPrimSpecHandle> PxrSdfPrimSpec;
#endif	  // #if USE_USD_SDK
		};
	}

	FSdfPrimSpec::FSdfPrimSpec()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfPrimSpecImpl>();
	}

	FSdfPrimSpec::FSdfPrimSpec(const FSdfPrimSpec& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfPrimSpecImpl>(Other.Impl->PxrSdfPrimSpec.Get());
#endif	  // #if USE_USD_SDK
	}

	FSdfPrimSpec::FSdfPrimSpec(FSdfPrimSpec&& Other) = default;

	FSdfPrimSpec::~FSdfPrimSpec()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	FSdfPrimSpec& FSdfPrimSpec::operator=(const FSdfPrimSpec& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfPrimSpecImpl>(Other.Impl->PxrSdfPrimSpec.Get());
#endif	  // #if USE_USD_SDK
		return *this;
	}

	FSdfPrimSpec& FSdfPrimSpec::operator=(FSdfPrimSpec&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MoveTemp(Other.Impl);

		return *this;
	}

	FSdfPrimSpec::operator bool() const
	{
#if USE_USD_SDK
		return (bool)Impl->PxrSdfPrimSpec.Get();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FSdfPrimSpec::operator==(const FSdfPrimSpec& Other) const
	{
#if USE_USD_SDK
		return Impl->PxrSdfPrimSpec.Get() == Other.Impl->PxrSdfPrimSpec.Get();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FSdfPrimSpec::operator!=(const FSdfPrimSpec& Other) const
	{
		return !(*this == Other);
	}

#if USE_USD_SDK
	FSdfPrimSpec::FSdfPrimSpec(const pxr::SdfPrimSpecHandle& InSdfPrimSpec)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfPrimSpecImpl>(InSdfPrimSpec);
	}

	FSdfPrimSpec::FSdfPrimSpec(pxr::SdfPrimSpecHandle&& InSdfPrimSpec)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfPrimSpecImpl>(MoveTemp(InSdfPrimSpec));
	}

	FSdfPrimSpec& FSdfPrimSpec::operator=(const pxr::SdfPrimSpecHandle& InSdfPrimSpec)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfPrimSpecImpl>(InSdfPrimSpec);
		return *this;
	}

	FSdfPrimSpec& FSdfPrimSpec::operator=(pxr::SdfPrimSpecHandle&& InSdfPrimSpec)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfPrimSpecImpl>(MoveTemp(InSdfPrimSpec));
		return *this;
	}

	FSdfPrimSpec::operator pxr::SdfPrimSpecHandle&()
	{
		return Impl->PxrSdfPrimSpec.Get();
	}

	FSdfPrimSpec::operator const pxr::SdfPrimSpecHandle&() const
	{
		return Impl->PxrSdfPrimSpec.Get();
	}
#endif	  // #if USE_USD_SDK

	ESdfSpecType FSdfPrimSpec::GetSpecType() const
	{
#if USE_USD_SDK
		// Just in case they add another or reorder these
		static_assert((int)ESdfSpecType::SdfSpecTypeUnknown == (int)pxr::SdfSpecType::SdfSpecTypeUnknown);
		static_assert((int)ESdfSpecType::SdfSpecTypeAttribute == (int)pxr::SdfSpecType::SdfSpecTypeAttribute);
		static_assert((int)ESdfSpecType::SdfSpecTypeConnection == (int)pxr::SdfSpecType::SdfSpecTypeConnection);
		static_assert((int)ESdfSpecType::SdfSpecTypeExpression == (int)pxr::SdfSpecType::SdfSpecTypeExpression);
		static_assert((int)ESdfSpecType::SdfSpecTypeMapper == (int)pxr::SdfSpecType::SdfSpecTypeMapper);
		static_assert((int)ESdfSpecType::SdfSpecTypeMapperArg == (int)pxr::SdfSpecType::SdfSpecTypeMapperArg);
		static_assert((int)ESdfSpecType::SdfSpecTypePrim == (int)pxr::SdfSpecType::SdfSpecTypePrim);
		static_assert((int)ESdfSpecType::SdfSpecTypePseudoRoot == (int)pxr::SdfSpecType::SdfSpecTypePseudoRoot);
		static_assert((int)ESdfSpecType::SdfSpecTypeRelationship == (int)pxr::SdfSpecType::SdfSpecTypeRelationship);
		static_assert((int)ESdfSpecType::SdfSpecTypeRelationshipTarget == (int)pxr::SdfSpecType::SdfSpecTypeRelationshipTarget);
		static_assert((int)ESdfSpecType::SdfSpecTypeVariant == (int)pxr::SdfSpecType::SdfSpecTypeVariant);
		static_assert((int)ESdfSpecType::SdfSpecTypeVariantSet == (int)pxr::SdfSpecType::SdfSpecTypeVariantSet);
		static_assert((int)ESdfSpecType::SdfNumSpecTypes == (int)pxr::SdfSpecType::SdfNumSpecTypes);

		return static_cast<ESdfSpecType>(Impl->PxrSdfPrimSpec.Get()->GetSpecType());
#else
		return ESdfSpecType::SdfSpecTypeUnknown;
#endif	  // #if USE_USD_SDK
	}

	FSdfLayerWeak FSdfPrimSpec::GetLayer() const
	{
#if USE_USD_SDK
		return FSdfLayerWeak(Impl->PxrSdfPrimSpec.Get()->GetLayer());
#else
		return FSdfLayerWeak();
#endif	  // #if USE_USD_SDK
	}

	FSdfPath FSdfPrimSpec::GetPath() const
	{
#if USE_USD_SDK
		return FSdfPath(Impl->PxrSdfPrimSpec.Get()->GetPath());
#else
		return FSdfPath();
#endif	  // #if USE_USD_SDK
	}

	FSdfPrimSpec FSdfPrimSpec::GetRealNameParent() const
	{
#if USE_USD_SDK
		return FSdfPrimSpec{Impl->PxrSdfPrimSpec.Get()->GetRealNameParent()};
#else
		return FSdfPrimSpec{};
#endif	  // #if USE_USD_SDK
	}

	bool FSdfPrimSpec::RemoveNameChild(const FSdfPrimSpec& Child)
	{
#if USE_USD_SDK
		return Impl->PxrSdfPrimSpec.Get()->RemoveNameChild(Child);
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	UE::FSdfAttributeSpec FSdfPrimSpec::GetAttributeAtPath(const UE::FSdfPath& Path) const
	{
#if USE_USD_SDK
		return UE::FSdfAttributeSpec{Impl->PxrSdfPrimSpec.Get()->GetAttributeAtPath(Path)};
#else
		return UE::FSdfAttributeSpec{};
#endif	  // #if USE_USD_SDK
	}

	FSdfPayloadsProxy FSdfPrimSpec::GetPayloadList() const
	{
#if USE_USD_SDK
		return UE::FSdfPayloadsProxy{Impl->PxrSdfPrimSpec.Get()->GetPayloadList()};
#else
		return UE::FSdfPayloadsProxy{};
#endif	  // #if USE_USD_SDK
	}

	FSdfReferencesProxy FSdfPrimSpec::GetReferenceList() const
	{
#if USE_USD_SDK
		return UE::FSdfReferencesProxy{Impl->PxrSdfPrimSpec.Get()->GetReferenceList()};
#else
		return UE::FSdfReferencesProxy{};
#endif	  // #if USE_USD_SDK
	}

	FName FSdfPrimSpec::GetTypeName() const
	{
#if USE_USD_SDK
		return FName(UTF8_TO_TCHAR(Impl->PxrSdfPrimSpec.Get()->GetTypeName().GetString().c_str()));
#else
		return FName();
#endif	  // #if USE_USD_SDK
	}

	FName FSdfPrimSpec::GetName() const
	{
#if USE_USD_SDK
		return FName(UTF8_TO_TCHAR(Impl->PxrSdfPrimSpec.Get()->GetName().c_str()));
#else
		return FName();
#endif	  // #if USE_USD_SDK
	}
}	 // namespace UE
