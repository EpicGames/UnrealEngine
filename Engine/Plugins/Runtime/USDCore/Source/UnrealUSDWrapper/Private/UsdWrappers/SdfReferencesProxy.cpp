// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdWrappers/SdfReferencesProxy.h"

#include "USDMemory.h"

#include "UsdWrappers/SdfAttributeSpec.h"
#include "UsdWrappers/SdfLayer.h"
#include "UsdWrappers/SdfPath.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/sdf/listEditorProxy.h"
#include "pxr/usd/sdf/proxyPolicies.h"
#include "pxr/usd/sdf/reference.h"
#include "USDIncludesEnd.h"
#endif	  // #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
		class FSdfReferencesProxyImpl
		{
		public:
			FSdfReferencesProxyImpl() = default;

#if USE_USD_SDK
			explicit FSdfReferencesProxyImpl(const pxr::SdfReferencesProxy& InSdfReferencesProxy)
				: PxrSdfReferencesProxy(InSdfReferencesProxy)
			{
			}

			explicit FSdfReferencesProxyImpl(pxr::SdfReferencesProxy&& InSdfReferencesProxy)
				: PxrSdfReferencesProxy(MoveTemp(InSdfReferencesProxy))
			{
			}

			TUsdStore<pxr::SdfReferencesProxy> PxrSdfReferencesProxy;
#endif	  // #if USE_USD_SDK
		};
	}

	FSdfReferencesProxy::FSdfReferencesProxy()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfReferencesProxyImpl>();
	}

	FSdfReferencesProxy::FSdfReferencesProxy(const FSdfReferencesProxy& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfReferencesProxyImpl>(Other.Impl->PxrSdfReferencesProxy.Get());
#endif	  // #if USE_USD_SDK
	}

	FSdfReferencesProxy::FSdfReferencesProxy(FSdfReferencesProxy&& Other) = default;

	FSdfReferencesProxy::~FSdfReferencesProxy()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	FSdfReferencesProxy& FSdfReferencesProxy::operator=(const FSdfReferencesProxy& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfReferencesProxyImpl>(Other.Impl->PxrSdfReferencesProxy.Get());
#endif	  // #if USE_USD_SDK
		return *this;
	}

	FSdfReferencesProxy& FSdfReferencesProxy::operator=(FSdfReferencesProxy&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MoveTemp(Other.Impl);

		return *this;
	}

	FSdfReferencesProxy::operator bool() const
	{
#if USE_USD_SDK
		return (bool)Impl->PxrSdfReferencesProxy.Get();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

#if USE_USD_SDK
	FSdfReferencesProxy::FSdfReferencesProxy(const pxr::SdfReferencesProxy& InSdfReferencesProxy)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfReferencesProxyImpl>(InSdfReferencesProxy);
	}

	FSdfReferencesProxy::FSdfReferencesProxy(pxr::SdfReferencesProxy&& InSdfReferencesProxy)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfReferencesProxyImpl>(MoveTemp(InSdfReferencesProxy));
	}

	FSdfReferencesProxy& FSdfReferencesProxy::operator=(const pxr::SdfReferencesProxy& InSdfReferencesProxy)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfReferencesProxyImpl>(InSdfReferencesProxy);
		return *this;
	}

	FSdfReferencesProxy& FSdfReferencesProxy::operator=(pxr::SdfReferencesProxy&& InSdfReferencesProxy)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSdfReferencesProxyImpl>(MoveTemp(InSdfReferencesProxy));
		return *this;
	}

	FSdfReferencesProxy::operator pxr::SdfReferencesProxy&()
	{
		return Impl->PxrSdfReferencesProxy.Get();
	}

	FSdfReferencesProxy::operator const pxr::SdfReferencesProxy&() const
	{
		return Impl->PxrSdfReferencesProxy.Get();
	}
#endif	  // #if USE_USD_SDK

	TArray<FSdfReference> FSdfReferencesProxy::GetAppliedItems() const
	{
		TArray<FSdfReference> Result;
#if USE_USD_SDK
		TUsdStore<std::vector<pxr::SdfReference>> SdfReferences = Impl->PxrSdfReferencesProxy->GetAppliedItems();
		Result.Reserve(SdfReferences->size());

		for (const pxr::SdfReference& SdfReference : *SdfReferences)
		{
			const pxr::SdfLayerOffset& SdfOffset = SdfReference.GetLayerOffset();

			Result.Add(FSdfReference{
				FString{UTF8_TO_TCHAR(SdfReference.GetAssetPath().c_str())},
				FSdfPath{SdfReference.GetPrimPath()},
				FSdfLayerOffset{SdfOffset.GetOffset(), SdfOffset.GetScale()}
			});
		}
#endif	  // USE_USD_SDK
		return Result;
	}
}	 // namespace UE
