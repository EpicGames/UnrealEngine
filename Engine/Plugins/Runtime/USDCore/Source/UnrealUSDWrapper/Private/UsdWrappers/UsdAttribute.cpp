// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdWrappers/UsdAttribute.h"

#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdResolveInfo.h"
#include "UsdWrappers/VtValue.h"

#include "USDMemory.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/usd/attribute.h"
#include "USDIncludesEnd.h"
#endif	  // #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
		class FUsdAttributeImpl
		{
		public:
			FUsdAttributeImpl() = default;

#if USE_USD_SDK

#if defined(ENABLE_USD_DEBUG_PATH) && ENABLE_USD_DEBUG_PATH
			FString DebugPath;
#endif

			explicit FUsdAttributeImpl(const pxr::UsdAttribute& InUsdAttribute)
				: PxrUsdAttribute(InUsdAttribute)
			{
#if defined(ENABLE_USD_DEBUG_PATH) && ENABLE_USD_DEBUG_PATH
				DebugPath = FString{UTF8_TO_TCHAR(PxrUsdAttribute.Get().GetPath().GetString().c_str())};
#endif
			}

			explicit FUsdAttributeImpl(pxr::UsdAttribute&& InUsdAttribute)
				: PxrUsdAttribute(MoveTemp(InUsdAttribute))
			{
#if defined(ENABLE_USD_DEBUG_PATH) && ENABLE_USD_DEBUG_PATH
				DebugPath = FString{UTF8_TO_TCHAR(PxrUsdAttribute.Get().GetPath().GetString().c_str())};
#endif
			}

			TUsdStore<pxr::UsdAttribute> PxrUsdAttribute;
#endif	  // #if USE_USD_SDK
		};
	}	 // namespace Internal

	FUsdAttribute::FUsdAttribute()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdAttributeImpl>();
	}

	FUsdAttribute::FUsdAttribute(const FUsdAttribute& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdAttributeImpl>(Other.Impl->PxrUsdAttribute.Get());
#endif	  // #if USE_USD_SDK
	}

	FUsdAttribute::FUsdAttribute(FUsdAttribute&& Other) = default;

	FUsdAttribute::~FUsdAttribute()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	FUsdAttribute& FUsdAttribute::operator=(const FUsdAttribute& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdAttributeImpl>(Other.Impl->PxrUsdAttribute.Get());
#endif	  // #if USE_USD_SDK
		return *this;
	}

	FUsdAttribute& FUsdAttribute::operator=(FUsdAttribute&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MoveTemp(Other.Impl);

		return *this;
	}

	bool FUsdAttribute::operator==(const FUsdAttribute& Other) const
	{
#if USE_USD_SDK
		return Impl->PxrUsdAttribute.Get() == Other.Impl->PxrUsdAttribute.Get();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdAttribute::operator!=(const FUsdAttribute& Other) const
	{
		return !(*this == Other);
	}

	FUsdAttribute::operator bool() const
	{
#if USE_USD_SDK
		return (bool)Impl->PxrUsdAttribute.Get();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

#if USE_USD_SDK
	FUsdAttribute::FUsdAttribute(const pxr::UsdAttribute& InUsdAttribute)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdAttributeImpl>(InUsdAttribute);
	}

	FUsdAttribute::FUsdAttribute(pxr::UsdAttribute&& InUsdAttribute)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdAttributeImpl>(MoveTemp(InUsdAttribute));
	}

	FUsdAttribute& FUsdAttribute::operator=(const pxr::UsdAttribute& InUsdAttribute)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdAttributeImpl>(InUsdAttribute);
		return *this;
	}

	FUsdAttribute& FUsdAttribute::operator=(pxr::UsdAttribute&& InUsdAttribute)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdAttributeImpl>(MoveTemp(InUsdAttribute));
		return *this;
	}

	FUsdAttribute::operator pxr::UsdAttribute&()
	{
		return Impl->PxrUsdAttribute.Get();
	}

	FUsdAttribute::operator const pxr::UsdAttribute&() const
	{
		return Impl->PxrUsdAttribute.Get();
	}

	FUsdAttribute::operator pxr::UsdProperty&()
	{
		return Impl->PxrUsdAttribute.Get();
	}

	FUsdAttribute::operator const pxr::UsdProperty&() const
	{
		return Impl->PxrUsdAttribute.Get();
	}
#endif	  // #if USE_USD_SDK

	bool FUsdAttribute::GetMetadata(const TCHAR* Key, UE::FVtValue& Value) const
	{
#if USE_USD_SDK
		return Impl->PxrUsdAttribute.Get().GetMetadata(pxr::TfToken{TCHAR_TO_UTF8(Key)}, &Value.GetUsdValue());
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdAttribute::HasMetadata(const TCHAR* Key) const
	{
#if USE_USD_SDK
		return Impl->PxrUsdAttribute.Get().HasMetadata(pxr::TfToken{TCHAR_TO_UTF8(Key)});
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdAttribute::SetMetadata(const TCHAR* Key, const UE::FVtValue& Value) const
	{
#if USE_USD_SDK
		return Impl->PxrUsdAttribute.Get().SetMetadata(pxr::TfToken{TCHAR_TO_UTF8(Key)}, Value.GetUsdValue());
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdAttribute::ClearMetadata(const TCHAR* Key) const
	{
#if USE_USD_SDK
		return Impl->PxrUsdAttribute.Get().ClearMetadata(pxr::TfToken{TCHAR_TO_UTF8(Key)});
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	FName FUsdAttribute::GetName() const
	{
#if USE_USD_SDK
		return FName(UTF8_TO_TCHAR(Impl->PxrUsdAttribute.Get().GetName().GetString().c_str()));
#else
		return FName();
#endif	  // #if USE_USD_SDK
	}

	FName FUsdAttribute::GetBaseName() const
	{
#if USE_USD_SDK
		return FName(UTF8_TO_TCHAR(Impl->PxrUsdAttribute.Get().GetBaseName().GetString().c_str()));
#else
		return FName();
#endif	  // #if USE_USD_SDK
	}

	FName FUsdAttribute::GetTypeName() const
	{
#if USE_USD_SDK
		return FName(UTF8_TO_TCHAR(Impl->PxrUsdAttribute.Get().GetTypeName().GetAsToken().GetString().c_str()));
#else
		return FName();
#endif	  // #if USE_USD_SDK
	}

	FString FUsdAttribute::GetCPPTypeName() const
	{
#if USE_USD_SDK
		return UTF8_TO_TCHAR(Impl->PxrUsdAttribute.Get().GetTypeName().GetCPPTypeName().c_str());
#else
		return FString();
#endif	  // #if USE_USD_SDK
	}

	bool FUsdAttribute::GetTimeSamples(TArray<double>& Times) const
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		std::vector<double> UsdTimes;
		bool bResult = Impl->PxrUsdAttribute.Get().GetTimeSamples(&UsdTimes);
		if (!bResult)
		{
			return false;
		}

		Times.SetNumUninitialized(UsdTimes.size());
		FMemory::Memcpy(Times.GetData(), UsdTimes.data(), UsdTimes.size() * sizeof(double));

		return true;
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	size_t FUsdAttribute::GetNumTimeSamples() const
	{
#if USE_USD_SDK
		return Impl->PxrUsdAttribute.Get().GetNumTimeSamples();
#else
		return 0;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdAttribute::HasValue() const
	{
#if USE_USD_SDK
		return Impl->PxrUsdAttribute.Get().HasValue();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdAttribute::HasAuthoredValue() const
	{
#if USE_USD_SDK
		return Impl->PxrUsdAttribute.Get().HasAuthoredValue();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdAttribute::HasFallbackValue() const
	{
#if USE_USD_SDK
		return Impl->PxrUsdAttribute.Get().HasFallbackValue();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdAttribute::ValueMightBeTimeVarying() const
	{
#if USE_USD_SDK
		return Impl->PxrUsdAttribute.Get().ValueMightBeTimeVarying();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdAttribute::Get(UE::FVtValue& Value, TOptional<double> Time /*= {} */) const
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;

		pxr::UsdTimeCode TimeCode = Time.IsSet() ? Time.GetValue() : pxr::UsdTimeCode::Default();
		return Impl->PxrUsdAttribute.Get().Get(&Value.GetUsdValue(), TimeCode);
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	FUsdResolveInfo FUsdAttribute::GetResolveInfo(double Time) const
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;
		return FUsdResolveInfo{Impl->PxrUsdAttribute.Get().GetResolveInfo(Time)};
#else
		return {};
#endif	  // #if USE_USD_SDK
	}

	bool FUsdAttribute::Set(const UE::FVtValue& Value, TOptional<double> Time /*= {} */) const
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;

		pxr::UsdTimeCode TimeCode = Time.IsSet() ? Time.GetValue() : pxr::UsdTimeCode::Default();
		return Impl->PxrUsdAttribute.Get().Set(Value.GetUsdValue(), TimeCode);
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdAttribute::Clear() const
	{
#if USE_USD_SDK
		return Impl->PxrUsdAttribute.Get().Clear();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdAttribute::ClearAtTime(double Time) const
	{
#if USE_USD_SDK
		return Impl->PxrUsdAttribute.Get().ClearAtTime(pxr::UsdTimeCode(Time));
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdAttribute::ClearConnections() const
	{
#if USE_USD_SDK
		return Impl->PxrUsdAttribute.Get().ClearConnections();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdAttribute::GetUnionedTimeSamples(const TArray<UE::FUsdAttribute>& Attrs, TArray<double>& OutTimes)
	{
		bool bResult = false;

#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;

		std::vector<pxr::UsdAttribute> UsdAttrs;
		UsdAttrs.reserve(Attrs.Num());
		for (const UE::FUsdAttribute& Attr : Attrs)
		{
			UsdAttrs.push_back(Attr);
		}

		std::vector<double> UsdTimes;

		bResult = pxr::UsdAttribute::GetUnionedTimeSamples(UsdAttrs, &UsdTimes);
		if (bResult)
		{
			OutTimes.SetNumUninitialized(UsdTimes.size());
			FMemory::Memcpy(OutTimes.GetData(), UsdTimes.data(), OutTimes.Num() * OutTimes.GetTypeSize());
		}
#endif	  // #if USE_USD_SDK

		return bResult;
	}

	FSdfPath FUsdAttribute::GetPath() const
	{
#if USE_USD_SDK
		return FSdfPath(Impl->PxrUsdAttribute.Get().GetPath());
#else
		return FSdfPath();
#endif	  // #if USE_USD_SDK
	}

	FUsdPrim FUsdAttribute::GetPrim() const
	{
#if USE_USD_SDK
		return FUsdPrim(Impl->PxrUsdAttribute.Get().GetPrim());
#else
		return FUsdPrim();
#endif	  // #if USE_USD_SDK
	}

	template<typename T>
	bool FUsdAttribute::Get(T& Value, TOptional<double> Time) const
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;

		pxr::UsdTimeCode TimeCode = Time.IsSet() ? Time.GetValue() : pxr::UsdTimeCode::Default();
		pxr::UsdAttribute Attr = Impl->PxrUsdAttribute.Get();

		pxr::VtValue VtValue;
		if (Attr.Get(&VtValue, TimeCode))
		{
			UE::FVtValue UEValue{VtValue};
			Value = UEValue.Get<T>();
			return true;
		}
#endif	  // #if USE_USD_SDK

		return false;
	}
}	 // namespace UE

namespace UsdUtils
{
	template<typename ValueType>
	ValueType GetAttributeValue(const UE::FUsdPrim& Prim, FStringView AttributeName, TOptional<double> Time)
	{
		if (!Prim)
		{
			return ValueType{};
		}

		UE::FUsdAttribute Attribute = Prim.GetAttribute(AttributeName.GetData());

		ValueType Value{};
		if (Attribute)
		{
			Attribute.Get(Value, Time);
		}

		return Value;
	}

	template<typename ValueType>
	bool GetAuthoredAttributeValue(const UE::FUsdPrim& InPrim, FStringView InAttributeName, ValueType& OutValue, TOptional<double> InTime)
	{
		if (!InPrim)
		{
			return false;
		}

		UE::FUsdAttribute Attribute = InPrim.GetAttribute(InAttributeName.GetData());
		if (!Attribute)
		{
			return false;
		}

		const bool bGotValue = Attribute.Get(OutValue, InTime);
		return bGotValue && Attribute.HasAuthoredValue();
	}
}	 // namespace UsdUtils

#define SPECIALIZE(X) \
	template UNREALUSDWRAPPER_API bool UE::FUsdAttribute::Get(X& Value, TOptional<double> Time) const;\
	template UNREALUSDWRAPPER_API X UsdUtils::GetAttributeValue(const UE::FUsdPrim& Prim, FStringView AttributeName, TOptional<double> Time); \
	template UNREALUSDWRAPPER_API bool UsdUtils::GetAuthoredAttributeValue(const UE::FUsdPrim& InPrim, FStringView InAttributeName, X& OutValue, TOptional<double> InTime);

SPECIALIZE(bool);
SPECIALIZE(uint8);
SPECIALIZE(int32);
SPECIALIZE(uint32);
SPECIALIZE(int64);
SPECIALIZE(uint64);

SPECIALIZE(FFloat16);
SPECIALIZE(float);
SPECIALIZE(double);
SPECIALIZE(FSdfTimeCode);

SPECIALIZE(FString);
SPECIALIZE(FName);
SPECIALIZE(FSdfAssetPath);

SPECIALIZE(FMatrix2D);
SPECIALIZE(FMatrix3D);
SPECIALIZE(FMatrix44d);
SPECIALIZE(FMatrix44f);

SPECIALIZE(FQuat4d);
SPECIALIZE(FQuat4f);
SPECIALIZE(FQuat4h);

SPECIALIZE(FVector2d);
SPECIALIZE(FVector2f);
SPECIALIZE(FVector2DHalf);
SPECIALIZE(FIntPoint);

SPECIALIZE(FVector3d);
SPECIALIZE(FVector3f);
SPECIALIZE(FVector3h);
SPECIALIZE(FIntVector);

SPECIALIZE(FVector4d);
SPECIALIZE(FVector4f);
SPECIALIZE(FVector4h);
SPECIALIZE(FIntRect);
SPECIALIZE(FLinearColor);
SPECIALIZE(FColor);

SPECIALIZE(TArray<bool>);
SPECIALIZE(TArray<uint8>);
SPECIALIZE(TArray<int32>);
SPECIALIZE(TArray<uint32>);
SPECIALIZE(TArray<int64>);
SPECIALIZE(TArray<uint64>);

SPECIALIZE(TArray<FFloat16>);
SPECIALIZE(TArray<float>);
SPECIALIZE(TArray<double>);
SPECIALIZE(TArray<FSdfTimeCode>);

SPECIALIZE(TArray<FString>);
SPECIALIZE(TArray<FName>);
SPECIALIZE(TArray<FSdfAssetPath>);

SPECIALIZE(TArray<FMatrix2D>);
SPECIALIZE(TArray<FMatrix3D>);
SPECIALIZE(TArray<FMatrix44d>);
SPECIALIZE(TArray<FMatrix44f>);

SPECIALIZE(TArray<FQuat4d>);
SPECIALIZE(TArray<FQuat4f>);
SPECIALIZE(TArray<FQuat4h>);

SPECIALIZE(TArray<FVector2d>);
SPECIALIZE(TArray<FVector2f>);
SPECIALIZE(TArray<FVector2DHalf>);
SPECIALIZE(TArray<FIntPoint>);

SPECIALIZE(TArray<FVector3d>);
SPECIALIZE(TArray<FVector3f>);
SPECIALIZE(TArray<FVector3h>);
SPECIALIZE(TArray<FIntVector>);

SPECIALIZE(TArray<FVector4d>);
SPECIALIZE(TArray<FVector4f>);
SPECIALIZE(TArray<FVector4h>);
SPECIALIZE(TArray<FIntRect>);
SPECIALIZE(TArray<FLinearColor>);
SPECIALIZE(TArray<FColor>);

#undef SPECIALIZE