// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdWrappers/UsdEditContext.h"

#include "USDMemory.h"
#include "UsdWrappers/SdfLayer.h"
#include "UsdWrappers/UsdStage.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/usd/editContext.h"
#include "USDIncludesEnd.h"
#endif	  // #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
		class FUsdEditContextImpl
		{
#if USE_USD_SDK
		public:
			explicit FUsdEditContextImpl(const pxr::UsdStageWeakPtr& StageWeakPtr, const pxr::UsdEditTarget& EditTarget = pxr::UsdEditTarget())
				: UsdEditContext(StageWeakPtr, EditTarget)
			{
			}

		private:
			pxr::UsdEditContext UsdEditContext;
#endif	  // #if USE_USD_SDK
		};
	}

	FUsdEditContext::FUsdEditContext(const FUsdStageWeak& Stage)
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;
		Impl = MakeUnique<Internal::FUsdEditContextImpl>(Stage);
#endif	  // #if USE_USD_SDK
	}

	FUsdEditContext::FUsdEditContext(const FUsdStageWeak& Stage, const FSdfLayer& EditTarget)
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;
		Impl = MakeUnique<Internal::FUsdEditContextImpl>(Stage, pxr::SdfLayerWeakPtr{EditTarget});
#endif	  // #if USE_USD_SDK
	}

	FUsdEditContext::~FUsdEditContext()
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;
		Impl.Reset();
#endif	  // #if USE_USD_SDK
	}
}	 // namespace UE
