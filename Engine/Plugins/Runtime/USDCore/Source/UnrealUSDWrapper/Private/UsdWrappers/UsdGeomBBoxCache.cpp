// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdWrappers/UsdGeomBBoxCache.h"

#include "UnrealUSDWrapper.h"
#include "USDMemory.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/usdGeom/bboxCache.h"
#include "USDIncludesEnd.h"
#endif	  //  #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
		class FUsdGeomBBoxCacheImpl
		{
		public:
			FUsdGeomBBoxCacheImpl(double Time, EUsdPurpose IncludedPurposeFlags, bool bUseExtentsHint, bool bIgnoreVisibility)
#if USE_USD_SDK
				: PxrUsdGeomBBoxCache(Time, {}, bUseExtentsHint, bIgnoreVisibility)
#endif	  // #if USE_USD_SDK
			{
#if USE_USD_SDK
				std::vector<pxr::TfToken> TokenVector;
				TokenVector.reserve(4);
				TokenVector.push_back(pxr::UsdGeomTokens->default_);

				if (EnumHasAllFlags(IncludedPurposeFlags, EUsdPurpose::Proxy))
				{
					TokenVector.push_back(pxr::UsdGeomTokens->proxy);
				}

				if (EnumHasAllFlags(IncludedPurposeFlags, EUsdPurpose::Render))
				{
					TokenVector.push_back(pxr::UsdGeomTokens->render);
				}

				if (EnumHasAllFlags(IncludedPurposeFlags, EUsdPurpose::Guide))
				{
					TokenVector.push_back(pxr::UsdGeomTokens->guide);
				}

				PxrUsdGeomBBoxCache.SetIncludedPurposes(TokenVector);
#endif	  // #if USE_USD_SDK
			}

#if USE_USD_SDK
			explicit FUsdGeomBBoxCacheImpl(const pxr::UsdGeomBBoxCache& InUsdGeomBBoxCache)
				: PxrUsdGeomBBoxCache(InUsdGeomBBoxCache)
			{
			}

			explicit FUsdGeomBBoxCacheImpl(pxr::UsdGeomBBoxCache&& InUsdGeomBBoxCache)
				: PxrUsdGeomBBoxCache(MoveTemp(InUsdGeomBBoxCache))
			{
			}

			pxr::UsdGeomBBoxCache PxrUsdGeomBBoxCache;
#endif	  // #if USE_USD_SDK
		};
	}	  // namespace Internal

	FUsdGeomBBoxCache::FUsdGeomBBoxCache(double Time, EUsdPurpose IncludedPurposeFlags, bool bUseExtentsHint, bool bIgnoreVisibility)
	{
		FScopedUsdAllocs UsdAllocs;
		FWriteScopeLock ScopeLock{Lock};
		Impl = MakeUnique<Internal::FUsdGeomBBoxCacheImpl>(Time, IncludedPurposeFlags, bUseExtentsHint, bIgnoreVisibility);
	}

	FUsdGeomBBoxCache::FUsdGeomBBoxCache(const FUsdGeomBBoxCache& Other)
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;
		FWriteScopeLock ScopeLock{Lock};
		Impl = MakeUnique<Internal::FUsdGeomBBoxCacheImpl>(Other.Impl->PxrUsdGeomBBoxCache);
#endif	  // #if USE_USD_SDK
	}

	FUsdGeomBBoxCache::FUsdGeomBBoxCache(FUsdGeomBBoxCache&& Other)
		: Impl(MakeUnique<Internal::FUsdGeomBBoxCacheImpl>(
			/*Time*/ 0.0,
			/*IncludedPurposeFlags*/ EUsdPurpose::Default,
			/*bUseExtentsHint*/ false,
			/*bIgnoreVisibility*/ false
		))
	{
		if (this != &Other)
		{
			FWriteScopeLock ScopeLock{Lock};
			FWriteScopeLock OtherScopeLock{Other.Lock};
			Impl = MoveTemp(Other.Impl);
		}
	}

	FUsdGeomBBoxCache& FUsdGeomBBoxCache::operator=(const FUsdGeomBBoxCache& Other)
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;
		FWriteScopeLock ScopeLock{Lock};
		Impl = MakeUnique<Internal::FUsdGeomBBoxCacheImpl>(Other.Impl->PxrUsdGeomBBoxCache);
#endif	  // #if USE_USD_SDK

		return *this;
	}

	FUsdGeomBBoxCache& FUsdGeomBBoxCache::operator=(FUsdGeomBBoxCache&& Other)
	{
		if (this != &Other)
		{
			FScopedUsdAllocs UsdAllocs;
			FWriteScopeLock ScopeLock{Lock};
			FWriteScopeLock OtherScopeLock{Other.Lock};
			Impl = MoveTemp(Other.Impl);
		}

		return *this;
	}

	FUsdGeomBBoxCache::~FUsdGeomBBoxCache()
	{
		FScopedUsdAllocs UsdAllocs;
		FWriteScopeLock ScopeLock{Lock};
		Impl.Reset();
	}

#if USE_USD_SDK
	FUsdGeomBBoxCache::FUsdGeomBBoxCache(const pxr::UsdGeomBBoxCache& InUsdGeomBBoxCache)
	{
		FScopedUsdAllocs UsdAllocs;
		FWriteScopeLock ScopeLock{Lock};
		Impl = MakeUnique<Internal::FUsdGeomBBoxCacheImpl>(InUsdGeomBBoxCache);
	}

	FUsdGeomBBoxCache::FUsdGeomBBoxCache(pxr::UsdGeomBBoxCache&& InUsdGeomBBoxCache)
	{
		FScopedUsdAllocs UsdAllocs;
		FWriteScopeLock ScopeLock{Lock};
		Impl = MakeUnique<Internal::FUsdGeomBBoxCacheImpl>(MoveTemp(InUsdGeomBBoxCache));
	}

	FUsdGeomBBoxCache::operator pxr::UsdGeomBBoxCache&()
	{
		// Our caller should have manually locked the mutex before calling this.
		// Note we need to check for a read lock here: It should be considered a mistaked to only have a read lock acquired
		// and call this function, as it provides mutable access to the underlying type.
		// If we checked for a write lock here, our lock would have failed to acquire (given the outer read lock from the user)
		// and we would have incorrectly assumed the user locked correctly. Checking for a read lock should succeed even in that
		// case, which will let us detect the mistake and cause our ensure to correctly trigger
		ensure(!Lock.TryReadLock());
		return Impl->PxrUsdGeomBBoxCache;
	}

	FUsdGeomBBoxCache::operator const pxr::UsdGeomBBoxCache&() const
	{
		// Our caller should have manually locked the mutex before calling this
		ensure(!Lock.TryReadLock());
		return Impl->PxrUsdGeomBBoxCache;
	}
#endif	  // #if USE_USD_SDK

	void FUsdGeomBBoxCache::Clear()
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;
		FWriteScopeLock ScopeLock{Lock};
		Impl->PxrUsdGeomBBoxCache.Clear();
#endif	  // #if USE_USD_SDK
	}

	void FUsdGeomBBoxCache::SetIncludedPurposes(EUsdPurpose IncludedPurposeFlags)
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;
		FWriteScopeLock ScopeLock{Lock};

		std::vector<pxr::TfToken> TokenVector;
		TokenVector.reserve(4);
		TokenVector.push_back(pxr::UsdGeomTokens->default_);

		if (EnumHasAllFlags(IncludedPurposeFlags, EUsdPurpose::Proxy))
		{
			TokenVector.push_back(pxr::UsdGeomTokens->proxy);
		}

		if (EnumHasAllFlags(IncludedPurposeFlags, EUsdPurpose::Render))
		{
			TokenVector.push_back(pxr::UsdGeomTokens->render);
		}

		if (EnumHasAllFlags(IncludedPurposeFlags, EUsdPurpose::Guide))
		{
			TokenVector.push_back(pxr::UsdGeomTokens->guide);
		}

		Impl->PxrUsdGeomBBoxCache.SetIncludedPurposes(TokenVector);
#endif	  // #if USE_USD_SDK
	}

	EUsdPurpose FUsdGeomBBoxCache::GetIncludedPurposes() const
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;
		FReadScopeLock ScopeLock{Lock};

		EUsdPurpose Flags = EUsdPurpose::Default;
		for (const pxr::TfToken& Token : Impl->PxrUsdGeomBBoxCache.GetIncludedPurposes())
		{
			if (Token == pxr::UsdGeomTokens->proxy)
			{
				Flags |= EUsdPurpose::Proxy;
			}
			else if (Token == pxr::UsdGeomTokens->render)
			{
				Flags |= EUsdPurpose::Render;
			}
			else if (Token == pxr::UsdGeomTokens->guide)
			{
				Flags |= EUsdPurpose::Guide;
			}
		}
		return Flags;
#else
		return EUsdPurpose::Default;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdGeomBBoxCache::GetUseExtentsHint() const
	{
#if USE_USD_SDK
		FReadScopeLock ScopeLock{Lock};
		return Impl->PxrUsdGeomBBoxCache.GetUseExtentsHint();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdGeomBBoxCache::GetIgnoreVisibility() const
	{
#if USE_USD_SDK
		FReadScopeLock ScopeLock{Lock};
		return Impl->PxrUsdGeomBBoxCache.GetIgnoreVisibility();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	void FUsdGeomBBoxCache::SetTime(double UsdTimeCode)
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;
		FWriteScopeLock ScopeLock{Lock};
		Impl->PxrUsdGeomBBoxCache.SetTime(UsdTimeCode);
#endif	  // #if USE_USD_SDK
	}

	double FUsdGeomBBoxCache::GetTime() const
	{
#if USE_USD_SDK
		FReadScopeLock ScopeLock{Lock};
		return Impl->PxrUsdGeomBBoxCache.GetTime().GetValue();
#else
		return 0;
#endif	  // #if USE_USD_SDK
	}

}	 // namespace UE
