// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdWrappers/UsdGeomSubset.h"

#include "USDMemory.h"

#include "UsdWrappers/UsdPrim.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/usd/attribute.h"
#include "pxr/usd/usdGeom/subset.h"
#include "USDIncludesEnd.h"
#endif	  // #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
		class FUsdGeomSubsetImpl
		{
		public:
			FUsdGeomSubsetImpl() = default;

#if USE_USD_SDK
			explicit FUsdGeomSubsetImpl(const pxr::UsdPrim& InUsdPrim)
				: PxrUsdGeomSubset(MakeUsdStore<pxr::UsdGeomSubset>(InUsdPrim))
			{
			}

			explicit FUsdGeomSubsetImpl(const pxr::UsdGeomSubset& InUsdGeomSubset)
				: PxrUsdGeomSubset(InUsdGeomSubset)
			{
			}

			explicit FUsdGeomSubsetImpl(pxr::UsdGeomSubset&& InUsdGeomSubset)
				: PxrUsdGeomSubset(MoveTemp(InUsdGeomSubset))
			{
			}

			TUsdStore<pxr::UsdGeomSubset> PxrUsdGeomSubset;
#endif	  // #if USE_USD_SDK
		};
	}	 // namespace Internal

	FUsdGeomSubset::FUsdGeomSubset()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdGeomSubsetImpl>();
	}

	FUsdGeomSubset::FUsdGeomSubset(const FUsdPrim& UsdPrim)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdGeomSubsetImpl>(UsdPrim);
#endif	  // #if USE_USD_SDK
	}

	FUsdGeomSubset::FUsdGeomSubset(const FUsdGeomSubset& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdGeomSubsetImpl>(Other.Impl->PxrUsdGeomSubset.Get());
#endif	  // #if USE_USD_SDK
	}

	FUsdGeomSubset::FUsdGeomSubset(FUsdGeomSubset&& Other) = default;

	FUsdGeomSubset::~FUsdGeomSubset()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	FUsdGeomSubset& FUsdGeomSubset::operator=(const FUsdGeomSubset& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdGeomSubsetImpl>(Other.Impl->PxrUsdGeomSubset.Get());
#endif	  // #if USE_USD_SDK
		return *this;
	}

	FUsdGeomSubset& FUsdGeomSubset::operator=(FUsdGeomSubset&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MoveTemp(Other.Impl);

		return *this;
	}

	FUsdGeomSubset::operator bool() const
	{
#if USE_USD_SDK
		return (bool)Impl->PxrUsdGeomSubset.Get();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

#if USE_USD_SDK
	FUsdGeomSubset::FUsdGeomSubset(const pxr::UsdGeomSubset& InUsdGeomSubset)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdGeomSubsetImpl>(InUsdGeomSubset);
	}

	FUsdGeomSubset::FUsdGeomSubset(pxr::UsdGeomSubset&& InUsdGeomSubset)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdGeomSubsetImpl>(MoveTemp(InUsdGeomSubset));
	}

	FUsdGeomSubset& FUsdGeomSubset::operator=(const pxr::UsdGeomSubset& InUsdGeomSubset)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdGeomSubsetImpl>(InUsdGeomSubset);
		return *this;
	}

	FUsdGeomSubset& FUsdGeomSubset::operator=(pxr::UsdGeomSubset&& InUsdGeomSubset)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdGeomSubsetImpl>(MoveTemp(InUsdGeomSubset));
		return *this;
	}

	FUsdGeomSubset::operator pxr::UsdGeomSubset&()
	{
		return Impl->PxrUsdGeomSubset.Get();
	}

	FUsdGeomSubset::operator const pxr::UsdGeomSubset&() const
	{
		return Impl->PxrUsdGeomSubset.Get();
	}
#endif	  // #if USE_USD_SDK

	FUsdPrim FUsdGeomSubset::GetPrim() const
	{
#if USE_USD_SDK
		return FUsdPrim(Impl->PxrUsdGeomSubset.Get().GetPrim());
#else
		return FUsdPrim();
#endif	  // #if USE_USD_SDK
	}

	FUsdAttribute FUsdGeomSubset::GetElementTypeAttr() const
	{
#if USE_USD_SDK
		return FUsdAttribute(Impl->PxrUsdGeomSubset.Get().GetElementTypeAttr());
#else
		return FUsdAttribute();
#endif	  // #if USE_USD_SDK
	}

	FUsdAttribute FUsdGeomSubset::GetFamilyNameAttr() const
	{
#if USE_USD_SDK
		return FUsdAttribute(Impl->PxrUsdGeomSubset.Get().GetFamilyNameAttr());
#else
		return FUsdAttribute();
#endif	  // #if USE_USD_SDK
	}

	FUsdAttribute FUsdGeomSubset::GetIndicesAttr() const
	{
#if USE_USD_SDK
		return FUsdAttribute(Impl->PxrUsdGeomSubset.Get().GetIndicesAttr());
#else
		return FUsdAttribute();
#endif	  // #if USE_USD_SDK
	}

}	 // namespace UE
