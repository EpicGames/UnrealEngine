// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdWrappers/UsdGeomXformable.h"

#include "USDMemory.h"

#include "UsdWrappers/UsdAttribute.h"
#include "UsdWrappers/UsdPrim.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/usdGeom/xformable.h"
#include "USDIncludesEnd.h"
#endif	  //  #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
		class FUsdGeomXformableImpl
		{
		public:
			FUsdGeomXformableImpl() = default;

#if USE_USD_SDK
			explicit FUsdGeomXformableImpl(const pxr::UsdGeomXformable& InUsdGeomXformable)
				: PxrUsdGeomXformable(InUsdGeomXformable)
			{
			}

			explicit FUsdGeomXformableImpl(pxr::UsdGeomXformable&& InUsdGeomXformable)
				: PxrUsdGeomXformable(MoveTemp(InUsdGeomXformable))
			{
			}

			explicit FUsdGeomXformableImpl(const pxr::UsdPrim& InUsdPrim)
				: PxrUsdGeomXformable(pxr::UsdGeomXformable(InUsdPrim))
			{
			}

			TUsdStore<pxr::UsdGeomXformable> PxrUsdGeomXformable;
#endif	  // #if USE_USD_SDK
		};
	}

	FUsdGeomXformable::FUsdGeomXformable()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdGeomXformableImpl>();
	}

	FUsdGeomXformable::FUsdGeomXformable(const FUsdGeomXformable& Other)
		: FUsdTyped(Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdGeomXformableImpl>(Other.Impl->PxrUsdGeomXformable.Get());
#endif	  // #if USE_USD_SDK
	}

	FUsdGeomXformable::FUsdGeomXformable(FUsdGeomXformable&& Other)
		: FUsdTyped(MoveTemp(Other))
		, Impl(MoveTemp(Other.Impl))
	{
	}

	FUsdGeomXformable::FUsdGeomXformable(const FUsdPrim& Prim)
		: FUsdTyped(Prim)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdGeomXformableImpl>(Prim);
#endif	  // #if USE_USD_SDK
	}

	FUsdGeomXformable& FUsdGeomXformable::operator=(const FUsdGeomXformable& Other)
	{
		FUsdTyped::operator=(Other);

#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdGeomXformableImpl>(Other.Impl->PxrUsdGeomXformable.Get());
#endif	  // #if USE_USD_SDK

		return *this;
	}

	FUsdGeomXformable& FUsdGeomXformable::operator=(FUsdGeomXformable&& Other)
	{
		FUsdTyped::operator=(MoveTemp(Other));

		FScopedUnrealAllocs UnrealAllocs;
		Impl = MoveTemp(Other.Impl);

		return *this;
	}

	FUsdGeomXformable::~FUsdGeomXformable()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	FUsdGeomXformable::operator bool() const
	{
#if USE_USD_SDK
		return (bool)Impl->PxrUsdGeomXformable.Get();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

#if USE_USD_SDK
	FUsdGeomXformable::FUsdGeomXformable(const pxr::UsdGeomXformable& InUsdGeomXformable)
		: FUsdTyped(InUsdGeomXformable)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdGeomXformableImpl>(InUsdGeomXformable);
	}

	FUsdGeomXformable::FUsdGeomXformable(pxr::UsdGeomXformable&& InUsdGeomXformable)
		: FUsdTyped(MoveTemp(InUsdGeomXformable))
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdGeomXformableImpl>(MoveTemp(InUsdGeomXformable));
	}

	FUsdGeomXformable::FUsdGeomXformable(const pxr::UsdPrim& Prim)
		: FUsdTyped(Prim)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdGeomXformableImpl>(Prim);
	}

	FUsdGeomXformable::operator pxr::UsdGeomXformable&()
	{
		return Impl->PxrUsdGeomXformable.Get();
	}

	FUsdGeomXformable::operator const pxr::UsdGeomXformable&() const
	{
		return Impl->PxrUsdGeomXformable.Get();
	}
#endif	  // #if USE_USD_SDK

	bool FUsdGeomXformable::GetResetXformStack() const
	{
#if USE_USD_SDK
		return Impl->PxrUsdGeomXformable.Get().GetResetXformStack();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdGeomXformable::TransformMightBeTimeVarying() const
	{
#if USE_USD_SDK
		return Impl->PxrUsdGeomXformable.Get().TransformMightBeTimeVarying();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdGeomXformable::GetTimeSamples(TArray<double>* Times) const
	{
		if (!Times)
		{
			return false;
		}

#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;
		std::vector<double> UsdTimes;

		if (Impl->PxrUsdGeomXformable.Get().GetTimeSamples(&UsdTimes))
		{
			Times->Empty(UsdTimes.size());

			for (double UsdTime : UsdTimes)
			{
				Times->Add(UsdTime);
			}

			return true;
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	FUsdAttribute FUsdGeomXformable::GetXformOpOrderAttr() const
	{
#if USE_USD_SDK
		return FUsdAttribute(Impl->PxrUsdGeomXformable.Get().GetXformOpOrderAttr());
#else
		return FUsdAttribute();
#endif	  // #if USE_USD_SDK
	}

	bool FUsdGeomXformable::ClearXformOpOrder() const
	{
#if USE_USD_SDK
		return Impl->PxrUsdGeomXformable.Get().ClearXformOpOrder();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}
}	 // namespace UE
