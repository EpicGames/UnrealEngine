// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdWrappers/UsdPayloads.h"

#include "USDMemory.h"

#include "UsdWrappers/UsdPrim.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/usd/payloads.h"
#include "pxr/usd/usd/prim.h"
#include "USDIncludesEnd.h"
#endif	  // #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
		class FUsdPayloadsImpl
		{
		public:
			FUsdPayloadsImpl() = default;

#if USE_USD_SDK
			explicit FUsdPayloadsImpl(const pxr::UsdPayloads& InUsdPayloads)
				: PxrUsdPrim(InUsdPayloads.GetPrim())
			{
			}

			explicit FUsdPayloadsImpl(pxr::UsdPayloads&& InUsdPayloads)
				: PxrUsdPrim(InUsdPayloads.GetPrim())
			{
			}

			explicit FUsdPayloadsImpl(const pxr::UsdPrim& InUsdPrim)
				: PxrUsdPrim(InUsdPrim)
			{
			}

			pxr::UsdPayloads GetInner()
			{
				FScopedUsdAllocs UsdAllocs;
				return PxrUsdPrim.Get().GetPayloads();
			}

			// pxr::UsdPayloads has no default constructor, so we store the prim itself.
			TUsdStore<pxr::UsdPrim> PxrUsdPrim;
#endif	  // #if USE_USD_SDK
		};
	}	  // namespace Internal

	FUsdPayloads::FUsdPayloads()
	{
		Impl = MakeUnique<Internal::FUsdPayloadsImpl>();
	}

	FUsdPayloads::FUsdPayloads(const FUsdPayloads& Other)
	{
#if USE_USD_SDK
		Impl = MakeUnique<Internal::FUsdPayloadsImpl>(Other.Impl->PxrUsdPrim.Get());
#endif	  // #if USE_USD_SDK
	}

	FUsdPayloads::FUsdPayloads(FUsdPayloads&& Other) = default;

	FUsdPayloads& FUsdPayloads::operator=(const FUsdPayloads& Other)
	{
#if USE_USD_SDK
		Impl = MakeUnique<Internal::FUsdPayloadsImpl>(Other.Impl->PxrUsdPrim.Get());
#endif	  // #if USE_USD_SDK

		return *this;
	}

	FUsdPayloads& FUsdPayloads::operator=(FUsdPayloads&& Other)
	{
		Impl = MoveTemp(Other.Impl);

		return *this;
	}

	FUsdPayloads::~FUsdPayloads()
	{
		Impl.Reset();
	}

	FUsdPayloads::operator bool() const
	{
#if USE_USD_SDK
		return (bool)Impl->PxrUsdPrim.Get();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

#if USE_USD_SDK
	FUsdPayloads::FUsdPayloads(const pxr::UsdPayloads& InUsdPayloads)
	{
		Impl = MakeUnique<Internal::FUsdPayloadsImpl>(InUsdPayloads);
	}

	FUsdPayloads::FUsdPayloads(pxr::UsdPayloads&& InUsdPayloads)
	{
		Impl = MakeUnique<Internal::FUsdPayloadsImpl>(MoveTemp(InUsdPayloads));
	}

	FUsdPayloads& FUsdPayloads::operator=(const pxr::UsdPayloads& InUsdPayloads)
	{
		Impl = MakeUnique<Internal::FUsdPayloadsImpl>(InUsdPayloads);
		return *this;
	}

	FUsdPayloads& FUsdPayloads::operator=(pxr::UsdPayloads&& InUsdPayloads)
	{
		Impl = MakeUnique<Internal::FUsdPayloadsImpl>(MoveTemp(InUsdPayloads));
		return *this;
	}

	FUsdPayloads::operator pxr::UsdPayloads()
	{
		return Impl->GetInner();
	}

	FUsdPayloads::operator const pxr::UsdPayloads() const
	{
		return Impl->GetInner();
	}

	// Private constructor accessible by FUsdPrim.
	FUsdPayloads::FUsdPayloads(const pxr::UsdPrim& InPrim)
	{
		Impl = MakeUnique<Internal::FUsdPayloadsImpl>(InPrim);
	}
#endif	  // #if USE_USD_SDK

	bool FUsdPayloads::AddPayload(const FSdfPayload& Payload, EUsdListPosition Position)
	{
#if USE_USD_SDK
		if (Impl->PxrUsdPrim.Get())
		{
			FScopedUsdAllocs UsdAllocs;

			const std::string AssetPath(TCHAR_TO_UTF8(*Payload.AssetPath));
			const pxr::SdfPath UsdPrimPath(Payload.PrimPath);
			const pxr::SdfLayerOffset UsdLayerOffset(Payload.LayerOffset.Offset, Payload.LayerOffset.Scale);

			const pxr::SdfPayload UsdPayload(AssetPath, UsdPrimPath, UsdLayerOffset);

			return Impl->PxrUsdPrim.Get().GetPayloads().AddPayload(UsdPayload, static_cast<pxr::UsdListPosition>(Position));
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	bool FUsdPayloads::AddPayload(const FString& Identifier, const FSdfPath& PrimPath, const FSdfLayerOffset& LayerOffset, EUsdListPosition Position)
	{
#if USE_USD_SDK
		if (Impl->PxrUsdPrim.Get())
		{
			FScopedUsdAllocs UsdAllocs;

			const std::string UsdIdentifier(TCHAR_TO_UTF8(*Identifier));
			const pxr::SdfPath UsdPrimPath(PrimPath);
			const pxr::SdfLayerOffset UsdLayerOffset(LayerOffset.Offset, LayerOffset.Scale);

			return Impl->PxrUsdPrim.Get().GetPayloads().AddPayload(
				UsdIdentifier,
				UsdPrimPath,
				UsdLayerOffset,
				static_cast<pxr::UsdListPosition>(Position)
			);
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	bool FUsdPayloads::AddPayload(const FString& Identifier, const FSdfLayerOffset& LayerOffset, EUsdListPosition Position)
	{
#if USE_USD_SDK
		if (Impl->PxrUsdPrim.Get())
		{
			FScopedUsdAllocs UsdAllocs;

			const std::string UsdIdentifier(TCHAR_TO_UTF8(*Identifier));
			const pxr::SdfLayerOffset UsdLayerOffset(LayerOffset.Offset, LayerOffset.Scale);

			return Impl->PxrUsdPrim.Get().GetPayloads().AddPayload(UsdIdentifier, UsdLayerOffset, static_cast<pxr::UsdListPosition>(Position));
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	bool FUsdPayloads::AddInternalPayload(const FSdfPath& PrimPath, const FSdfLayerOffset& LayerOffset, EUsdListPosition Position)
	{
#if USE_USD_SDK
		if (Impl->PxrUsdPrim.Get())
		{
			FScopedUsdAllocs UsdAllocs;

			const pxr::SdfPath UsdPrimPath(PrimPath);
			const pxr::SdfLayerOffset UsdLayerOffset(LayerOffset.Offset, LayerOffset.Scale);

			return Impl->PxrUsdPrim.Get().GetPayloads().AddInternalPayload(UsdPrimPath, UsdLayerOffset, static_cast<pxr::UsdListPosition>(Position));
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	bool FUsdPayloads::RemovePayload(const FSdfPayload& Payload)
	{
#if USE_USD_SDK
		if (Impl->PxrUsdPrim.Get())
		{
			FScopedUsdAllocs UsdAllocs;

			const std::string AssetPath(TCHAR_TO_UTF8(*Payload.AssetPath));
			const pxr::SdfPath UsdPrimPath(Payload.PrimPath);
			const pxr::SdfLayerOffset UsdLayerOffset(Payload.LayerOffset.Offset, Payload.LayerOffset.Scale);

			const pxr::SdfPayload UsdPayload(AssetPath, UsdPrimPath, UsdLayerOffset);

			return Impl->PxrUsdPrim.Get().GetPayloads().RemovePayload(UsdPayload);
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	bool FUsdPayloads::ClearPayloads()
	{
#if USE_USD_SDK
		if (Impl->PxrUsdPrim.Get())
		{
			FScopedUsdAllocs UsdAllocs;

			return Impl->PxrUsdPrim.Get().GetPayloads().ClearPayloads();
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	bool FUsdPayloads::SetPayloads(const TArray<FSdfPayload>& Items)
	{
#if USE_USD_SDK
		if (Impl->PxrUsdPrim.Get())
		{
			FScopedUsdAllocs UsdAllocs;

			std::vector<pxr::SdfPayload> UsdPayloads;
			UsdPayloads.reserve(Items.Num());

			for (const FSdfPayload& Item : Items)
			{
				const std::string AssetPath(TCHAR_TO_UTF8(*Item.AssetPath));
				const pxr::SdfPath UsdPrimPath(Item.PrimPath);
				const pxr::SdfLayerOffset UsdLayerOffset(Item.LayerOffset.Offset, Item.LayerOffset.Scale);

				UsdPayloads.emplace_back(AssetPath, UsdPrimPath, UsdLayerOffset);
			}

			return Impl->PxrUsdPrim.Get().GetPayloads().SetPayloads(UsdPayloads);
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	FUsdPrim FUsdPayloads::GetPrim() const
	{
#if USE_USD_SDK
		if (Impl->PxrUsdPrim.Get())
		{
			return FUsdPrim(Impl->PxrUsdPrim.Get());
		}
#endif	  // #if USE_USD_SDK

		return FUsdPrim{};
	}
}	 // namespace UE
