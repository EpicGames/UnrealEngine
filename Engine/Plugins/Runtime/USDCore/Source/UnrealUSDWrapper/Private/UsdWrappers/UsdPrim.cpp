// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdWrappers/UsdPrim.h"

#include "USDMemory.h"

#include "UsdWrappers/PcpPrimIndex.h"
#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/SdfPrimSpec.h"
#include "UsdWrappers/UsdAttribute.h"
#include "UsdWrappers/UsdPayloads.h"
#include "UsdWrappers/UsdReferences.h"
#include "UsdWrappers/UsdRelationship.h"
#include "UsdWrappers/UsdStage.h"
#include "UsdWrappers/UsdVariantSets.h"
#include "UsdWrappers/VtValue.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/sdf/schema.h"
#include "pxr/usd/usd/attribute.h"
#include "pxr/usd/usd/prim.h"
#include "pxr/usd/usd/relationship.h"
#include "USDIncludesEnd.h"
#endif	  // #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
		class FUsdPrimImpl
		{
		public:
			FUsdPrimImpl() = default;

#if USE_USD_SDK

#if defined(ENABLE_USD_DEBUG_PATH) && ENABLE_USD_DEBUG_PATH
			FString DebugPath;
#endif
			explicit FUsdPrimImpl(const pxr::UsdPrim& InUsdPrim)
				: PxrUsdPrim(InUsdPrim)
			{
#if defined(ENABLE_USD_DEBUG_PATH) && ENABLE_USD_DEBUG_PATH
				DebugPath = FString{UTF8_TO_TCHAR(PxrUsdPrim.Get().GetPrimPath().GetString().c_str())};
#endif
			}

			explicit FUsdPrimImpl(pxr::UsdPrim&& InUsdPrim)
				: PxrUsdPrim(MoveTemp(InUsdPrim))
			{
#if defined(ENABLE_USD_DEBUG_PATH) && ENABLE_USD_DEBUG_PATH
				DebugPath = FString{UTF8_TO_TCHAR(PxrUsdPrim.Get().GetPrimPath().GetString().c_str())};
#endif
			}

			TUsdStore<pxr::UsdPrim> PxrUsdPrim;
#endif	  // #if USE_USD_SDK
		};
	}	 // namespace Internal

	FUsdPrim::FUsdPrim()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdPrimImpl>();
	}

	FUsdPrim::FUsdPrim(const FUsdPrim& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdPrimImpl>(Other.Impl->PxrUsdPrim.Get());
#endif	  // #if USE_USD_SDK
	}

	FUsdPrim::FUsdPrim(FUsdPrim&& Other) = default;

	FUsdPrim::~FUsdPrim()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	FUsdPrim& FUsdPrim::operator=(const FUsdPrim& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdPrimImpl>(Other.Impl->PxrUsdPrim.Get());
#endif	  // #if USE_USD_SDK
		return *this;
	}

	FUsdPrim& FUsdPrim::operator=(FUsdPrim&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MoveTemp(Other.Impl);

		return *this;
	}

	FUsdPrim::operator bool() const
	{
#if USE_USD_SDK
		return (bool)Impl->PxrUsdPrim.Get();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdPrim::operator==(const FUsdPrim& Other) const
	{
#if USE_USD_SDK
		return Impl->PxrUsdPrim.Get() == Other.Impl->PxrUsdPrim.Get();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdPrim::operator!=(const FUsdPrim& Other) const
	{
		return !(*this == Other);
	}

#if USE_USD_SDK
	FUsdPrim::FUsdPrim(const pxr::UsdPrim& InUsdPrim)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdPrimImpl>(InUsdPrim);
	}

	FUsdPrim::FUsdPrim(pxr::UsdPrim&& InUsdPrim)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdPrimImpl>(MoveTemp(InUsdPrim));
	}

	FUsdPrim& FUsdPrim::operator=(const pxr::UsdPrim& InUsdPrim)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdPrimImpl>(InUsdPrim);
		return *this;
	}

	FUsdPrim& FUsdPrim::operator=(pxr::UsdPrim&& InUsdPrim)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdPrimImpl>(MoveTemp(InUsdPrim));
		return *this;
	}

	FUsdPrim::operator pxr::UsdPrim&()
	{
		return Impl->PxrUsdPrim.Get();
	}

	FUsdPrim::operator const pxr::UsdPrim&() const
	{
		return Impl->PxrUsdPrim.Get();
	}
#endif	  // #if USE_USD_SDK

	bool FUsdPrim::GetMetadata(const TCHAR* Key, UE::FVtValue& Value) const
	{
#if USE_USD_SDK
		return Impl->PxrUsdPrim.Get().GetMetadata(pxr::TfToken{TCHAR_TO_UTF8(Key)}, &Value.GetUsdValue());
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdPrim::HasMetadata(const TCHAR* Key) const
	{
#if USE_USD_SDK
		return Impl->PxrUsdPrim.Get().HasMetadata(pxr::TfToken{TCHAR_TO_UTF8(Key)});
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdPrim::SetMetadata(const TCHAR* Key, const UE::FVtValue& Value) const
	{
#if USE_USD_SDK
		return Impl->PxrUsdPrim.Get().SetMetadata(pxr::TfToken{TCHAR_TO_UTF8(Key)}, Value.GetUsdValue());
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdPrim::ClearMetadata(const TCHAR* Key) const
	{
#if USE_USD_SDK
		return Impl->PxrUsdPrim.Get().ClearMetadata(pxr::TfToken{TCHAR_TO_UTF8(Key)});
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	TMap<FString, UE::FVtValue> FUsdPrim::GetAllMetadata() const
	{
		// Note that in USD the key types are pxr::TfToken, and so traditionally we should be using FNames instead of FStrings here.
		// That could cause trouble however, due to how pxr::TfToken is case sensitive and FNames are "case preserving" (sometimes).
		// Also, it's a bit annoying how pxr::VtDictionary (in case of nested metadata) does have std::string as keys... So for overall
		// convenience we use FString here so that we can use the same types for both this result and nested metadata maps, and can
		// avoid issues due to casing differences
		TMap<FString, UE::FVtValue> Result;
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		std::map<pxr::TfToken, pxr::VtValue, pxr::TfDictionaryLessThan> UsdMetadata = Impl->PxrUsdPrim.Get().GetAllMetadata();
		Result.Reserve(UsdMetadata.size());

		std::map<pxr::TfToken, pxr::VtValue, pxr::TfDictionaryLessThan>::iterator Iter = UsdMetadata.begin();
		for (; Iter != UsdMetadata.end(); ++Iter)
		{
			Result.Add(UTF8_TO_TCHAR(Iter->first.GetString().c_str()), UE::FVtValue{std::move(Iter->second)});
		}
#endif	  // #if USE_USD_SDK
		return Result;
	}

	TMap<FString, UE::FVtValue> FUsdPrim::GetAllAuthoredMetadata() const
	{
		TMap<FString, UE::FVtValue> Result;
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		std::map<pxr::TfToken, pxr::VtValue, pxr::TfDictionaryLessThan> UsdMetadata = Impl->PxrUsdPrim.Get().GetAllAuthoredMetadata();
		Result.Reserve(UsdMetadata.size());

		std::map<pxr::TfToken, pxr::VtValue, pxr::TfDictionaryLessThan>::iterator Iter = UsdMetadata.begin();
		for (; Iter != UsdMetadata.end(); ++Iter)
		{
			Result.Add(UTF8_TO_TCHAR(Iter->first.GetString().c_str()), UE::FVtValue{std::move(Iter->second)});
		}
#endif	  // #if USE_USD_SDK
		return Result;
	}

	FString FUsdPrim::GetDisplayName() const
	{
		FString Result;

#if USE_USD_SDK
		Result = UTF8_TO_TCHAR(Impl->PxrUsdPrim.Get().GetDisplayName().c_str());
#endif	  // #if USE_USD_SDK

		return Result;
	}

	bool FUsdPrim::SetDisplayName(const FString& Name) const
	{
#if USE_USD_SDK
		return Impl->PxrUsdPrim.Get().SetDisplayName(TCHAR_TO_UTF8(*Name));
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdPrim::ClearDisplayName() const
	{
#if USE_USD_SDK
		return Impl->PxrUsdPrim.Get().ClearDisplayName();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdPrim::HasAuthoredDisplayName() const
	{
#if USE_USD_SDK
		return Impl->PxrUsdPrim.Get().HasAuthoredDisplayName();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdPrim::SetSpecifier(ESdfSpecifier Specifier)
	{
#if USE_USD_SDK
		static_assert((int32)ESdfSpecifier::Def == (int32)pxr::SdfSpecifierDef, "ESdfSpecifier enum doesn't match USD!");
		static_assert((int32)ESdfSpecifier::Over == (int32)pxr::SdfSpecifierOver, "ESdfSpecifier enum doesn't match USD!");
		static_assert((int32)ESdfSpecifier::Class == (int32)pxr::SdfSpecifierClass, "ESdfSpecifier enum doesn't match USD!");
		static_assert((int32)ESdfSpecifier::Num == (int32)pxr::SdfNumSpecifiers, "ESdfSpecifier enum doesn't match USD!");

		return Impl->PxrUsdPrim.Get().SetSpecifier(static_cast<pxr::SdfSpecifier>(Specifier));
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdPrim::IsActive() const
	{
#if USE_USD_SDK
		return Impl->PxrUsdPrim.Get().IsActive();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdPrim::SetActive(bool bActive)
	{
#if USE_USD_SDK
		return Impl->PxrUsdPrim.Get().SetActive(bActive);
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdPrim::IsValid() const
	{
#if USE_USD_SDK
		return Impl->PxrUsdPrim.Get().IsValid();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdPrim::IsPseudoRoot() const
	{
#if USE_USD_SDK
		return Impl->PxrUsdPrim.Get().IsPseudoRoot();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdPrim::IsModel() const
	{
#if USE_USD_SDK
		return Impl->PxrUsdPrim.Get().IsModel();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdPrim::IsDefined() const
	{
#if USE_USD_SDK
		return Impl->PxrUsdPrim.Get().IsDefined();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdPrim::IsGroup() const
	{
#if USE_USD_SDK
		return Impl->PxrUsdPrim.Get().IsGroup();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	TArray<FName> FUsdPrim::GetAppliedSchemas() const
	{
		TArray<FName> AppliedSchemas;

#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		std::vector<pxr::TfToken> UsdAppliedSchemas = Impl->PxrUsdPrim.Get().GetAppliedSchemas();
		AppliedSchemas.Reserve(UsdAppliedSchemas.size());

		for (const pxr::TfToken& UsdSchema : UsdAppliedSchemas)
		{
			AppliedSchemas.Add(UTF8_TO_TCHAR(UsdSchema.GetString().c_str()));
		}
#endif	  // #if USE_USD_SDK

		return AppliedSchemas;
	}

	bool FUsdPrim::IsA(FName SchemaIdentifier) const
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		const pxr::TfToken UsdSchemaIdentifier(TCHAR_TO_UTF8(*SchemaIdentifier.ToString()));

		return Impl->PxrUsdPrim.Get().IsA(UsdSchemaIdentifier);
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdPrim::HasAPI(FName SchemaIdentifier) const
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		const pxr::TfToken UsdSchemaIdentifier(TCHAR_TO_UTF8(*SchemaIdentifier.ToString()));

		return Impl->PxrUsdPrim.Get().HasAPI(UsdSchemaIdentifier);
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdPrim::HasAPI(FName SchemaIdentifier, FName InstanceName) const
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		const pxr::TfToken UsdSchemaIdentifier(TCHAR_TO_UTF8(*SchemaIdentifier.ToString()));
		const pxr::TfToken UsdInstanceName(TCHAR_TO_UTF8(*InstanceName.ToString()));

		return Impl->PxrUsdPrim.Get().HasAPI(UsdSchemaIdentifier, UsdInstanceName);
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdPrim::CanApplyAPI(FName SchemaIdentifier, FString* OutWhyNot) const
	{
		bool bResult = false;

#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		const pxr::TfToken UsdSchemaIdentifier(TCHAR_TO_UTF8(*SchemaIdentifier.ToString()));

		std::string WhyNot;
		bResult = Impl->PxrUsdPrim.Get().CanApplyAPI(UsdSchemaIdentifier, OutWhyNot != nullptr ? &WhyNot : nullptr);

		if (!bResult && OutWhyNot != nullptr)
		{
			*OutWhyNot = UTF8_TO_TCHAR(WhyNot.c_str());
		}
#endif	  // #if USE_USD_SDK

		return bResult;
	}

	bool FUsdPrim::CanApplyAPI(FName SchemaIdentifier, FName InstanceName, FString* OutWhyNot) const
	{
		bool bResult = false;

#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		const pxr::TfToken UsdSchemaIdentifier(TCHAR_TO_UTF8(*SchemaIdentifier.ToString()));
		const pxr::TfToken UsdInstanceName(TCHAR_TO_UTF8(*InstanceName.ToString()));

		std::string WhyNot;
		bResult = Impl->PxrUsdPrim.Get().CanApplyAPI(UsdSchemaIdentifier, UsdInstanceName, OutWhyNot != nullptr ? &WhyNot : nullptr);

		if (!bResult && OutWhyNot != nullptr)
		{
			*OutWhyNot = UTF8_TO_TCHAR(WhyNot.c_str());
		}
#endif	  // #if USE_USD_SDK

		return bResult;
	}

	bool FUsdPrim::ApplyAPI(FName SchemaIdentifier) const
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		const pxr::TfToken UsdSchemaIdentifier(TCHAR_TO_UTF8(*SchemaIdentifier.ToString()));

		return Impl->PxrUsdPrim.Get().ApplyAPI(UsdSchemaIdentifier);
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdPrim::ApplyAPI(FName SchemaIdentifier, FName InstanceName) const
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		const pxr::TfToken UsdSchemaIdentifier(TCHAR_TO_UTF8(*SchemaIdentifier.ToString()));
		const pxr::TfToken UsdInstanceName(TCHAR_TO_UTF8(*InstanceName.ToString()));

		return Impl->PxrUsdPrim.Get().ApplyAPI(UsdSchemaIdentifier, UsdInstanceName);
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdPrim::RemoveAPI(FName SchemaIdentifier) const
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		const pxr::TfToken UsdSchemaIdentifier(TCHAR_TO_UTF8(*SchemaIdentifier.ToString()));

		return Impl->PxrUsdPrim.Get().RemoveAPI(UsdSchemaIdentifier);
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdPrim::RemoveAPI(FName SchemaIdentifier, FName InstanceName) const
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		const pxr::TfToken UsdSchemaIdentifier(TCHAR_TO_UTF8(*SchemaIdentifier.ToString()));
		const pxr::TfToken UsdInstanceName(TCHAR_TO_UTF8(*InstanceName.ToString()));

		return Impl->PxrUsdPrim.Get().RemoveAPI(UsdSchemaIdentifier, UsdInstanceName);
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	const FSdfPath FUsdPrim::GetPrimPath() const
	{
#if USE_USD_SDK
		return FSdfPath(Impl->PxrUsdPrim.Get().GetPrimPath());
#else
		return FSdfPath();
#endif	  // #if USE_USD_SDK
	}

	FUsdStage FUsdPrim::GetStage() const
	{
#if USE_USD_SDK
		return FUsdStage(Impl->PxrUsdPrim.Get().GetStage());
#else
		return FUsdStage();
#endif	  // #if USE_USD_SDK
	}

	FName FUsdPrim::GetName() const
	{
#if USE_USD_SDK
		return FName(UTF8_TO_TCHAR(Impl->PxrUsdPrim.Get().GetName().GetString().c_str()));
#else
		return FName();
#endif	  // #if USE_USD_SDK
	}

	FName FUsdPrim::GetTypeName() const
	{
#if USE_USD_SDK
		return FName(UTF8_TO_TCHAR(Impl->PxrUsdPrim.Get().GetTypeName().GetString().c_str()));
#else
		return FName();
#endif	  // #if USE_USD_SDK
	}

	bool FUsdPrim::SetTypeName(FName TypeName) const
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;
		return Impl->PxrUsdPrim.Get().SetTypeName(pxr::TfToken(TCHAR_TO_UTF8(*TypeName.ToString())));
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdPrim::ClearTypeName() const
	{
#if USE_USD_SDK
		return Impl->PxrUsdPrim.Get().ClearTypeName();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdPrim::HasAuthoredTypeName() const
	{
#if USE_USD_SDK
		return Impl->PxrUsdPrim.Get().HasAuthoredTypeName();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	FUsdPrim FUsdPrim::GetParent() const
	{
#if USE_USD_SDK
		return FUsdPrim(Impl->PxrUsdPrim.Get().GetParent());
#else
		return FUsdPrim();
#endif	  // #if USE_USD_SDK
	}

	TArray<FUsdPrim> FUsdPrim::GetChildren() const
	{
		TArray<FUsdPrim> Children;

#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;

		pxr::UsdPrimSiblingRange PrimChildren = Impl->PxrUsdPrim.Get().GetChildren();

		for (const pxr::UsdPrim& Child : PrimChildren)
		{
			Children.Emplace(Child);
		}
#endif	  // #if USE_USD_SDK

		return Children;
	}

	TArray<FUsdPrim> FUsdPrim::GetFilteredChildren(bool bTraverseInstanceProxies) const
	{
		TArray<FUsdPrim> Children;

#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;

		pxr::Usd_PrimFlagsPredicate Predicate = pxr::UsdPrimDefaultPredicate;

		if (bTraverseInstanceProxies)
		{
			Predicate = pxr::UsdTraverseInstanceProxies(Predicate);
		}

		pxr::UsdPrimSiblingRange PrimChildren = Impl->PxrUsdPrim.Get().GetFilteredChildren(Predicate);

		for (const pxr::UsdPrim& Child : PrimChildren)
		{
			Children.Emplace(Child);
		}
#endif	  // #if USE_USD_SDK

		return Children;
	}

	FUsdVariantSets FUsdPrim::GetVariantSets() const
	{
#if USE_USD_SDK
		return FUsdVariantSets(Impl->PxrUsdPrim.Get());
#else
		return FUsdVariantSets{};
#endif	  // #if USE_USD_SDK
	}

	FUsdVariantSet FUsdPrim::GetVariantSet(const FString& VariantSetName) const
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;

		const std::string UsdVariantSetName(TCHAR_TO_UTF8(*VariantSetName));

		return FUsdVariantSet(Impl->PxrUsdPrim.Get(), UsdVariantSetName);
#else
		return FUsdVariantSet{};
#endif	  // #if USE_USD_SDK
	}

	bool FUsdPrim::HasVariantSets() const
	{
#if USE_USD_SDK
		return Impl->PxrUsdPrim.Get().HasVariantSets();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	FUsdPayloads FUsdPrim::GetPayloads() const
	{
#if USE_USD_SDK
		return FUsdPayloads(Impl->PxrUsdPrim.Get());
#else
		return FUsdPayloads{};
#endif	  // #if USE_USD_SDK
	}

	bool FUsdPrim::HasAuthoredPayloads() const
	{
#if USE_USD_SDK
		return Impl->PxrUsdPrim.Get().HasAuthoredPayloads();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdPrim::IsLoaded() const
	{
#if USE_USD_SDK
		return Impl->PxrUsdPrim.Get().IsLoaded();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	void FUsdPrim::Load(EUsdLoadPolicy Policy)
	{
#if USE_USD_SDK
		static_assert((int32)EUsdLoadPolicy::UsdLoadWithDescendants == (int32)pxr::UsdLoadWithDescendants, "EUsdLoadPolicy enum doesn't match USD!");
		static_assert(
			(int32)EUsdLoadPolicy::UsdLoadWithoutDescendants == (int32)pxr::UsdLoadWithoutDescendants,
			"EUsdLoadPolicy enum doesn't match USD!"
		);

		Impl->PxrUsdPrim.Get().Load(static_cast<pxr::UsdLoadPolicy>(Policy));
#endif	  // #if USE_USD_SDK
	}

	void FUsdPrim::Unload()
	{
#if USE_USD_SDK
		Impl->PxrUsdPrim.Get().Unload();
#endif	  // #if USE_USD_SDK
	}

	TArray<FUsdAttribute> FUsdPrim::GetAttributes() const
	{
		TArray<FUsdAttribute> Attributes;

#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;

		for (const pxr::UsdAttribute& Attribute : Impl->PxrUsdPrim.Get().GetAttributes())
		{
			Attributes.Emplace(Attribute);
		}
#endif	  // #if USE_USD_SDK

		return Attributes;
	}

	FUsdReferences FUsdPrim::GetReferences() const
	{
#if USE_USD_SDK
		return FUsdReferences(Impl->PxrUsdPrim.Get());
#else
		return FUsdReferences{};
#endif	  // #if USE_USD_SDK
	}

	bool FUsdPrim::HasAuthoredReferences() const
	{
#if USE_USD_SDK
		return Impl->PxrUsdPrim.Get().HasAuthoredReferences();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	TArray<FSdfPrimSpec> FUsdPrim::GetPrimStack() const
	{
		TArray<FSdfPrimSpec> Result;

#if USE_USD_SDK
		TUsdStore<std::vector<pxr::SdfPrimSpecHandle>> UsdSpecStack = Impl->PxrUsdPrim.Get().GetPrimStack();

		Result.Reserve(UsdSpecStack->size());
		for (const pxr::SdfPrimSpecHandle& Handle : *UsdSpecStack)
		{
			Result.Add(FSdfPrimSpec{Handle});
		}
#endif	  // USE_USD_SDK

		return Result;
	}

	FPcpPrimIndex FUsdPrim::GetPrimIndex() const
	{
#if USE_USD_SDK
		return FPcpPrimIndex{Impl->PxrUsdPrim.Get().GetPrimIndex()};
#else
		return {};
#endif	  // #if USE_USD_SDK
	}

	FPcpPrimIndex FUsdPrim::ComputeExpandedPrimIndex() const
	{
#if USE_USD_SDK
		return FPcpPrimIndex{Impl->PxrUsdPrim.Get().ComputeExpandedPrimIndex()};
#else
		return {};
#endif	  // #if USE_USD_SDK
	}

	FUsdAttribute FUsdPrim::GetAttribute(const TCHAR* AttrName) const
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;

		return FUsdAttribute(Impl->PxrUsdPrim.Get().GetAttribute(pxr::TfToken(TCHAR_TO_UTF8(AttrName))));
#else
		return FUsdAttribute{};
#endif	  // #if USE_USD_SDK
	}

	bool FUsdPrim::HasAttribute(const TCHAR* AttrName) const
	{
#if USE_USD_SDK
		return Impl->PxrUsdPrim.Get().HasAttribute(pxr::TfToken(TCHAR_TO_UTF8(AttrName)));
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	FUsdAttribute FUsdPrim::CreateAttribute(const TCHAR* AttrName, FName TypeName) const
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;

		return FUsdAttribute(Impl->PxrUsdPrim.Get().CreateAttribute(
			pxr::TfToken(TCHAR_TO_UTF8(AttrName)),
			pxr::SdfSchema::GetInstance().FindType(TCHAR_TO_UTF8(*TypeName.ToString()))
		));
#else
		return FUsdAttribute{};
#endif	  // #if USE_USD_SDK
	}

	FUsdRelationship FUsdPrim::CreateRelationship(const TCHAR* RelName, bool bCustom) const
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;

		return FUsdRelationship(Impl->PxrUsdPrim.Get().CreateRelationship(pxr::TfToken(TCHAR_TO_UTF8(RelName)), bCustom));
#else
		return FUsdRelationship{};
#endif	  // #if USE_USD_SDK
	}

	FUsdRelationship FUsdPrim::GetRelationship(const TCHAR* RelationshipName) const
	{
#if USE_USD_SDK
		return FUsdRelationship(Impl->PxrUsdPrim.Get().GetRelationship(pxr::TfToken(TCHAR_TO_UTF8(RelationshipName))));
#else
		return FUsdRelationship();
#endif	  // #if USE_USD_SDK
	}

	TArray<FUsdRelationship> FUsdPrim::GetRelationships() const
	{
		TArray<FUsdRelationship> Relationships;

#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;

		for (const pxr::UsdRelationship& Relationship : Impl->PxrUsdPrim.Get().GetRelationships())
		{
			Relationships.Emplace(UE::FUsdRelationship{Relationship});
		}
#endif	  // #if USE_USD_SDK

		return Relationships;
	}

	bool FUsdPrim::HasRelationship(const TCHAR* RelName) const
	{
#if USE_USD_SDK
		return Impl->PxrUsdPrim.Get().HasRelationship(pxr::TfToken(TCHAR_TO_UTF8(RelName)));
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdPrim::RemoveProperty(FName PropName) const
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;
		return Impl->PxrUsdPrim.Get().RemoveProperty(pxr::TfToken(TCHAR_TO_UTF8(*PropName.ToString())));
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdPrim::IsInstanceable() const
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;
		return Impl->PxrUsdPrim.Get().IsInstanceable();
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool FUsdPrim::SetInstanceable(bool bInstanceable) const
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;
		return Impl->PxrUsdPrim.Get().SetInstanceable(bInstanceable);
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool FUsdPrim::ClearInstanceable() const
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;
		return Impl->PxrUsdPrim.Get().ClearInstanceable();
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool FUsdPrim::HasAuthoredInstanceable() const
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;
		return Impl->PxrUsdPrim.Get().HasAuthoredInstanceable();
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool FUsdPrim::IsInstance() const
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;
		return Impl->PxrUsdPrim.Get().IsInstance();
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool FUsdPrim::IsInstanceProxy() const
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;
		return Impl->PxrUsdPrim.Get().IsInstanceProxy();
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool FUsdPrim::IsPrototype() const
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;
		return Impl->PxrUsdPrim.Get().IsPrototype();
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool FUsdPrim::IsInPrototype() const
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;
		return Impl->PxrUsdPrim.Get().IsInPrototype();
#else
		return false;
#endif	  // USE_USD_SDK
	}

	FUsdPrim FUsdPrim::GetPrototype() const
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;
		return UE::FUsdPrim{Impl->PxrUsdPrim.Get().GetPrototype()};
#else
		return {};
#endif	  // USE_USD_SDK
	}

	FUsdPrim FUsdPrim::GetPrimInPrototype() const
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;
		return UE::FUsdPrim{Impl->PxrUsdPrim.Get().GetPrimInPrototype()};
#else
		return {};
#endif	  // USE_USD_SDK
	}

	TArray<FUsdPrim> FUsdPrim::GetInstances() const
	{
		TArray<FUsdPrim> Result;

#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		std::vector<pxr::UsdPrim> Instances = Impl->PxrUsdPrim.Get().GetInstances();
		Result.Reserve(Instances.size());

		for (const pxr::UsdPrim& Instance : Instances)
		{
			Result.Emplace(Instance);
		}
#endif	  // USE_USD_SDK

		return Result;
	}

	bool FUsdPrim::IsPrototypePath(const FSdfPath& Path)
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;
		return pxr::UsdPrim::IsPrototypePath(Path);
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool FUsdPrim::IsPathInPrototype(const FSdfPath& Path)
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;
		return pxr::UsdPrim::IsPathInPrototype(Path);
#else
		return false;
#endif	  // USE_USD_SDK
	}

}	 // namespace UE
