// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdWrappers/UsdReferences.h"

#include "USDMemory.h"

#include "UsdWrappers/UsdPrim.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/usd/prim.h"
#include "pxr/usd/usd/references.h"
#include "USDIncludesEnd.h"
#endif	  // #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
		class FUsdReferencesImpl
		{
		public:
			FUsdReferencesImpl() = default;

#if USE_USD_SDK
			explicit FUsdReferencesImpl(const pxr::UsdReferences& InUsdReferences)
				: PxrUsdPrim(InUsdReferences.GetPrim())
			{
			}

			explicit FUsdReferencesImpl(pxr::UsdReferences&& InUsdReferences)
				: PxrUsdPrim(InUsdReferences.GetPrim())
			{
			}

			explicit FUsdReferencesImpl(const pxr::UsdPrim& InUsdPrim)
				: PxrUsdPrim(InUsdPrim)
			{
			}

			pxr::UsdReferences GetInner()
			{
				FScopedUsdAllocs UsdAllocs;
				return PxrUsdPrim.Get().GetReferences();
			}

			// pxr::UsdReferences has no default constructor, so we store the prim itself.
			TUsdStore<pxr::UsdPrim> PxrUsdPrim;
#endif	  // #if USE_USD_SDK
		};
	}	  // namespace Internal

	FUsdReferences::FUsdReferences()
	{
		Impl = MakeUnique<Internal::FUsdReferencesImpl>();
	}

	FUsdReferences::FUsdReferences(const FUsdReferences& Other)
	{
#if USE_USD_SDK
		Impl = MakeUnique<Internal::FUsdReferencesImpl>(Other.Impl->PxrUsdPrim.Get());
#endif	  // #if USE_USD_SDK
	}

	FUsdReferences::FUsdReferences(FUsdReferences&& Other) = default;

	FUsdReferences& FUsdReferences::operator=(const FUsdReferences& Other)
	{
#if USE_USD_SDK
		Impl = MakeUnique<Internal::FUsdReferencesImpl>(Other.Impl->PxrUsdPrim.Get());
#endif	  // #if USE_USD_SDK

		return *this;
	}

	FUsdReferences& FUsdReferences::operator=(FUsdReferences&& Other)
	{
		Impl = MoveTemp(Other.Impl);

		return *this;
	}

	FUsdReferences::~FUsdReferences()
	{
		Impl.Reset();
	}

	FUsdReferences::operator bool() const
	{
#if USE_USD_SDK
		return (bool)Impl->PxrUsdPrim.Get();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

#if USE_USD_SDK
	FUsdReferences::FUsdReferences(const pxr::UsdReferences& InUsdReferences)
	{
		Impl = MakeUnique<Internal::FUsdReferencesImpl>(InUsdReferences);
	}

	FUsdReferences::FUsdReferences(pxr::UsdReferences&& InUsdReferences)
	{
		Impl = MakeUnique<Internal::FUsdReferencesImpl>(MoveTemp(InUsdReferences));
	}

	FUsdReferences& FUsdReferences::operator=(const pxr::UsdReferences& InUsdReferences)
	{
		Impl = MakeUnique<Internal::FUsdReferencesImpl>(InUsdReferences);
		return *this;
	}

	FUsdReferences& FUsdReferences::operator=(pxr::UsdReferences&& InUsdReferences)
	{
		Impl = MakeUnique<Internal::FUsdReferencesImpl>(MoveTemp(InUsdReferences));
		return *this;
	}

	FUsdReferences::operator pxr::UsdReferences()
	{
		return Impl->GetInner();
	}

	FUsdReferences::operator const pxr::UsdReferences() const
	{
		return Impl->GetInner();
	}

	// Private constructor accessible by FUsdPrim.
	FUsdReferences::FUsdReferences(const pxr::UsdPrim& InPrim)
	{
		Impl = MakeUnique<Internal::FUsdReferencesImpl>(InPrim);
	}
#endif	  // #if USE_USD_SDK

	bool FUsdReferences::AddReference(const FSdfReference& Reference, EUsdListPosition Position)
	{
#if USE_USD_SDK
		static_assert(
			(int32)EUsdListPosition::FrontOfPrependList == (int32)pxr::UsdListPositionFrontOfPrependList,
			"EUsdListPosition enum doesn't match USD!"
		);
		static_assert(
			(int32)EUsdListPosition::BackOfPrependList == (int32)pxr::UsdListPositionBackOfPrependList,
			"EUsdListPosition enum doesn't match USD!"
		);
		static_assert(
			(int32)EUsdListPosition::FrontOfAppendList == (int32)pxr::UsdListPositionFrontOfAppendList,
			"EUsdListPosition enum doesn't match USD!"
		);
		static_assert(
			(int32)EUsdListPosition::BackOfAppendList == (int32)pxr::UsdListPositionBackOfAppendList,
			"EUsdListPosition enum doesn't match USD!"
		);

		if (Impl->PxrUsdPrim.Get())
		{
			FScopedUsdAllocs UsdAllocs;

			const std::string AssetPath(TCHAR_TO_UTF8(*Reference.AssetPath));
			const pxr::SdfPath UsdPrimPath(Reference.PrimPath);
			const pxr::SdfLayerOffset UsdLayerOffset(Reference.LayerOffset.Offset, Reference.LayerOffset.Scale);

			const pxr::SdfReference UsdReference(AssetPath, UsdPrimPath, UsdLayerOffset);

			return Impl->PxrUsdPrim.Get().GetReferences().AddReference(UsdReference, static_cast<pxr::UsdListPosition>(Position));
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	bool FUsdReferences::AddReference(
		const FString& Identifier,
		const FSdfPath& PrimPath,
		const FSdfLayerOffset& LayerOffset,
		EUsdListPosition Position
	)
	{
#if USE_USD_SDK
		if (Impl->PxrUsdPrim.Get())
		{
			FScopedUsdAllocs UsdAllocs;

			const std::string UsdIdentifier(TCHAR_TO_UTF8(*Identifier));
			const pxr::SdfPath UsdPrimPath(PrimPath);
			const pxr::SdfLayerOffset UsdLayerOffset(LayerOffset.Offset, LayerOffset.Scale);

			return Impl->PxrUsdPrim.Get().GetReferences().AddReference(
				UsdIdentifier,
				UsdPrimPath,
				UsdLayerOffset,
				static_cast<pxr::UsdListPosition>(Position)
			);
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	bool FUsdReferences::AddReference(const FString& Identifier, const FSdfLayerOffset& LayerOffset, EUsdListPosition Position)
	{
#if USE_USD_SDK
		if (Impl->PxrUsdPrim.Get())
		{
			FScopedUsdAllocs UsdAllocs;

			const std::string UsdIdentifier(TCHAR_TO_UTF8(*Identifier));
			const pxr::SdfLayerOffset UsdLayerOffset(LayerOffset.Offset, LayerOffset.Scale);

			return Impl->PxrUsdPrim.Get().GetReferences().AddReference(UsdIdentifier, UsdLayerOffset, static_cast<pxr::UsdListPosition>(Position));
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	bool FUsdReferences::AddInternalReference(const FSdfPath& PrimPath, const FSdfLayerOffset& LayerOffset, EUsdListPosition Position)
	{
#if USE_USD_SDK
		if (Impl->PxrUsdPrim.Get())
		{
			FScopedUsdAllocs UsdAllocs;

			const pxr::SdfPath UsdPrimPath(PrimPath);
			const pxr::SdfLayerOffset UsdLayerOffset(LayerOffset.Offset, LayerOffset.Scale);

			return Impl->PxrUsdPrim.Get().GetReferences().AddInternalReference(
				UsdPrimPath,
				UsdLayerOffset,
				static_cast<pxr::UsdListPosition>(Position)
			);
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	bool FUsdReferences::RemoveReference(const FSdfReference& Reference)
	{
#if USE_USD_SDK
		if (Impl->PxrUsdPrim.Get())
		{
			FScopedUsdAllocs UsdAllocs;

			const std::string AssetPath(TCHAR_TO_UTF8(*Reference.AssetPath));
			const pxr::SdfPath UsdPrimPath(Reference.PrimPath);
			const pxr::SdfLayerOffset UsdLayerOffset(Reference.LayerOffset.Offset, Reference.LayerOffset.Scale);

			const pxr::SdfReference UsdReference(AssetPath, UsdPrimPath, UsdLayerOffset);

			return Impl->PxrUsdPrim.Get().GetReferences().RemoveReference(UsdReference);
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	bool FUsdReferences::ClearReferences()
	{
#if USE_USD_SDK
		if (Impl->PxrUsdPrim.Get())
		{
			FScopedUsdAllocs UsdAllocs;

			return Impl->PxrUsdPrim.Get().GetReferences().ClearReferences();
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	bool FUsdReferences::SetReferences(const TArray<FSdfReference>& Items)
	{
#if USE_USD_SDK
		if (Impl->PxrUsdPrim.Get())
		{
			FScopedUsdAllocs UsdAllocs;

			std::vector<pxr::SdfReference> UsdReferences;
			UsdReferences.reserve(Items.Num());

			for (const FSdfReference& Item : Items)
			{
				const std::string AssetPath(TCHAR_TO_UTF8(*Item.AssetPath));
				const pxr::SdfPath UsdPrimPath(Item.PrimPath);
				const pxr::SdfLayerOffset UsdLayerOffset(Item.LayerOffset.Offset, Item.LayerOffset.Scale);

				UsdReferences.emplace_back(AssetPath, UsdPrimPath, UsdLayerOffset);
			}

			return Impl->PxrUsdPrim.Get().GetReferences().SetReferences(UsdReferences);
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	FUsdPrim FUsdReferences::GetPrim() const
	{
#if USE_USD_SDK
		if (Impl->PxrUsdPrim.Get())
		{
			return FUsdPrim(Impl->PxrUsdPrim.Get());
		}
#endif	  // #if USE_USD_SDK

		return FUsdPrim{};
	}
}	 // namespace UE
