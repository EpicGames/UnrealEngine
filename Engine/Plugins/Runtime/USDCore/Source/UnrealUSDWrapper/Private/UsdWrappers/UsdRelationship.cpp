// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdWrappers/UsdRelationship.h"
#include "USDMemory.h"
#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/VtValue.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/usd/relationship.h"
#include "USDIncludesEnd.h"
#endif	  // #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
		class FUsdRelationshipImpl
		{
		public:
			FUsdRelationshipImpl() = default;

#if USE_USD_SDK
			explicit FUsdRelationshipImpl(const pxr::UsdRelationship& InUsdRelationship)
				: PxrUsdRelationship(InUsdRelationship)
			{
			}

			explicit FUsdRelationshipImpl(pxr::UsdRelationship&& InUsdRelationship)
				: PxrUsdRelationship(MoveTemp(InUsdRelationship))
			{
			}

			TUsdStore<pxr::UsdRelationship> PxrUsdRelationship;
#endif	  // #if USE_USD_SDK
		};
	}	  // namespace Internal
}

namespace UE
{
	FUsdRelationship::FUsdRelationship()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdRelationshipImpl>();
	}

	FUsdRelationship::FUsdRelationship(const FUsdRelationship& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdRelationshipImpl>(Other.Impl->PxrUsdRelationship.Get());
#endif	  // #if USE_USD_SDK
	}

	FUsdRelationship::FUsdRelationship(FUsdRelationship&& Other) = default;

	FUsdRelationship::~FUsdRelationship()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	FUsdRelationship& FUsdRelationship::operator=(const FUsdRelationship& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdRelationshipImpl>(Other.Impl->PxrUsdRelationship.Get());
#endif	  // #if USE_USD_SDK
		return *this;
	}

	FUsdRelationship& FUsdRelationship::operator=(FUsdRelationship&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MoveTemp(Other.Impl);

		return *this;
	}

	bool FUsdRelationship::operator==(const FUsdRelationship& Other) const
	{
#if USE_USD_SDK
		return Impl->PxrUsdRelationship.Get() == Other.Impl->PxrUsdRelationship.Get();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdRelationship::operator!=(const FUsdRelationship& Other) const
	{
		return !(*this == Other);
	}

	FUsdRelationship::operator bool() const
	{
#if USE_USD_SDK
		return (bool)Impl->PxrUsdRelationship.Get();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

#if USE_USD_SDK
	FUsdRelationship::FUsdRelationship(const pxr::UsdRelationship& InUsdRelationship)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdRelationshipImpl>(InUsdRelationship);
	}

	FUsdRelationship::FUsdRelationship(pxr::UsdRelationship&& InUsdRelationship)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdRelationshipImpl>(MoveTemp(InUsdRelationship));
	}

	FUsdRelationship& FUsdRelationship::operator=(const pxr::UsdRelationship& InUsdRelationship)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdRelationshipImpl>(InUsdRelationship);
		return *this;
	}

	FUsdRelationship& FUsdRelationship::operator=(pxr::UsdRelationship&& InUsdRelationship)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdRelationshipImpl>(MoveTemp(InUsdRelationship));
		return *this;
	}

	FUsdRelationship::operator pxr::UsdRelationship&()
	{
		return Impl->PxrUsdRelationship.Get();
	}

	FUsdRelationship::operator const pxr::UsdRelationship&() const
	{
		return Impl->PxrUsdRelationship.Get();
	}

	FUsdRelationship::operator pxr::UsdProperty&()
	{
		return Impl->PxrUsdRelationship.Get();
	}

	FUsdRelationship::operator const pxr::UsdProperty&() const
	{
		return Impl->PxrUsdRelationship.Get();
	}
#endif

	bool FUsdRelationship::GetMetadata(const TCHAR* Key, UE::FVtValue& Value) const
	{
#if USE_USD_SDK
		return Impl->PxrUsdRelationship.Get().GetMetadata(pxr::TfToken{TCHAR_TO_UTF8(Key)}, &Value.GetUsdValue());
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdRelationship::HasMetadata(const TCHAR* Key) const
	{
#if USE_USD_SDK
		return Impl->PxrUsdRelationship.Get().HasMetadata(pxr::TfToken{TCHAR_TO_UTF8(Key)});
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdRelationship::SetMetadata(const TCHAR* Key, const UE::FVtValue& Value) const
	{
#if USE_USD_SDK
		return Impl->PxrUsdRelationship.Get().SetMetadata(pxr::TfToken{TCHAR_TO_UTF8(Key)}, Value.GetUsdValue());
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdRelationship::ClearMetadata(const TCHAR* Key) const
	{
#if USE_USD_SDK
		return Impl->PxrUsdRelationship.Get().ClearMetadata(pxr::TfToken{TCHAR_TO_UTF8(Key)});
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdRelationship::SetTargets(const TArray<UE::FSdfPath>& Targets) const
	{
#if USE_USD_SDK
		std::vector<pxr::SdfPath> UsdTargets;
		UsdTargets.reserve(Targets.Num());
		for (const UE::FSdfPath& UETarget : Targets)
		{
			UsdTargets.push_back(pxr::SdfPath{UETarget});
		}

		return Impl->PxrUsdRelationship.Get().SetTargets(UsdTargets);
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool FUsdRelationship::ClearTargets(bool bRemoveSpec) const
	{
#if USE_USD_SDK
		return Impl->PxrUsdRelationship.Get().ClearTargets(bRemoveSpec);
#else
		return false;
#endif	  // USE_USD_SDK
	}

	bool FUsdRelationship::GetTargets(TArray<UE::FSdfPath>& Targets) const
	{
		Targets.Reset();
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		pxr::SdfPathVector UsdTargets;
		if (Impl->PxrUsdRelationship.Get().GetTargets(&UsdTargets))
		{
			const uint32 NumTargets = UsdTargets.size();
			for (uint32 TargetIndex = 0; TargetIndex < NumTargets; ++TargetIndex)
			{
				Targets.Add(UE::FSdfPath(UsdTargets[TargetIndex]));
			}
			return true;
		}

#endif	  // USE_USD_SDK
		return false;
	}

}	 // namespace UE
