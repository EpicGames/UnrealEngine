// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdWrappers/UsdResolveInfo.h"

#include "USDMemory.h"
#include "UsdWrappers/PcpNodeRef.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/usd/resolveInfo.h"
#include "USDIncludesEnd.h"
#endif	  // #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
		class FUsdResolveInfoImpl
		{
		public:
			FUsdResolveInfoImpl() = default;

#if USE_USD_SDK
			explicit FUsdResolveInfoImpl(const pxr::UsdResolveInfo& InUsdResolveInfo)
				: PxrUsdResolveInfo(InUsdResolveInfo)
			{
			}

			explicit FUsdResolveInfoImpl(pxr::UsdResolveInfo&& InUsdResolveInfo)
				: PxrUsdResolveInfo(MoveTemp(InUsdResolveInfo))
			{
			}

			TUsdStore<pxr::UsdResolveInfo> PxrUsdResolveInfo;
#endif	  // #if USE_USD_SDK
		};
	}
}

namespace UE
{
	FUsdResolveInfo::FUsdResolveInfo()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdResolveInfoImpl>();
	}

	FUsdResolveInfo::FUsdResolveInfo(const FUsdResolveInfo& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdResolveInfoImpl>(Other.Impl->PxrUsdResolveInfo.Get());
#endif	  // #if USE_USD_SDK
	}

	FUsdResolveInfo::FUsdResolveInfo(FUsdResolveInfo&& Other) = default;

	FUsdResolveInfo::~FUsdResolveInfo()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

#if USE_USD_SDK
	FUsdResolveInfo::FUsdResolveInfo(const pxr::UsdResolveInfo& InUsdResolveInfo)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdResolveInfoImpl>(InUsdResolveInfo);
	}

	FUsdResolveInfo::FUsdResolveInfo(pxr::UsdResolveInfo&& InUsdResolveInfo)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdResolveInfoImpl>(MoveTemp(InUsdResolveInfo));
	}

	FUsdResolveInfo::operator pxr::UsdResolveInfo&()
	{
		return Impl->PxrUsdResolveInfo.Get();
	}

	FUsdResolveInfo::operator const pxr::UsdResolveInfo&() const
	{
		return Impl->PxrUsdResolveInfo.Get();
	}
#endif	  // #if USE_USD_SDK

	UE::FPcpNodeRef FUsdResolveInfo::GetNode() const
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;
		return UE::FPcpNodeRef{Impl->PxrUsdResolveInfo.Get().GetNode()};
#else
		return {};
#endif	  // #if USE_USD_SDK
	}
}	 // namespace UE
