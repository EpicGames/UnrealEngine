// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdWrappers/UsdSkelAnimQuery.h"

#include "USDMemory.h"
#include "UsdWrappers/UsdPrim.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/usdSkel/animQuery.h"
#include "USDIncludesEnd.h"
#endif	  //  #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
		class FUsdSkelAnimQueryImpl
		{
		public:
			FUsdSkelAnimQueryImpl()
#if USE_USD_SDK
				: PxrUsdSkelAnimQuery()
#endif	  // #if USE_USD_SDK
			{
			}

#if USE_USD_SDK
			explicit FUsdSkelAnimQueryImpl(const pxr::UsdSkelAnimQuery& InUsdSkelAnimQuery)
				: PxrUsdSkelAnimQuery(InUsdSkelAnimQuery)
			{
			}

			explicit FUsdSkelAnimQueryImpl(pxr::UsdSkelAnimQuery&& InUsdSkelAnimQuery)
				: PxrUsdSkelAnimQuery(MoveTemp(InUsdSkelAnimQuery))
			{
			}

			TUsdStore<pxr::UsdSkelAnimQuery> PxrUsdSkelAnimQuery;
#endif	  // #if USE_USD_SDK
		};
	}	  // namespace Internal

	FUsdSkelAnimQuery::FUsdSkelAnimQuery()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelAnimQueryImpl>();
	}

	FUsdSkelAnimQuery::FUsdSkelAnimQuery(const FUsdSkelAnimQuery& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelAnimQueryImpl>(Other.Impl->PxrUsdSkelAnimQuery.Get());
#endif	  // #if USE_USD_SDK
	}

	FUsdSkelAnimQuery::FUsdSkelAnimQuery(FUsdSkelAnimQuery&& Other)
		: Impl(MakeUnique<Internal::FUsdSkelAnimQueryImpl>())
	{
		if (this != &Other)
		{
			Impl = MoveTemp(Other.Impl);
		}
	}

	FUsdSkelAnimQuery::~FUsdSkelAnimQuery()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	FUsdSkelAnimQuery& FUsdSkelAnimQuery::operator=(const FUsdSkelAnimQuery& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelAnimQueryImpl>(Other.Impl->PxrUsdSkelAnimQuery.Get());
#endif	  // #if USE_USD_SDK

		return *this;
	}

	FUsdSkelAnimQuery& FUsdSkelAnimQuery::operator=(FUsdSkelAnimQuery&& Other)
	{
		if (this != &Other)
		{
			FScopedUnrealAllocs UnrealAllocs;
			Impl = MoveTemp(Other.Impl);
		}

		return *this;
	}

	FUsdSkelAnimQuery::operator bool() const
	{
#if USE_USD_SDK
		return static_cast<bool>(Impl->PxrUsdSkelAnimQuery.Get());
#else
		return false;
#endif
	}

#if USE_USD_SDK
	FUsdSkelAnimQuery::FUsdSkelAnimQuery(const pxr::UsdSkelAnimQuery& InUsdSkelAnimQuery)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelAnimQueryImpl>(InUsdSkelAnimQuery);
	}

	FUsdSkelAnimQuery::FUsdSkelAnimQuery(pxr::UsdSkelAnimQuery&& InUsdSkelAnimQuery)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelAnimQueryImpl>(MoveTemp(InUsdSkelAnimQuery));
	}

	FUsdSkelAnimQuery::operator pxr::UsdSkelAnimQuery&()
	{
		return Impl->PxrUsdSkelAnimQuery.Get();
	}

	FUsdSkelAnimQuery::operator const pxr::UsdSkelAnimQuery&() const
	{
		return Impl->PxrUsdSkelAnimQuery.Get();
	}
#endif	  // #if USE_USD_SDK

	bool FUsdSkelAnimQuery::IsValid() const
	{
#if USE_USD_SDK
		return Impl->PxrUsdSkelAnimQuery.Get().IsValid();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	FUsdPrim FUsdSkelAnimQuery::GetPrim() const
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;
		return FUsdPrim{Impl->PxrUsdSkelAnimQuery.Get().GetPrim()};
#else
		return {};
#endif	  // #if USE_USD_SDK
	}

	bool FUsdSkelAnimQuery::ComputeBlendShapeWeights(TArray<float>& Weights, TOptional<double> TimeCode) const
	{
		bool bResult = false;

#if USE_USD_SDK
		pxr::UsdTimeCode UsdTimeCode = TimeCode.IsSet() ? TimeCode.GetValue() : pxr::UsdTimeCode::Default();

		TUsdStore<pxr::VtArray<float>> UsdWeights;
		bResult = Impl->PxrUsdSkelAnimQuery.Get().ComputeBlendShapeWeights(&UsdWeights.Get(), UsdTimeCode);
		if (!bResult)
		{
			return bResult;
		}

		Weights.SetNumUninitialized(UsdWeights.Get().size());
		FMemory::Memcpy(Weights.GetData(), UsdWeights.Get().cdata(), UsdWeights.Get().size() * sizeof(float));
#endif	  // #if USE_USD_SDK

		return bResult;
	}

	bool FUsdSkelAnimQuery::GetJointTransformTimeSamples(TArray<double>& TimeCodes) const
	{
		bool bResult = false;

#if USE_USD_SDK
		TUsdStore<std::vector<double>> UsdTimes;
		bResult = Impl->PxrUsdSkelAnimQuery.Get().GetJointTransformTimeSamples(&UsdTimes.Get());
		if (!bResult)
		{
			return bResult;
		}

		TimeCodes.SetNumUninitialized(UsdTimes.Get().size());
		FMemory::Memcpy(TimeCodes.GetData(), UsdTimes.Get().data(), UsdTimes.Get().size() * sizeof(double));
#endif	  // #if USE_USD_SDK

		return bResult;
	}

	bool FUsdSkelAnimQuery::GetBlendShapeWeightTimeSamples(TArray<double>& TimeCodes) const
	{
		bool bResult = false;

#if USE_USD_SDK
		TUsdStore<std::vector<double>> UsdTimes;
		bResult = Impl->PxrUsdSkelAnimQuery.Get().GetBlendShapeWeightTimeSamples(&UsdTimes.Get());
		if (!bResult)
		{
			return bResult;
		}

		TimeCodes.SetNumUninitialized(UsdTimes.Get().size());
		FMemory::Memcpy(TimeCodes.GetData(), UsdTimes.Get().data(), UsdTimes.Get().size() * sizeof(double));
#endif	  // #if USE_USD_SDK

		return bResult;
	}

	TArray<FString> FUsdSkelAnimQuery::GetJointOrder() const
	{
		TArray<FString> Result;

#if USE_USD_SDK
		TUsdStore<pxr::VtArray<pxr::TfToken>> UsdJointOrder = Impl->PxrUsdSkelAnimQuery.Get().GetJointOrder();
		Result.Reserve(UsdJointOrder.Get().size());

		for (const pxr::TfToken& UsdToken : UsdJointOrder.Get())
		{
			Result.Add(UTF8_TO_TCHAR(UsdToken.GetString().c_str()));
		}
#endif

		return Result;
	}

	TArray<FString> FUsdSkelAnimQuery::GetBlendShapeOrder() const
	{
		TArray<FString> Result;

#if USE_USD_SDK
		TUsdStore<pxr::VtArray<pxr::TfToken>> UsdJointOrder = Impl->PxrUsdSkelAnimQuery.Get().GetBlendShapeOrder();
		Result.Reserve(UsdJointOrder.Get().size());

		for (const pxr::TfToken& UsdToken : UsdJointOrder.Get())
		{
			Result.Add(UTF8_TO_TCHAR(UsdToken.GetString().c_str()));
		}
#endif

		return Result;
	}
}	 // namespace UE
