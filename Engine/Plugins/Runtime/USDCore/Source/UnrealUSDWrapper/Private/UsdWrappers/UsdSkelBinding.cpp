// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdWrappers/UsdSkelBinding.h"

#include "USDMemory.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdSkelSkinningQuery.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/usdSkel/binding.h"
#include "pxr/usd/usdSkel/skeleton.h"
#include "USDIncludesEnd.h"
#endif	  //  #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
		class FUsdSkelBindingImpl
		{
		public:
			FUsdSkelBindingImpl()
#if USE_USD_SDK
				: PxrUsdSkelBinding()
#endif	  // #if USE_USD_SDK
			{
			}

#if USE_USD_SDK
			explicit FUsdSkelBindingImpl(const pxr::UsdSkelBinding& InUsdSkelBinding)
				: PxrUsdSkelBinding(InUsdSkelBinding)
			{
			}

			explicit FUsdSkelBindingImpl(pxr::UsdSkelBinding&& InUsdSkelBinding)
				: PxrUsdSkelBinding(MoveTemp(InUsdSkelBinding))
			{
			}

			TUsdStore<pxr::UsdSkelBinding> PxrUsdSkelBinding;
#endif	  // #if USE_USD_SDK
		};
	}	  // namespace Internal

	FUsdSkelBinding::FUsdSkelBinding()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelBindingImpl>();
	}

	FUsdSkelBinding::FUsdSkelBinding(const FUsdSkelBinding& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelBindingImpl>(Other.Impl->PxrUsdSkelBinding.Get());
#endif	  // #if USE_USD_SDK
	}

	FUsdSkelBinding::FUsdSkelBinding(FUsdSkelBinding&& Other)
		: Impl(MakeUnique<Internal::FUsdSkelBindingImpl>())
	{
		if (this != &Other)
		{
			Impl = MoveTemp(Other.Impl);
		}
	}

	FUsdSkelBinding::~FUsdSkelBinding()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	FUsdSkelBinding& FUsdSkelBinding::operator=(const FUsdSkelBinding& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelBindingImpl>(Other.Impl->PxrUsdSkelBinding.Get());
#endif	  // #if USE_USD_SDK

		return *this;
	}

	FUsdSkelBinding& FUsdSkelBinding::operator=(FUsdSkelBinding&& Other)
	{
		if (this != &Other)
		{
			FScopedUnrealAllocs UnrealAllocs;
			Impl = MoveTemp(Other.Impl);
		}

		return *this;
	}

#if USE_USD_SDK
	FUsdSkelBinding::FUsdSkelBinding(const pxr::UsdSkelBinding& InUsdSkelBinding)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelBindingImpl>(InUsdSkelBinding);
	}

	FUsdSkelBinding::FUsdSkelBinding(pxr::UsdSkelBinding&& InUsdSkelBinding)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelBindingImpl>(MoveTemp(InUsdSkelBinding));
	}

	FUsdSkelBinding::operator pxr::UsdSkelBinding&()
	{
		return Impl->PxrUsdSkelBinding.Get();
	}

	FUsdSkelBinding::operator const pxr::UsdSkelBinding&() const
	{
		return Impl->PxrUsdSkelBinding.Get();
	}
#endif	  // #if USE_USD_SDK

	FUsdPrim FUsdSkelBinding::GetSkeleton() const
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;
		return FUsdPrim{Impl->PxrUsdSkelBinding.Get().GetSkeleton().GetPrim()};
#else
		return {};
#endif	  // #if USE_USD_SDK
	}

	TArray<FUsdSkelSkinningQuery> FUsdSkelBinding::GetSkinningTargets() const
	{
#if USE_USD_SDK
		TArray<FUsdSkelSkinningQuery> Result;

		FScopedUsdAllocs Allocs;
		pxr::VtArray<pxr::UsdSkelSkinningQuery> UsdQueries = Impl->PxrUsdSkelBinding.Get().GetSkinningTargets();

		Result.Reserve(UsdQueries.size());
		for (const pxr::UsdSkelSkinningQuery& UsdQuery : UsdQueries)
		{
			Result.Add(FUsdSkelSkinningQuery{UsdQuery});
		}

		return Result;
#else
		return {};
#endif	  // #if USE_USD_SDK
	}
}	 // namespace UE
