// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdWrappers/UsdSkelBlendShape.h"

#include "USDMemory.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdSkelBlendShape.h"
#include "UsdWrappers/UsdSkelInbetweenShape.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/usdSkel/blendShape.h"
#include "USDIncludesEnd.h"
#endif	  //  #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
		class FUsdSkelBlendShapeImpl
		{
		public:
			FUsdSkelBlendShapeImpl()
#if USE_USD_SDK
				: PxrUsdSkelBlendShape()
#endif	  // #if USE_USD_SDK
			{
			}

			FUsdSkelBlendShapeImpl(const UE::FUsdPrim& Prim)
#if USE_USD_SDK
				: PxrUsdSkelBlendShape(pxr::UsdSkelBlendShape{Prim})
#endif	  // #if USE_USD_SDK
			{
			}

#if USE_USD_SDK
			explicit FUsdSkelBlendShapeImpl(const pxr::UsdSkelBlendShape& InUsdSkelBlendShape)
				: PxrUsdSkelBlendShape(InUsdSkelBlendShape)
			{
			}

			explicit FUsdSkelBlendShapeImpl(pxr::UsdSkelBlendShape&& InUsdSkelBlendShape)
				: PxrUsdSkelBlendShape(MoveTemp(InUsdSkelBlendShape))
			{
			}

			TUsdStore<pxr::UsdSkelBlendShape> PxrUsdSkelBlendShape;
#endif	  // #if USE_USD_SDK
		};
	}	  // namespace Internal

	FUsdSkelBlendShape::FUsdSkelBlendShape()
		: FUsdTyped()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelBlendShapeImpl>();
	}

	FUsdSkelBlendShape::FUsdSkelBlendShape(const UE::FUsdPrim& Prim)
		: FUsdTyped(Prim)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelBlendShapeImpl>(Prim);
	}

	FUsdSkelBlendShape::FUsdSkelBlendShape(const FUsdSkelBlendShape& Other)
		: FUsdTyped(Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelBlendShapeImpl>(Other.Impl->PxrUsdSkelBlendShape.Get());
#endif	  // #if USE_USD_SDK
	}

	FUsdSkelBlendShape::FUsdSkelBlendShape(FUsdSkelBlendShape&& Other)
		: FUsdTyped(Other)
		, Impl(MakeUnique<Internal::FUsdSkelBlendShapeImpl>())
	{
		if (this != &Other)
		{
			Impl = MoveTemp(Other.Impl);
		}
	}

	FUsdSkelBlendShape::~FUsdSkelBlendShape()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	FUsdSkelBlendShape& FUsdSkelBlendShape::operator=(const FUsdSkelBlendShape& Other)
	{
		FUsdTyped::operator=(Other);

#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelBlendShapeImpl>(Other.Impl->PxrUsdSkelBlendShape.Get());
#endif	  // #if USE_USD_SDK

		return *this;
	}

	FUsdSkelBlendShape& FUsdSkelBlendShape::operator=(FUsdSkelBlendShape&& Other)
	{
		if (this != &Other)
		{
			// It's moving twice from Other but this should be fine, as FUsdTyped::operator=
			// will only really move FUsdTyped::Impl, and shouldn't have any effect on
			// Other's FUsdSkelBlendShape::Impl
			FUsdTyped::operator=(MoveTemp(Other));

			FScopedUnrealAllocs UnrealAllocs;
			Impl = MoveTemp(Other.Impl);
		}

		return *this;
	}

	FUsdSkelBlendShape::operator bool() const
	{
#if USE_USD_SDK
		return static_cast<bool>(Impl->PxrUsdSkelBlendShape.Get());
#else
		return false;
#endif
	}

#if USE_USD_SDK
	FUsdSkelBlendShape::FUsdSkelBlendShape(const pxr::UsdSkelBlendShape& InUsdSkelBlendShape)
		: FUsdTyped(InUsdSkelBlendShape)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelBlendShapeImpl>(InUsdSkelBlendShape);
	}

	FUsdSkelBlendShape::FUsdSkelBlendShape(pxr::UsdSkelBlendShape&& InUsdSkelBlendShape)
		: FUsdTyped(MoveTemp(InUsdSkelBlendShape))
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelBlendShapeImpl>(MoveTemp(InUsdSkelBlendShape));
	}

	FUsdSkelBlendShape::operator pxr::UsdSkelBlendShape&()
	{
		return Impl->PxrUsdSkelBlendShape.Get();
	}

	FUsdSkelBlendShape::operator const pxr::UsdSkelBlendShape&() const
	{
		return Impl->PxrUsdSkelBlendShape.Get();
	}
#endif	  // #if USE_USD_SDK

	FUsdSkelInbetweenShape FUsdSkelBlendShape::GetInbetween(const FString& Name) const
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		pxr::TfToken TokenName{TCHAR_TO_UTF8(*Name)};
		return UE::FUsdSkelInbetweenShape{Impl->PxrUsdSkelBlendShape.Get().GetInbetween(TokenName)};
#else
		return {};
#endif	  // #if USE_USD_SDK
	}

	TArray<FUsdSkelInbetweenShape> FUsdSkelBlendShape::GetInbetweens() const
	{
		TArray<FUsdSkelInbetweenShape> Result;

#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		std::vector<pxr::UsdSkelInbetweenShape> UsdInbetweens = Impl->PxrUsdSkelBlendShape.Get().GetInbetweens();
		Result.Reserve(UsdInbetweens.size());
		for (const pxr::UsdSkelInbetweenShape& UsdInbetween : UsdInbetweens)
		{
			Result.Add(UE::FUsdSkelInbetweenShape{UsdInbetween});
		}
#endif	  // #if USE_USD_SDK

		return Result;
	}

}	 // namespace UE
