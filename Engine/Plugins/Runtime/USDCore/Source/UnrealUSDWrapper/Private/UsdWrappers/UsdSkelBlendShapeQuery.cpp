// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdWrappers/UsdSkelBlendShapeQuery.h"

#include "USDMemory.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdSkelBlendShape.h"
#include "UsdWrappers/UsdSkelBlendShapeQuery.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/usdSkel/bindingAPI.h"
#include "pxr/usd/usdSkel/blendShapeQuery.h"
#include "USDIncludesEnd.h"
#endif	  //  #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
		class FUsdSkelBlendShapeQueryImpl
		{
		public:
			FUsdSkelBlendShapeQueryImpl()
#if USE_USD_SDK
				: PxrUsdSkelBlendShapeQuery()
#endif	  // #if USE_USD_SDK
			{
			}

			FUsdSkelBlendShapeQueryImpl(const UE::FUsdPrim& SkelBindingAPIPrim)
#if USE_USD_SDK
				: PxrUsdSkelBlendShapeQuery(pxr::UsdSkelBindingAPI{SkelBindingAPIPrim})
#endif	  // #if USE_USD_SDK
			{
			}

#if USE_USD_SDK
			explicit FUsdSkelBlendShapeQueryImpl(const pxr::UsdSkelBlendShapeQuery& InUsdSkelBlendShapeQuery)
				: PxrUsdSkelBlendShapeQuery(InUsdSkelBlendShapeQuery)
			{
			}

			explicit FUsdSkelBlendShapeQueryImpl(pxr::UsdSkelBlendShapeQuery&& InUsdSkelBlendShapeQuery)
				: PxrUsdSkelBlendShapeQuery(MoveTemp(InUsdSkelBlendShapeQuery))
			{
			}

			TUsdStore<pxr::UsdSkelBlendShapeQuery> PxrUsdSkelBlendShapeQuery;
#endif	  // #if USE_USD_SDK
		};
	}	  // namespace Internal

	FUsdSkelBlendShapeQuery::FUsdSkelBlendShapeQuery()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelBlendShapeQueryImpl>();
	}

	FUsdSkelBlendShapeQuery::FUsdSkelBlendShapeQuery(const UE::FUsdPrim& SkelBindingAPIPrim)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelBlendShapeQueryImpl>(SkelBindingAPIPrim);
	}

	FUsdSkelBlendShapeQuery::FUsdSkelBlendShapeQuery(const FUsdSkelBlendShapeQuery& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelBlendShapeQueryImpl>(Other.Impl->PxrUsdSkelBlendShapeQuery.Get());
#endif	  // #if USE_USD_SDK
	}

	FUsdSkelBlendShapeQuery::FUsdSkelBlendShapeQuery(FUsdSkelBlendShapeQuery&& Other)
		: Impl(MakeUnique<Internal::FUsdSkelBlendShapeQueryImpl>())
	{
		if (this != &Other)
		{
			Impl = MoveTemp(Other.Impl);
		}
	}

	FUsdSkelBlendShapeQuery::~FUsdSkelBlendShapeQuery()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	FUsdSkelBlendShapeQuery& FUsdSkelBlendShapeQuery::operator=(const FUsdSkelBlendShapeQuery& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelBlendShapeQueryImpl>(Other.Impl->PxrUsdSkelBlendShapeQuery.Get());
#endif	  // #if USE_USD_SDK

		return *this;
	}

	FUsdSkelBlendShapeQuery& FUsdSkelBlendShapeQuery::operator=(FUsdSkelBlendShapeQuery&& Other)
	{
		if (this != &Other)
		{
			FScopedUnrealAllocs UnrealAllocs;
			Impl = MoveTemp(Other.Impl);
		}

		return *this;
	}

	FUsdSkelBlendShapeQuery::operator bool() const
	{
#if USE_USD_SDK
		return static_cast<bool>(Impl->PxrUsdSkelBlendShapeQuery.Get());
#else
		return false;
#endif
	}

#if USE_USD_SDK
	FUsdSkelBlendShapeQuery::FUsdSkelBlendShapeQuery(const pxr::UsdSkelBlendShapeQuery& InUsdSkelBlendShapeQuery)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelBlendShapeQueryImpl>(InUsdSkelBlendShapeQuery);
	}

	FUsdSkelBlendShapeQuery::FUsdSkelBlendShapeQuery(pxr::UsdSkelBlendShapeQuery&& InUsdSkelBlendShapeQuery)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelBlendShapeQueryImpl>(MoveTemp(InUsdSkelBlendShapeQuery));
	}

	FUsdSkelBlendShapeQuery::operator pxr::UsdSkelBlendShapeQuery&()
	{
		return Impl->PxrUsdSkelBlendShapeQuery.Get();
	}

	FUsdSkelBlendShapeQuery::operator const pxr::UsdSkelBlendShapeQuery&() const
	{
		return Impl->PxrUsdSkelBlendShapeQuery.Get();
	}
#endif	  // #if USE_USD_SDK

	bool FUsdSkelBlendShapeQuery::IsValid() const
	{
#if USE_USD_SDK
		return Impl->PxrUsdSkelBlendShapeQuery.Get().IsValid();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	FUsdPrim FUsdSkelBlendShapeQuery::GetPrim() const
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;
		return FUsdPrim{Impl->PxrUsdSkelBlendShapeQuery.Get().GetPrim()};
#else
		return {};
#endif	  // #if USE_USD_SDK
	}

	size_t FUsdSkelBlendShapeQuery::GetNumBlendShapes() const
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;
		return Impl->PxrUsdSkelBlendShapeQuery.Get().GetNumBlendShapes();
#else
		return 0;
#endif	  // #if USE_USD_SDK
	}

	FUsdSkelBlendShape FUsdSkelBlendShapeQuery::GetBlendShape(size_t BlendShapeIndex) const
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		pxr::UsdSkelBlendShape BlendShape = Impl->PxrUsdSkelBlendShapeQuery.Get().GetBlendShape(BlendShapeIndex);
		if (BlendShape)
		{
			return UE::FUsdSkelBlendShape{BlendShape};
		}
#endif	  // #if USE_USD_SDK

		return {};
	}

}	 // namespace UE
