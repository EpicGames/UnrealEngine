// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdWrappers/UsdSkelCache.h"

#include "USDMemory.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdSkelAnimQuery.h"
#include "UsdWrappers/UsdSkelBinding.h"
#include "UsdWrappers/UsdSkelSkeletonQuery.h"
#include "UsdWrappers/UsdSkelSkinningQuery.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/usdSkel/animation.h"
#include "pxr/usd/usdSkel/cache.h"
#include "pxr/usd/usdSkel/root.h"
#include "pxr/usd/usdSkel/skeletonQuery.h"
#include "USDIncludesEnd.h"
#endif	  //  #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
		class FUsdSkelCacheImpl
		{
		public:
			FUsdSkelCacheImpl()
#if USE_USD_SDK
				: PxrUsdSkelCache()
#endif	  // #if USE_USD_SDK
			{
			}

#if USE_USD_SDK
			explicit FUsdSkelCacheImpl(const pxr::UsdSkelCache& InUsdSkelCache)
				: PxrUsdSkelCache(InUsdSkelCache)
			{
			}

			explicit FUsdSkelCacheImpl(pxr::UsdSkelCache&& InUsdSkelCache)
				: PxrUsdSkelCache(MoveTemp(InUsdSkelCache))
			{
			}

			TUsdStore<pxr::UsdSkelCache> PxrUsdSkelCache;
#endif	  // #if USE_USD_SDK
		};
	}	  // namespace Internal

	FUsdSkelCache::FUsdSkelCache()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelCacheImpl>();
	}

	FUsdSkelCache::FUsdSkelCache(const FUsdSkelCache& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelCacheImpl>(Other.Impl->PxrUsdSkelCache.Get());
#endif	  // #if USE_USD_SDK
	}

	FUsdSkelCache::FUsdSkelCache(FUsdSkelCache&& Other)
		: Impl(MakeUnique<Internal::FUsdSkelCacheImpl>())
	{
		if (this != &Other)
		{
			Impl = MoveTemp(Other.Impl);
		}
	}

	FUsdSkelCache::~FUsdSkelCache()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	FUsdSkelCache& FUsdSkelCache::operator=(const FUsdSkelCache& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelCacheImpl>(Other.Impl->PxrUsdSkelCache.Get());
#endif	  // #if USE_USD_SDK

		return *this;
	}

	FUsdSkelCache& FUsdSkelCache::operator=(FUsdSkelCache&& Other)
	{
		if (this != &Other)
		{
			FScopedUnrealAllocs UnrealAllocs;
			Impl = MoveTemp(Other.Impl);
		}

		return *this;
	}

#if USE_USD_SDK
	FUsdSkelCache::FUsdSkelCache(const pxr::UsdSkelCache& InUsdSkelCache)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelCacheImpl>(InUsdSkelCache);
	}

	FUsdSkelCache::FUsdSkelCache(pxr::UsdSkelCache&& InUsdSkelCache)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelCacheImpl>(MoveTemp(InUsdSkelCache));
	}

	FUsdSkelCache::operator pxr::UsdSkelCache&()
	{
		return Impl->PxrUsdSkelCache.Get();
	}

	FUsdSkelCache::operator const pxr::UsdSkelCache&() const
	{
		return Impl->PxrUsdSkelCache.Get();
	}
#endif	  // #if USE_USD_SDK

	void FUsdSkelCache::Clear()
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;
		Impl->PxrUsdSkelCache.Get().Clear();
#endif	  // #if USE_USD_SDK
	}

	bool FUsdSkelCache::Populate(const UE::FUsdPrim& SkelRootPrim, bool bTraverseInstanceProxies)
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		pxr::Usd_PrimFlagsPredicate Predicate = pxr::UsdPrimDefaultPredicate;
		if (bTraverseInstanceProxies)
		{
			Predicate = pxr::UsdTraverseInstanceProxies();
		}

		return Impl->PxrUsdSkelCache.Get().Populate(pxr::UsdSkelRoot{SkelRootPrim}, Predicate);
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	UE::FUsdSkelSkeletonQuery FUsdSkelCache::GetSkelQuery(const UE::FUsdPrim& SkeletonPrim) const
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;
		return UE::FUsdSkelSkeletonQuery{Impl->PxrUsdSkelCache.Get().GetSkelQuery(pxr::UsdSkelSkeleton{SkeletonPrim})};
#else
		return UE::FUsdSkelSkeletonQuery{};
#endif	  // #if USE_USD_SDK
	}

	UE::FUsdSkelAnimQuery FUsdSkelCache::GetAnimQuery(const UE::FUsdPrim& SkelAnimationPrim) const
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;
		return UE::FUsdSkelAnimQuery{Impl->PxrUsdSkelCache.Get().GetAnimQuery(pxr::UsdSkelAnimation{SkelAnimationPrim})};
#else
		return UE::FUsdSkelAnimQuery{};
#endif	  // #if USE_USD_SDK
	}

	UE::FUsdSkelSkinningQuery FUsdSkelCache::GetSkinningQuery(const UE::FUsdPrim& SkinnedPrim) const
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;
		return UE::FUsdSkelSkinningQuery{Impl->PxrUsdSkelCache.Get().GetSkinningQuery(SkinnedPrim)};
#else
		return UE::FUsdSkelSkinningQuery{};
#endif	  // #if USE_USD_SDK
	}

	bool FUsdSkelCache::ComputeSkelBindings(
		const UE::FUsdPrim& InSkelRootPrim,
		TArray<UE::FUsdSkelBinding>& OutBindings,
		bool bTraverseInstanceProxies
	) const
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		std::vector<pxr::UsdSkelBinding> UsdBindings;

		pxr::Usd_PrimFlagsPredicate Predicate = pxr::UsdPrimDefaultPredicate;
		if (bTraverseInstanceProxies)
		{
			Predicate = pxr::UsdTraverseInstanceProxies();
		}

		bool bSuccess = Impl->PxrUsdSkelCache.Get().ComputeSkelBindings(pxr::UsdSkelRoot{InSkelRootPrim}, &UsdBindings, Predicate);
		if (!bSuccess)
		{
			return false;
		}

		OutBindings.SetNum(UsdBindings.size());
		for (int32 Index = 0; Index < OutBindings.Num(); ++Index)
		{
			OutBindings[Index] = UE::FUsdSkelBinding{UsdBindings[Index]};
		}

		return true;
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FUsdSkelCache::ComputeSkelBinding(
		const UE::FUsdPrim& InSkelRootPrim,
		const UE::FUsdPrim& InSkeletonPrim,
		UE::FUsdSkelBinding& OutBinding,
		bool bTraverseInstanceProxies
	) const
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		pxr::Usd_PrimFlagsPredicate Predicate = pxr::UsdPrimDefaultPredicate;
		if (bTraverseInstanceProxies)
		{
			Predicate = pxr::UsdTraverseInstanceProxies();
		}

		return Impl->PxrUsdSkelCache.Get().ComputeSkelBinding(
			pxr::UsdSkelRoot{InSkelRootPrim},
			pxr::UsdSkelSkeleton{InSkeletonPrim},
			&static_cast<pxr::UsdSkelBinding&>(OutBinding),
			Predicate
		);
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

}	 // namespace UE
