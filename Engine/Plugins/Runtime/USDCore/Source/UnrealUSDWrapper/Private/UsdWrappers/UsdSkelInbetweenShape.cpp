// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdWrappers/UsdSkelInbetweenShape.h"

#include "USDMemory.h"
#include "UsdWrappers/UsdAttribute.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdSkelInbetweenShape.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/usdSkel/inbetweenShape.h"
#include "USDIncludesEnd.h"
#endif	  //  #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
		class FUsdSkelInbetweenShapeImpl
		{
		public:
			FUsdSkelInbetweenShapeImpl()
#if USE_USD_SDK
				: PxrUsdSkelInbetweenShape()
#endif	  // #if USE_USD_SDK
			{
			}

#if USE_USD_SDK
			explicit FUsdSkelInbetweenShapeImpl(const pxr::UsdSkelInbetweenShape& InUsdSkelInbetweenShape)
				: PxrUsdSkelInbetweenShape(InUsdSkelInbetweenShape)
			{
			}

			explicit FUsdSkelInbetweenShapeImpl(pxr::UsdSkelInbetweenShape&& InUsdSkelInbetweenShape)
				: PxrUsdSkelInbetweenShape(MoveTemp(InUsdSkelInbetweenShape))
			{
			}

			TUsdStore<pxr::UsdSkelInbetweenShape> PxrUsdSkelInbetweenShape;
#endif	  // #if USE_USD_SDK
		};
	}	  // namespace Internal

	FUsdSkelInbetweenShape::FUsdSkelInbetweenShape()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelInbetweenShapeImpl>();
	}

	FUsdSkelInbetweenShape::FUsdSkelInbetweenShape(const FUsdSkelInbetweenShape& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelInbetweenShapeImpl>(Other.Impl->PxrUsdSkelInbetweenShape.Get());
#endif	  // #if USE_USD_SDK
	}

	FUsdSkelInbetweenShape::FUsdSkelInbetweenShape(FUsdSkelInbetweenShape&& Other)
		: Impl(MakeUnique<Internal::FUsdSkelInbetweenShapeImpl>())
	{
		if (this != &Other)
		{
			Impl = MoveTemp(Other.Impl);
		}
	}

	FUsdSkelInbetweenShape::~FUsdSkelInbetweenShape()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	FUsdSkelInbetweenShape& FUsdSkelInbetweenShape::operator=(const FUsdSkelInbetweenShape& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelInbetweenShapeImpl>(Other.Impl->PxrUsdSkelInbetweenShape.Get());
#endif	  // #if USE_USD_SDK

		return *this;
	}

	FUsdSkelInbetweenShape& FUsdSkelInbetweenShape::operator=(FUsdSkelInbetweenShape&& Other)
	{
		if (this != &Other)
		{
			FScopedUnrealAllocs UnrealAllocs;
			Impl = MoveTemp(Other.Impl);
		}

		return *this;
	}

	FUsdSkelInbetweenShape::operator bool() const
	{
#if USE_USD_SDK
		return static_cast<bool>(Impl->PxrUsdSkelInbetweenShape.Get());
#else
		return false;
#endif
	}

#if USE_USD_SDK
	FUsdSkelInbetweenShape::FUsdSkelInbetweenShape(const pxr::UsdSkelInbetweenShape& InUsdSkelInbetweenShape)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelInbetweenShapeImpl>(InUsdSkelInbetweenShape);
	}

	FUsdSkelInbetweenShape::FUsdSkelInbetweenShape(pxr::UsdSkelInbetweenShape&& InUsdSkelInbetweenShape)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelInbetweenShapeImpl>(MoveTemp(InUsdSkelInbetweenShape));
	}

	FUsdSkelInbetweenShape::operator pxr::UsdSkelInbetweenShape&()
	{
		return Impl->PxrUsdSkelInbetweenShape.Get();
	}

	FUsdSkelInbetweenShape::operator const pxr::UsdSkelInbetweenShape&() const
	{
		return Impl->PxrUsdSkelInbetweenShape.Get();
	}
#endif	  // #if USE_USD_SDK

	bool FUsdSkelInbetweenShape::GetWeight(float* OutWeight) const
	{
#if USE_USD_SDK
		return Impl->PxrUsdSkelInbetweenShape.Get().GetWeight(OutWeight);
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	UE::FUsdAttribute FUsdSkelInbetweenShape::GetAttr() const
	{
#if USE_USD_SDK
		return UE::FUsdAttribute{Impl->PxrUsdSkelInbetweenShape.Get().GetAttr()};
#else
		return {};
#endif	  // #if USE_USD_SDK
	}

	bool FUsdSkelInbetweenShape::IsDefined() const
	{
#if USE_USD_SDK
		return Impl->PxrUsdSkelInbetweenShape.Get().IsDefined();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}
}	 // namespace UE
