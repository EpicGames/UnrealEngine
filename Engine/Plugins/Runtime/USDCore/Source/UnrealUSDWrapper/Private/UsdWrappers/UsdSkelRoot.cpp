// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdWrappers/UsdSkelRoot.h"

#include "USDMemory.h"

#include "UsdWrappers/UsdAttribute.h"
#include "UsdWrappers/UsdPrim.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/usdSkel/root.h"
#include "USDIncludesEnd.h"
#endif	  //  #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
		class FUsdSkelRootImpl
		{
		public:
			FUsdSkelRootImpl() = default;

#if USE_USD_SDK
			explicit FUsdSkelRootImpl(const pxr::UsdSkelRoot& InUsdSkelRoot)
				: PxrUsdSkelRoot(InUsdSkelRoot)
			{
			}

			explicit FUsdSkelRootImpl(pxr::UsdSkelRoot&& InUsdSkelRoot)
				: PxrUsdSkelRoot(MoveTemp(InUsdSkelRoot))
			{
			}

			explicit FUsdSkelRootImpl(const pxr::UsdPrim& InUsdPrim)
				: PxrUsdSkelRoot(pxr::UsdSkelRoot(InUsdPrim))
			{
			}

			TUsdStore<pxr::UsdSkelRoot> PxrUsdSkelRoot;
#endif	  // #if USE_USD_SDK
		};
	}

	FUsdSkelRoot::FUsdSkelRoot()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelRootImpl>();
	}

	FUsdSkelRoot::FUsdSkelRoot(const FUsdSkelRoot& Other)
		: FUsdTyped(Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelRootImpl>(Other.Impl->PxrUsdSkelRoot.Get());
#endif	  // #if USE_USD_SDK
	}

	FUsdSkelRoot::FUsdSkelRoot(FUsdSkelRoot&& Other)
		: FUsdTyped(MoveTemp(Other))
		, Impl(MoveTemp(Other.Impl))
	{
	}

	FUsdSkelRoot::FUsdSkelRoot(const FUsdPrim& Prim)
		: FUsdTyped(Prim)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelRootImpl>(Prim);
#endif	  // #if USE_USD_SDK
	}

	FUsdSkelRoot& FUsdSkelRoot::operator=(const FUsdSkelRoot& Other)
	{
		FUsdTyped::operator=(Other);

#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelRootImpl>(Other.Impl->PxrUsdSkelRoot.Get());
#endif	  // #if USE_USD_SDK

		return *this;
	}

	FUsdSkelRoot& FUsdSkelRoot::operator=(FUsdSkelRoot&& Other)
	{
		FUsdTyped::operator=(MoveTemp(Other));

		FScopedUnrealAllocs UnrealAllocs;
		Impl = MoveTemp(Other.Impl);

		return *this;
	}

	FUsdSkelRoot::~FUsdSkelRoot()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	FUsdSkelRoot::operator bool() const
	{
#if USE_USD_SDK
		return (bool)Impl->PxrUsdSkelRoot.Get();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

#if USE_USD_SDK
	FUsdSkelRoot::FUsdSkelRoot(const pxr::UsdSkelRoot& InUsdSkelRoot)
		: FUsdTyped(InUsdSkelRoot)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelRootImpl>(InUsdSkelRoot);
	}

	FUsdSkelRoot::FUsdSkelRoot(pxr::UsdSkelRoot&& InUsdSkelRoot)
		: FUsdTyped(MoveTemp(InUsdSkelRoot))
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelRootImpl>(MoveTemp(InUsdSkelRoot));
	}

	FUsdSkelRoot::FUsdSkelRoot(const pxr::UsdPrim& Prim)
		: FUsdTyped(Prim)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelRootImpl>(Prim);
	}

	FUsdSkelRoot::operator pxr::UsdSkelRoot&()
	{
		return Impl->PxrUsdSkelRoot.Get();
	}

	FUsdSkelRoot::operator const pxr::UsdSkelRoot&() const
	{
		return Impl->PxrUsdSkelRoot.Get();
	}
#endif	  // #if USE_USD_SDK

	FUsdSkelRoot FUsdSkelRoot::Find(const UE::FUsdPrim& Prim)
	{
#if USE_USD_SDK
		return FUsdSkelRoot{pxr::UsdSkelRoot::Find(Prim)};
#else
		return {};
#endif	  // #if USE_USD_SDK
	}
}	 // namespace UE
