// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdWrappers/UsdSkelSkeletonQuery.h"

#include "UnrealUSDWrapper.h"
#include "USDMemory.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdSkelAnimQuery.h"
#include "UsdWrappers/UsdSkelSkeletonQuery.h"
#include "UsdWrappers/UsdStage.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/usdSkel/skeleton.h"
#include "pxr/usd/usdSkel/skeletonQuery.h"
#include "USDIncludesEnd.h"
#endif	  //  #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
		class FUsdSkelSkeletonQueryImpl
		{
		public:
			FUsdSkelSkeletonQueryImpl()
#if USE_USD_SDK
				: PxrUsdSkelSkeletonQuery()
#endif	  // #if USE_USD_SDK
			{
			}

#if USE_USD_SDK
			explicit FUsdSkelSkeletonQueryImpl(const pxr::UsdSkelSkeletonQuery& InUsdSkelSkeletonQuery)
				: PxrUsdSkelSkeletonQuery(InUsdSkelSkeletonQuery)
			{
			}

			explicit FUsdSkelSkeletonQueryImpl(pxr::UsdSkelSkeletonQuery&& InUsdSkelSkeletonQuery)
				: PxrUsdSkelSkeletonQuery(MoveTemp(InUsdSkelSkeletonQuery))
			{
			}

			TUsdStore<pxr::UsdSkelSkeletonQuery> PxrUsdSkelSkeletonQuery;
#endif	  // #if USE_USD_SDK
		};
	}	  // namespace Internal

	FUsdSkelSkeletonQuery::FUsdSkelSkeletonQuery()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelSkeletonQueryImpl>();
	}

	FUsdSkelSkeletonQuery::FUsdSkelSkeletonQuery(const FUsdSkelSkeletonQuery& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelSkeletonQueryImpl>(Other.Impl->PxrUsdSkelSkeletonQuery.Get());
#endif	  // #if USE_USD_SDK
	}

	FUsdSkelSkeletonQuery::FUsdSkelSkeletonQuery(FUsdSkelSkeletonQuery&& Other)
		: Impl(MakeUnique<Internal::FUsdSkelSkeletonQueryImpl>())
	{
		if (this != &Other)
		{
			Impl = MoveTemp(Other.Impl);
		}
	}

	FUsdSkelSkeletonQuery::~FUsdSkelSkeletonQuery()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	FUsdSkelSkeletonQuery& FUsdSkelSkeletonQuery::operator=(const FUsdSkelSkeletonQuery& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelSkeletonQueryImpl>(Other.Impl->PxrUsdSkelSkeletonQuery.Get());
#endif	  // #if USE_USD_SDK

		return *this;
	}

	FUsdSkelSkeletonQuery& FUsdSkelSkeletonQuery::operator=(FUsdSkelSkeletonQuery&& Other)
	{
		if (this != &Other)
		{
			FScopedUnrealAllocs UnrealAllocs;
			Impl = MoveTemp(Other.Impl);
		}

		return *this;
	}

	FUsdSkelSkeletonQuery::operator bool() const
	{
#if USE_USD_SDK
		return static_cast<bool>(Impl->PxrUsdSkelSkeletonQuery.Get());
#else
		return false;
#endif
	}

#if USE_USD_SDK
	FUsdSkelSkeletonQuery::FUsdSkelSkeletonQuery(const pxr::UsdSkelSkeletonQuery& InUsdSkelSkeletonQuery)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelSkeletonQueryImpl>(InUsdSkelSkeletonQuery);
	}

	FUsdSkelSkeletonQuery::FUsdSkelSkeletonQuery(pxr::UsdSkelSkeletonQuery&& InUsdSkelSkeletonQuery)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelSkeletonQueryImpl>(MoveTemp(InUsdSkelSkeletonQuery));
	}

	FUsdSkelSkeletonQuery::operator pxr::UsdSkelSkeletonQuery&()
	{
		return Impl->PxrUsdSkelSkeletonQuery.Get();
	}

	FUsdSkelSkeletonQuery::operator const pxr::UsdSkelSkeletonQuery&() const
	{
		return Impl->PxrUsdSkelSkeletonQuery.Get();
	}
#endif	  // #if USE_USD_SDK

	bool FUsdSkelSkeletonQuery::IsValid() const
	{
#if USE_USD_SDK
		return Impl->PxrUsdSkelSkeletonQuery.Get().IsValid();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	FUsdPrim FUsdSkelSkeletonQuery::GetPrim() const
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;
		return FUsdPrim{Impl->PxrUsdSkelSkeletonQuery.Get().GetPrim()};
#else
		return {};
#endif	  // #if USE_USD_SDK
	}

	FUsdPrim FUsdSkelSkeletonQuery::GetSkeleton() const
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;
		return FUsdPrim{Impl->PxrUsdSkelSkeletonQuery.Get().GetSkeleton().GetPrim()};
#else
		return {};
#endif	  // #if USE_USD_SDK
	}

	FUsdSkelAnimQuery FUsdSkelSkeletonQuery::GetAnimQuery() const
	{
#if USE_USD_SDK
		FScopedUsdAllocs Allocs;
		return FUsdSkelAnimQuery{Impl->PxrUsdSkelSkeletonQuery.Get().GetAnimQuery()};
#else
		return {};
#endif	  // #if USE_USD_SDK
	}

	bool FUsdSkelSkeletonQuery::ComputeJointLocalTransforms(TArray<FTransform>& UESpaceTransforms, double TimeCode, bool bAtRest) const
	{
		bool bResult = false;

#if USE_USD_SDK
		TUsdStore<pxr::VtArray<pxr::GfMatrix4d>> UsdTransforms;
		bResult = Impl->PxrUsdSkelSkeletonQuery.Get().ComputeJointLocalTransforms(&UsdTransforms.Get(), TimeCode, bAtRest);
		if (!bResult)
		{
			return false;
		}

		UESpaceTransforms.Reset(UsdTransforms.Get().size());
		for (const pxr::GfMatrix4d& Matrix : UsdTransforms.Get())
		{
			// Copy-pasting from UsdToUnreal::ConvertMatrix since we UnrealUSDWrapper can't depend on USDUtilities
			FMatrix UnrealMatrix(
				FPlane(Matrix[0][0], Matrix[0][1], Matrix[0][2], Matrix[0][3]),
				FPlane(Matrix[1][0], Matrix[1][1], Matrix[1][2], Matrix[1][3]),
				FPlane(Matrix[2][0], Matrix[2][1], Matrix[2][2], Matrix[2][3]),
				FPlane(Matrix[3][0], Matrix[3][1], Matrix[3][2], Matrix[3][3])
			);
			UESpaceTransforms.Emplace(UnrealMatrix);
		}
#endif	  // #if USE_USD_SDK

		return bResult;
	}
}	 // namespace UE
