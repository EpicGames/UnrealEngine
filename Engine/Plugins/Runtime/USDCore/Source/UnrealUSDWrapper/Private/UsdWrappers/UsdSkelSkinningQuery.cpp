// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdWrappers/UsdSkelSkinningQuery.h"

#include "USDMemory.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdRelationship.h"
#include "UsdWrappers/UsdSkelSkinningQuery.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/usdSkel/skinningQuery.h"
#include "USDIncludesEnd.h"
#endif	  //  #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
		class FUsdSkelSkinningQueryImpl
		{
		public:
			FUsdSkelSkinningQueryImpl()
#if USE_USD_SDK
				: PxrUsdSkelSkinningQuery()
#endif	  // #if USE_USD_SDK
			{
			}

#if USE_USD_SDK
			explicit FUsdSkelSkinningQueryImpl(const pxr::UsdSkelSkinningQuery& InUsdSkelSkinningQuery)
				: PxrUsdSkelSkinningQuery(InUsdSkelSkinningQuery)
			{
			}

			explicit FUsdSkelSkinningQueryImpl(pxr::UsdSkelSkinningQuery&& InUsdSkelSkinningQuery)
				: PxrUsdSkelSkinningQuery(MoveTemp(InUsdSkelSkinningQuery))
			{
			}

			TUsdStore<pxr::UsdSkelSkinningQuery> PxrUsdSkelSkinningQuery;
#endif	  // #if USE_USD_SDK
		};
	}	  // namespace Internal

	FUsdSkelSkinningQuery::FUsdSkelSkinningQuery()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelSkinningQueryImpl>();
	}

	FUsdSkelSkinningQuery::FUsdSkelSkinningQuery(const FUsdSkelSkinningQuery& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelSkinningQueryImpl>(Other.Impl->PxrUsdSkelSkinningQuery.Get());
#endif	  // #if USE_USD_SDK
	}

	FUsdSkelSkinningQuery::FUsdSkelSkinningQuery(FUsdSkelSkinningQuery&& Other)
		: Impl(MakeUnique<Internal::FUsdSkelSkinningQueryImpl>())
	{
		if (this != &Other)
		{
			Impl = MoveTemp(Other.Impl);
		}
	}

	FUsdSkelSkinningQuery::~FUsdSkelSkinningQuery()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	FUsdSkelSkinningQuery& FUsdSkelSkinningQuery::operator=(const FUsdSkelSkinningQuery& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelSkinningQueryImpl>(Other.Impl->PxrUsdSkelSkinningQuery.Get());
#endif	  // #if USE_USD_SDK

		return *this;
	}

	FUsdSkelSkinningQuery& FUsdSkelSkinningQuery::operator=(FUsdSkelSkinningQuery&& Other)
	{
		if (this != &Other)
		{
			FScopedUnrealAllocs UnrealAllocs;
			Impl = MoveTemp(Other.Impl);
		}

		return *this;
	}

	FUsdSkelSkinningQuery::operator bool() const
	{
#if USE_USD_SDK
		return static_cast<bool>(Impl->PxrUsdSkelSkinningQuery.Get());
#else
		return false;
#endif
	}

#if USE_USD_SDK
	FUsdSkelSkinningQuery::FUsdSkelSkinningQuery(const pxr::UsdSkelSkinningQuery& InUsdSkelSkinningQuery)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelSkinningQueryImpl>(InUsdSkelSkinningQuery);
	}

	FUsdSkelSkinningQuery::FUsdSkelSkinningQuery(pxr::UsdSkelSkinningQuery&& InUsdSkelSkinningQuery)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdSkelSkinningQueryImpl>(MoveTemp(InUsdSkelSkinningQuery));
	}

	FUsdSkelSkinningQuery::operator pxr::UsdSkelSkinningQuery&()
	{
		return Impl->PxrUsdSkelSkinningQuery.Get();
	}

	FUsdSkelSkinningQuery::operator const pxr::UsdSkelSkinningQuery&() const
	{
		return Impl->PxrUsdSkelSkinningQuery.Get();
	}
#endif	  // #if USE_USD_SDK

	bool FUsdSkelSkinningQuery::IsValid() const
	{
#if USE_USD_SDK
		return Impl->PxrUsdSkelSkinningQuery.Get().IsValid();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	FUsdPrim FUsdSkelSkinningQuery::GetPrim() const
	{
#if USE_USD_SDK
		return FUsdPrim{Impl->PxrUsdSkelSkinningQuery.Get().GetPrim()};
#else
		return {};
#endif	  // #if USE_USD_SDK
	}

	UE::FUsdRelationship FUsdSkelSkinningQuery::GetBlendShapeTargetsRel() const
	{
#if USE_USD_SDK
		return UE::FUsdRelationship{Impl->PxrUsdSkelSkinningQuery.Get().GetBlendShapeTargetsRel()};
#else
		return {};
#endif	  // #if USE_USD_SDK
	}

	bool FUsdSkelSkinningQuery::GetBlendShapeOrder(TArray<FString>& BlendShapes) const
	{
		bool bResult = false;

#if USE_USD_SDK
		pxr::VtArray<pxr::TfToken> UsdOrder;
		{
			FScopedUsdAllocs Allocs;
			bResult = Impl->PxrUsdSkelSkinningQuery.Get().GetBlendShapeOrder(&UsdOrder);
			if (!bResult)
			{
				return bResult;
			}
		}

		BlendShapes.Reset(UsdOrder.size());
		for (const pxr::TfToken& BlendShape : UsdOrder)
		{
			BlendShapes.Add(UTF8_TO_TCHAR(BlendShape.GetString().c_str()));
		}
#endif

		return bResult;
	}

	FMatrix FUsdSkelSkinningQuery::GetGeomBindTransform(double UsdTimeCode) const
	{
		FMatrix Result = FMatrix::Identity;

#if USE_USD_SDK
		pxr::GfMatrix4d USDMatrix = Impl->PxrUsdSkelSkinningQuery.Get().GetGeomBindTransform(UsdTimeCode);

		Result = FMatrix(
			FPlane(USDMatrix[0][0], USDMatrix[0][1], USDMatrix[0][2], USDMatrix[0][3]),
			FPlane(USDMatrix[1][0], USDMatrix[1][1], USDMatrix[1][2], USDMatrix[1][3]),
			FPlane(USDMatrix[2][0], USDMatrix[2][1], USDMatrix[2][2], USDMatrix[2][3]),
			FPlane(USDMatrix[3][0], USDMatrix[3][1], USDMatrix[3][2], USDMatrix[3][3])
		);
#endif

		return Result;
	}

}	 // namespace UE
