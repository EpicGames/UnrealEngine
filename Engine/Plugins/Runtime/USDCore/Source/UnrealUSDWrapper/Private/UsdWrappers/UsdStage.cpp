// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdWrappers/UsdStage.h"

#include "UsdWrappers/SdfLayer.h"
#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/VtValue.h"

#include "USDMemory.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/usd/editContext.h"
#include "pxr/usd/usd/prim.h"
#include "pxr/usd/usd/stage.h"
#include "USDIncludesEnd.h"
#endif	  // #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
#if defined(ENABLE_USD_DEBUG_PATH) && ENABLE_USD_DEBUG_PATH
		// The only purpose of this class is so that we can cast the UsdStage impl to this type in the Unreal.natvis debug
		// visualizers, allowing us to inspect these FUsdStage and FUsdStageWeak variables in the debugger.
		//
		// Unlike the other wrapper types, we need these for FUsdStage because it is templated, and we can't express templated
		// type cast on the .natvis files very well.
		class UNREALUSDWRAPPER_API FUsdStageDebugImpl
		{
			FString DebugRootLayerIdentifier;
			FString DebugSessionLayerIdentifier;
			TUsdStore<int> FakePointer;
		};
#endif	  // ENABLE_USD_DEBUG_PATH

		template<typename PtrType>
		class FUsdStageImpl
		{
		public:
			FUsdStageImpl() = default;

#if defined(ENABLE_USD_DEBUG_PATH) && ENABLE_USD_DEBUG_PATH
			FString DebugRootLayerIdentifier;
			FString DebugSessionLayerIdentifier;
#endif

			explicit FUsdStageImpl(const PtrType& InUsdPtr)
				: UsdPtr(InUsdPtr)
			{
#if defined(ENABLE_USD_DEBUG_PATH) && ENABLE_USD_DEBUG_PATH
				if (const PtrType& Ptr = GetInner())
				{
					DebugRootLayerIdentifier = FString{UTF8_TO_TCHAR(Ptr->GetRootLayer()->GetIdentifier().c_str())};
					DebugSessionLayerIdentifier = FString{UTF8_TO_TCHAR(Ptr->GetSessionLayer()->GetIdentifier().c_str())};
				}
#endif
			}

			explicit FUsdStageImpl(PtrType&& InUsdPtr)
				: UsdPtr(MoveTemp(InUsdPtr))
			{
#if defined(ENABLE_USD_DEBUG_PATH) && ENABLE_USD_DEBUG_PATH
				if (const PtrType& Ptr = GetInner())
				{
					DebugRootLayerIdentifier = FString{UTF8_TO_TCHAR(Ptr->GetRootLayer()->GetIdentifier().c_str())};
					DebugSessionLayerIdentifier = FString{UTF8_TO_TCHAR(Ptr->GetSessionLayer()->GetIdentifier().c_str())};
				}
#endif
			}

			PtrType& GetInner()
			{
				return UsdPtr.Get();
			}

			const PtrType& GetInner() const
			{
				return UsdPtr.Get();
			}

			TUsdStore<PtrType> UsdPtr;
		};
	}	 // namespace Internal

	template<typename PtrType>
	FUsdStageBase<PtrType>::FUsdStageBase()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdStageImpl<PtrType>>();
	}

	template<typename PtrType>
	FUsdStageBase<PtrType>::FUsdStageBase(const FUsdStage& Other)
		: Impl(MakeUnique<Internal::FUsdStageImpl<PtrType>>(Other.Impl->GetInner()))
	{
	}

	template<typename PtrType>
	FUsdStageBase<PtrType>::FUsdStageBase(FUsdStage&& Other)
		: Impl(MakeUnique<Internal::FUsdStageImpl<PtrType>>(MoveTemp(Other.Impl->GetInner())))
	{
	}

	template<typename PtrType>
	FUsdStageBase<PtrType>::FUsdStageBase(const FUsdStageWeak& Other)
		: Impl(MakeUnique<Internal::FUsdStageImpl<PtrType>>(Other.Impl->GetInner()))
	{
	}

	template<typename PtrType>
	FUsdStageBase<PtrType>::FUsdStageBase(FUsdStageWeak&& Other)
		: Impl(MakeUnique<Internal::FUsdStageImpl<PtrType>>(MoveTemp(Other.Impl->GetInner())))
	{
	}

	template<typename PtrType>
	FUsdStageBase<PtrType>& FUsdStageBase<PtrType>::operator=(const FUsdStage& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdStageImpl<PtrType>>(Other.Impl->GetInner());
		return *this;
	}

	template<typename PtrType>
	FUsdStageBase<PtrType>& FUsdStageBase<PtrType>::operator=(FUsdStage&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdStageImpl<PtrType>>(MoveTemp(Other.Impl->GetInner()));
		return *this;
	}

	template<typename PtrType>
	FUsdStageBase<PtrType>& FUsdStageBase<PtrType>::operator=(const FUsdStageWeak& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdStageImpl<PtrType>>(Other.Impl->GetInner());
		return *this;
	}

	template<typename PtrType>
	FUsdStageBase<PtrType>& FUsdStageBase<PtrType>::operator=(FUsdStageWeak&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdStageImpl<PtrType>>(MoveTemp(Other.Impl->GetInner()));
		return *this;
	}

	template<typename PtrType>
	FUsdStageBase<PtrType>::operator bool() const
	{
		return (bool)Impl->GetInner();
	}

	template<typename PtrType>
	template<typename OtherPtrType>
	bool FUsdStageBase<PtrType>::operator==(const FUsdStageBase<OtherPtrType>& Other) const
	{
		return Impl->GetInner() == Other.Impl->GetInner();
	}

	template UNREALUSDWRAPPER_API bool FUsdStage::operator==(const FUsdStage& Other) const;
	template UNREALUSDWRAPPER_API bool FUsdStage::operator==(const FUsdStageWeak& Other) const;
	template UNREALUSDWRAPPER_API bool FUsdStageWeak::operator==(const FUsdStage& Other) const;
	template UNREALUSDWRAPPER_API bool FUsdStageWeak::operator==(const FUsdStageWeak& Other) const;

	template<typename PtrType>
	FUsdStageBase<PtrType>::~FUsdStageBase()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	// Auto conversion from/to PtrType
#if USE_USD_SDK
	template<typename PtrType>
	FUsdStageBase<PtrType>::FUsdStageBase(const pxr::UsdStageRefPtr& InUsdPtr)
	{
		FScopedUnrealAllocs UnrealAllocs;

		Impl = MakeUnique<Internal::FUsdStageImpl<PtrType>>(InUsdPtr);
	}

	template<typename PtrType>
	FUsdStageBase<PtrType>::FUsdStageBase(pxr::UsdStageRefPtr&& InUsdPtr)
	{
		FScopedUnrealAllocs UnrealAllocs;

		Impl = MakeUnique<Internal::FUsdStageImpl<PtrType>>(InUsdPtr);
	}

	template<typename PtrType>
	FUsdStageBase<PtrType>::FUsdStageBase(const pxr::UsdStageWeakPtr& InUsdPtr)
	{
		FScopedUnrealAllocs UnrealAllocs;

		Impl = MakeUnique<Internal::FUsdStageImpl<PtrType>>(InUsdPtr);
	}

	template<typename PtrType>
	FUsdStageBase<PtrType>::FUsdStageBase(pxr::UsdStageWeakPtr&& InUsdPtr)
	{
		FScopedUnrealAllocs UnrealAllocs;

		Impl = MakeUnique<Internal::FUsdStageImpl<PtrType>>(InUsdPtr);
	}

	template<typename PtrType>
	FUsdStageBase<PtrType>::operator PtrType&()
	{
		return Impl->GetInner();
	}

	template<typename PtrType>
	FUsdStageBase<PtrType>::operator const PtrType&() const
	{
		return Impl->GetInner();
	}

	template<typename PtrType>
	FUsdStageBase<PtrType>::operator pxr::UsdStageRefPtr() const
	{
		return Impl->GetInner();
	}

	template<typename PtrType>
	FUsdStageBase<PtrType>::operator pxr::UsdStageWeakPtr() const
	{
		return Impl->GetInner();
	}
#endif	  // USE_USD_SDK

	template<typename PtrType>
	void FUsdStageBase<PtrType>::LoadAndUnload(const TSet<FSdfPath>& LoadSet, const TSet<FSdfPath>& UnloadSet, EUsdLoadPolicy Policy)
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			FScopedUsdAllocs Allocs;

			pxr::SdfPathSet UsdLoadSet;
			for (const FSdfPath& Path : LoadSet)
			{
				UsdLoadSet.insert(static_cast<pxr::SdfPath>(Path));
			}

			pxr::SdfPathSet UsdUnloadSet;
			for (const FSdfPath& Path : UnloadSet)
			{
				UsdUnloadSet.insert(static_cast<pxr::SdfPath>(Path));
			}

			Ptr->LoadAndUnload(
				UsdLoadSet,
				UsdUnloadSet,
				Policy == EUsdLoadPolicy::UsdLoadWithDescendants ? pxr::UsdLoadPolicy::UsdLoadWithDescendants
																 : pxr::UsdLoadPolicy::UsdLoadWithoutDescendants
			);
		}
#endif
	}

	template<typename PtrType>
	FSdfLayer FUsdStageBase<PtrType>::GetRootLayer() const
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			return FSdfLayer(Ptr->GetRootLayer());
		}
#endif	  // #if USE_USD_SDK
		return FSdfLayer();
	}

	template<typename PtrType>
	bool FUsdStageBase<PtrType>::Export(const TCHAR* FileName, bool bAddSourceFileComment, const TMap<FString, FString>& FileFormatArguments) const
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			FScopedUsdAllocs Allocs;

			std::map<std::string, std::string> UsdFileFormatArguments;
			for (const TPair<FString, FString>& Pair : FileFormatArguments)
			{
				UsdFileFormatArguments.insert({TCHAR_TO_UTF8(*Pair.Key), TCHAR_TO_UTF8(*Pair.Value)});
			}

			return Ptr->Export(TCHAR_TO_UTF8(FileName), bAddSourceFileComment, UsdFileFormatArguments);
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	template<typename PtrType>
	FSdfLayer FUsdStageBase<PtrType>::GetSessionLayer() const
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			return FSdfLayer(Ptr->GetSessionLayer());
		}
#endif	  // #if USE_USD_SDK

		return FSdfLayer();
	}

	template<typename PtrType>
	bool FUsdStageBase<PtrType>::HasLocalLayer(const FSdfLayer& Layer) const
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			return Ptr->HasLocalLayer(pxr::SdfLayerRefPtr(Layer));
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	template<typename PtrType>
	FUsdPrim FUsdStageBase<PtrType>::GetPseudoRoot() const
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			return FUsdPrim(Ptr->GetPseudoRoot());
		}
#endif	  // #if USE_USD_SDK

		return FUsdPrim();
	}

	template<typename PtrType>
	FUsdPrim FUsdStageBase<PtrType>::GetDefaultPrim() const
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			return FUsdPrim(Ptr->GetDefaultPrim());
		}
#endif	  // #if USE_USD_SDK

		return FUsdPrim();
	}

	template<typename PtrType>
	FUsdPrim FUsdStageBase<PtrType>::GetPrimAtPath(const FSdfPath& Path) const
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			return FUsdPrim(Ptr->GetPrimAtPath(Path));
		}
#endif	  // #if USE_USD_SDK

		return FUsdPrim();
	}

	template<typename PtrType>
	bool FUsdStageBase<PtrType>::IsEditTargetValid() const
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			return Ptr->GetEditTarget().IsValid();
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	template<typename PtrType>
	void FUsdStageBase<PtrType>::SetEditTarget(const FSdfLayer& Layer)
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			FScopedUsdAllocs UsdAllocs;

			pxr::SdfLayerRefPtr LayerRef(Layer);
			const pxr::UsdEditTarget EditTarget = Ptr->GetEditTargetForLocalLayer(LayerRef);

			Ptr->SetEditTarget(EditTarget);
		}
#endif	  // #if USE_USD_SDK
	}

	template<typename PtrType>
	TArray<UE::FSdfLayer> FUsdStageBase<PtrType>::GetLayerStack(bool bIncludeSessionLayers /*= true */) const
	{
		TArray<UE::FSdfLayer> Result;
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			FScopedUsdAllocs UsdAllocs;

			pxr::SdfLayerHandleVector LayerHandleVector = Ptr->GetLayerStack(bIncludeSessionLayers);

			Result.Reserve(LayerHandleVector.size());

			for (pxr::SdfLayerHandle Layer : LayerHandleVector)
			{
				Result.Add(UE::FSdfLayer{Layer});
			}
		}
#endif	  // #if USE_USD_SDK
		return Result;
	}

	template<typename PtrType>
	TArray<UE::FSdfLayer> FUsdStageBase<PtrType>::GetUsedLayers(bool bIncludeClipLayers /*= true */) const
	{
		TArray<UE::FSdfLayer> Result;
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			FScopedUsdAllocs UsdAllocs;

			pxr::SdfLayerHandleVector LayerHandleVector = Ptr->GetUsedLayers(bIncludeClipLayers);

			Result.Reserve(LayerHandleVector.size());

			for (pxr::SdfLayerHandle Layer : LayerHandleVector)
			{
				Result.Add(UE::FSdfLayer{Layer});
			}
		}
#endif	  // #if USE_USD_SDK
		return Result;
	}

	template<typename PtrType>
	void FUsdStageBase<PtrType>::MuteAndUnmuteLayers(const TArray<FString>& MuteLayers, const TArray<FString>& UnmuteLayers)
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			FScopedUsdAllocs UsdAllocs;

			std::vector<std::string> UsdMuteLayers;
			UsdMuteLayers.reserve(MuteLayers.Num());
			for (const FString& MuteLayer : MuteLayers)
			{
				UsdMuteLayers.push_back(TCHAR_TO_UTF8(*MuteLayer));
			}

			std::vector<std::string> UsdUnmuteLayers;
			UsdUnmuteLayers.reserve(UnmuteLayers.Num());
			for (const FString& UnmuteLayer : UnmuteLayers)
			{
				UsdUnmuteLayers.push_back(TCHAR_TO_UTF8(*UnmuteLayer));
			}

			Ptr->MuteAndUnmuteLayers(UsdMuteLayers, UsdUnmuteLayers);
		}
#endif	  // #if USE_USD_SDK
	}

	template<typename PtrType>
	bool FUsdStageBase<PtrType>::IsLayerMuted(const FString& LayerIdentifier) const
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			FScopedUsdAllocs UsdAllocs;

			return Ptr->IsLayerMuted(TCHAR_TO_UTF8(*LayerIdentifier));
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	template<typename PtrType>
	FSdfLayer FUsdStageBase<PtrType>::GetEditTarget() const
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			if (IsEditTargetValid())
			{
				return FSdfLayer(Ptr->GetEditTarget().GetLayer());
			}
		}
#endif	  // #if USE_USD_SDK

		return FSdfLayer();
	}

	template<typename PtrType>
	bool FUsdStageBase<PtrType>::GetMetadata(const TCHAR* Key, UE::FVtValue& Value) const
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			return Ptr->GetMetadata(pxr::TfToken{TCHAR_TO_UTF8(Key)}, &Value.GetUsdValue());
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	template<typename PtrType>
	bool FUsdStageBase<PtrType>::HasMetadata(const TCHAR* Key) const
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			return Ptr->HasMetadata(pxr::TfToken{TCHAR_TO_UTF8(Key)});
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	template<typename PtrType>
	bool FUsdStageBase<PtrType>::SetMetadata(const TCHAR* Key, const UE::FVtValue& Value) const
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			return Ptr->SetMetadata(pxr::TfToken{TCHAR_TO_UTF8(Key)}, Value.GetUsdValue());
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	template<typename PtrType>
	bool FUsdStageBase<PtrType>::ClearMetadata(const TCHAR* Key) const
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			return Ptr->ClearMetadata(pxr::TfToken{TCHAR_TO_UTF8(Key)});
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	template<typename PtrType>
	double FUsdStageBase<PtrType>::GetStartTimeCode() const
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			return Ptr->GetStartTimeCode();
		}
#endif	  // #if USE_USD_SDK

		return 0.0;
	}

	template<typename PtrType>
	double FUsdStageBase<PtrType>::GetEndTimeCode() const
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			return Ptr->GetEndTimeCode();
		}
#endif	  // #if USE_USD_SDK

		return 0.0;
	}

	template<typename PtrType>
	void FUsdStageBase<PtrType>::SetStartTimeCode(double TimeCode)
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			if (!FMath::IsNearlyEqual(TimeCode, Ptr->GetStartTimeCode()) || !Ptr->HasAuthoredTimeCodeRange())
			{
				Ptr->SetStartTimeCode(TimeCode);
			}
		}
#endif	  // #if USE_USD_SDK
	}

	template<typename PtrType>
	void FUsdStageBase<PtrType>::SetEndTimeCode(double TimeCode)
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			if (!FMath::IsNearlyEqual(TimeCode, Ptr->GetEndTimeCode()) || !Ptr->HasAuthoredTimeCodeRange())
			{
				Ptr->SetEndTimeCode(TimeCode);
			}
		}
#endif	  // #if USE_USD_SDK
	}

	template<typename PtrType>
	double FUsdStageBase<PtrType>::GetTimeCodesPerSecond() const
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			return Ptr->GetTimeCodesPerSecond();
		}
#endif	  // #if USE_USD_SDK

		return 24.0;
	}

	template<typename PtrType>
	void FUsdStageBase<PtrType>::SetTimeCodesPerSecond(double TimeCodesPerSecond)
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			FScopedUsdAllocs Allocs;

			pxr::UsdEditContext EditContext(Ptr, Ptr->GetRootLayer());
			Ptr->SetTimeCodesPerSecond(TimeCodesPerSecond);
		}
#endif
	}

	template<typename PtrType>
	double FUsdStageBase<PtrType>::GetFramesPerSecond() const
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			return Ptr->GetFramesPerSecond();
		}
#endif	  // #if USE_USD_SDK

		return 24.0;
	}

	template<typename PtrType>
	void FUsdStageBase<PtrType>::SetFramesPerSecond(double FramesPerSecond)
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			FScopedUsdAllocs Allocs;

			pxr::UsdEditContext EditContext(Ptr, Ptr->GetRootLayer());
			Ptr->SetFramesPerSecond(FramesPerSecond);
		}
#endif
	}

	template<typename PtrType>
	void FUsdStageBase<PtrType>::SetInterpolationType(EUsdInterpolationType InterpolationType)
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			FScopedUsdAllocs Allocs;

			Ptr->SetInterpolationType(
				InterpolationType == EUsdInterpolationType::Held ? pxr::UsdInterpolationTypeHeld : pxr::UsdInterpolationTypeLinear
			);
		}
#endif
	}

	template<typename PtrType>
	EUsdInterpolationType FUsdStageBase<PtrType>::GetInterpolationType() const
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			if (Ptr->GetInterpolationType() == pxr::UsdInterpolationTypeHeld)
			{
				return EUsdInterpolationType::Held;
			}
		}
#endif

		return EUsdInterpolationType::Linear;
	}

	template<typename PtrType>
	void FUsdStageBase<PtrType>::SetDefaultPrim(const FUsdPrim& Prim)
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			Ptr->SetDefaultPrim(Prim);
		}
#endif	  // #if USE_USD_SDK
	}

	template<typename PtrType>
	FUsdPrim FUsdStageBase<PtrType>::OverridePrim(const FSdfPath& Path)
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			return FUsdPrim(Ptr->OverridePrim(Path));
		}
#endif	  // #if USE_USD_SDK

		return FUsdPrim();
	}

	template<typename PtrType>
	FUsdPrim FUsdStageBase<PtrType>::DefinePrim(const FSdfPath& Path, const TCHAR* TypeName)
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			return FUsdPrim(Ptr->DefinePrim(Path, pxr::TfToken(TCHAR_TO_UTF8(TypeName))));
		}
#endif	  // #if USE_USD_SDK

		return FUsdPrim();
	}

	template<typename PtrType>
	FUsdPrim FUsdStageBase<PtrType>::CreateClassPrim(const FSdfPath& RootPrimPath)
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			return FUsdPrim(Ptr->CreateClassPrim(RootPrimPath));
		}
#endif	  // #if USE_USD_SDK

		return FUsdPrim();
	}

	template<typename PtrType>
	bool FUsdStageBase<PtrType>::RemovePrim(const FSdfPath& Path)
	{
#if USE_USD_SDK
		if (PtrType& Ptr = Impl->GetInner())
		{
			return Ptr->RemovePrim(Path);
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

#if USE_USD_SDK
	template class FUsdStageBase<pxr::UsdStageRefPtr>;
	template class FUsdStageBase<pxr::UsdStageWeakPtr>;
#else
	template class FUsdStageBase<FDummyRefPtrType>;
	template class FUsdStageBase<FDummyWeakPtrType>;
#endif	  // #if USE_USD_SDK
}	 // namespace UE
