// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdWrappers/UsdTyped.h"

#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdPrim.h"

#include "USDMemory.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/usd/typed.h"
#include "USDIncludesEnd.h"
#endif	  //  #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
		class FUsdTypedImpl
		{
		public:
			FUsdTypedImpl() = default;

#if USE_USD_SDK
			explicit FUsdTypedImpl(const pxr::UsdTyped& InUsdTyped)
				: PxrUsdTyped(InUsdTyped)
			{
			}

			explicit FUsdTypedImpl(pxr::UsdTyped&& InUsdTyped)
				: PxrUsdTyped(MoveTemp(InUsdTyped))
			{
			}

			explicit FUsdTypedImpl(const pxr::UsdPrim& InUsdPrim)
				: PxrUsdTyped(pxr::UsdTyped(InUsdPrim))
			{
			}

			TUsdStore<pxr::UsdTyped> PxrUsdTyped;
#endif	  // #if USE_USD_SDK
		};
	}

	FUsdTyped::FUsdTyped()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdTypedImpl>();
	}

	FUsdTyped::FUsdTyped(const FUsdTyped& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdTypedImpl>(Other.Impl->PxrUsdTyped.Get());
#endif	  // #if USE_USD_SDK
	}

	FUsdTyped::FUsdTyped(FUsdTyped&& Other) = default;

	FUsdTyped::FUsdTyped(const FUsdPrim& Prim)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdTypedImpl>(Prim);
#endif	  // #if USE_USD_SDK
	}

	FUsdTyped& FUsdTyped::operator=(const FUsdTyped& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdTypedImpl>(Other.Impl->PxrUsdTyped.Get());
#endif	  // #if USE_USD_SDK
		return *this;
	}

	FUsdTyped& FUsdTyped::operator=(FUsdTyped&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MoveTemp(Other.Impl);

		return *this;
	}

	FUsdTyped::~FUsdTyped()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	FUsdTyped::operator bool() const
	{
#if USE_USD_SDK
		return (bool)Impl->PxrUsdTyped.Get();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

#if USE_USD_SDK
	FUsdTyped::FUsdTyped(const pxr::UsdTyped& InUsdTyped)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdTypedImpl>(InUsdTyped);
	}

	FUsdTyped::FUsdTyped(pxr::UsdTyped&& InUsdTyped)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdTypedImpl>(MoveTemp(InUsdTyped));
	}

	FUsdTyped::FUsdTyped(const pxr::UsdPrim& Prim)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FUsdTypedImpl>(Prim);
	}

	FUsdTyped::operator pxr::UsdTyped&()
	{
		return Impl->PxrUsdTyped.Get();
	}

	FUsdTyped::operator const pxr::UsdTyped&() const
	{
		return Impl->PxrUsdTyped.Get();
	}
#endif	  // #if USE_USD_SDK

	FSdfPath FUsdTyped::GetPath() const
	{
#if USE_USD_SDK
		return FSdfPath(Impl->PxrUsdTyped.Get().GetPath());
#else
		return FSdfPath();
#endif	  // #if USE_USD_SDK
	}

	FUsdPrim FUsdTyped::GetPrim() const
	{
#if USE_USD_SDK
		return FUsdPrim(Impl->PxrUsdTyped.Get().GetPrim());
#else
		return FUsdPrim();
#endif	  // #if USE_USD_SDK
	}
}	 // namespace UE
