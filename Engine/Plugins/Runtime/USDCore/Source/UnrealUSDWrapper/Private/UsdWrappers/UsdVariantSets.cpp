// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdWrappers/UsdVariantSets.h"

#include "USDMemory.h"

#include "UsdWrappers/UsdPrim.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/usd/prim.h"
#include "pxr/usd/usd/variantSets.h"
#include "USDIncludesEnd.h"

#include <map>
#include <utility>
#include <vector>
#endif	  // #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
		class FUsdVariantSetImpl
		{
		public:
			FUsdVariantSetImpl() = default;

#if USE_USD_SDK
			explicit FUsdVariantSetImpl(const pxr::UsdVariantSet& InUsdVariantSet)
				: PxrUsdPrim(InUsdVariantSet.GetPrim())
				, VariantSetName(InUsdVariantSet.GetName())
			{
			}

			explicit FUsdVariantSetImpl(pxr::UsdVariantSet&& InUsdVariantSet)
				: PxrUsdPrim(InUsdVariantSet.GetPrim())
				, VariantSetName(InUsdVariantSet.GetName())
			{
			}

			explicit FUsdVariantSetImpl(const pxr::UsdPrim& InUsdPrim, const std::string& InVariantSetName)
				: PxrUsdPrim(InUsdPrim)
				, VariantSetName(InVariantSetName)
			{
			}

			pxr::UsdVariantSet GetInner()
			{
				FScopedUsdAllocs UsdAllocs;
				return PxrUsdPrim.Get().GetVariantSet(VariantSetName);
			}

			// pxr::UsdVariantSet has no default constructor, so we store the prim
			// and the variant set name independently and fetch the object on demand.
			TUsdStore<pxr::UsdPrim> PxrUsdPrim;
			std::string VariantSetName;
#endif	  // #if USE_USD_SDK
		};

		class FUsdVariantSetsImpl
		{
		public:
			FUsdVariantSetsImpl() = default;

#if USE_USD_SDK
			explicit FUsdVariantSetsImpl(const pxr::UsdPrim& InUsdPrim)
				: PxrUsdPrim(InUsdPrim)
			{
			}

			pxr::UsdVariantSets GetInner()
			{
				FScopedUsdAllocs UsdAllocs;
				return PxrUsdPrim.Get().GetVariantSets();
			}

			// pxr::UsdVariantSets has no default constructor, so we store the
			// prim itself. It also does not provide an accessor for its prim,
			// so copy and move constructing from a pxr::UsdVariantSets object
			// are not supported.
			TUsdStore<pxr::UsdPrim> PxrUsdPrim;
#endif	  // #if USE_USD_SDK
		};
	}	  // namespace Internal

	// FUsdVariantSet

	FUsdVariantSet::FUsdVariantSet()
	{
		Impl = MakeUnique<Internal::FUsdVariantSetImpl>();
	}

	FUsdVariantSet::FUsdVariantSet(const FUsdVariantSet& Other)
	{
#if USE_USD_SDK
		Impl = MakeUnique<Internal::FUsdVariantSetImpl>(Other.Impl->PxrUsdPrim.Get(), Other.Impl->VariantSetName);
#endif	  // #if USE_USD_SDK
	}

	FUsdVariantSet::FUsdVariantSet(FUsdVariantSet&& Other) = default;

	FUsdVariantSet& FUsdVariantSet::operator=(const FUsdVariantSet& Other)
	{
#if USE_USD_SDK
		Impl = MakeUnique<Internal::FUsdVariantSetImpl>(Other.Impl->PxrUsdPrim.Get(), Other.Impl->VariantSetName);
#endif	  // #if USE_USD_SDK

		return *this;
	}

	FUsdVariantSet& FUsdVariantSet::operator=(FUsdVariantSet&& Other)
	{
		Impl = MoveTemp(Other.Impl);

		return *this;
	}

	FUsdVariantSet::~FUsdVariantSet()
	{
		Impl.Reset();
	}

	FUsdVariantSet::operator bool() const
	{
		return IsValid();
	}

#if USE_USD_SDK
	FUsdVariantSet::FUsdVariantSet(const pxr::UsdVariantSet& InUsdVariantSet)
	{
		Impl = MakeUnique<Internal::FUsdVariantSetImpl>(InUsdVariantSet);
	}

	FUsdVariantSet::FUsdVariantSet(pxr::UsdVariantSet&& InUsdVariantSet)
	{
		Impl = MakeUnique<Internal::FUsdVariantSetImpl>(MoveTemp(InUsdVariantSet));
	}

	FUsdVariantSet& FUsdVariantSet::operator=(const pxr::UsdVariantSet& InUsdVariantSet)
	{
		Impl = MakeUnique<Internal::FUsdVariantSetImpl>(InUsdVariantSet);
		return *this;
	}

	FUsdVariantSet& FUsdVariantSet::operator=(pxr::UsdVariantSet&& InUsdVariantSet)
	{
		Impl = MakeUnique<Internal::FUsdVariantSetImpl>(MoveTemp(InUsdVariantSet));
		return *this;
	}

	FUsdVariantSet::operator pxr::UsdVariantSet()
	{
		return Impl->GetInner();
	}

	FUsdVariantSet::operator const pxr::UsdVariantSet() const
	{
		return Impl->GetInner();
	}

	// Private constructor accessible by FUsdPrim and FUsdVariantSets.
	FUsdVariantSet::FUsdVariantSet(const pxr::UsdPrim& InPrim, const std::string& InVariantSetName)
	{
		Impl = MakeUnique<Internal::FUsdVariantSetImpl>(InPrim, InVariantSetName);
	}
#endif	  // #if USE_USD_SDK

	bool FUsdVariantSet::AddVariant(const FString& VariantName, EUsdListPosition Position)
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;

		pxr::UsdVariantSet UsdVariantSet = Impl->GetInner();
		if (UsdVariantSet.IsValid())
		{
			const std::string UsdVariantName(TCHAR_TO_UTF8(*VariantName));

			return UsdVariantSet.AddVariant(UsdVariantName, static_cast<pxr::UsdListPosition>(Position));
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	TArray<FString> FUsdVariantSet::GetVariantNames() const
	{
		TArray<FString> VariantNames;

#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;

		pxr::UsdVariantSet UsdVariantSet = Impl->GetInner();
		if (UsdVariantSet.IsValid())
		{
			const std::vector<std::string> UsdVariantNames = UsdVariantSet.GetVariantNames();
			if (UsdVariantNames.empty())
			{
				return VariantNames;
			}

			VariantNames.Reserve(UsdVariantNames.size());

			for (const std::string& UsdVariantName : UsdVariantNames)
			{
				VariantNames.Emplace(UTF8_TO_TCHAR(UsdVariantName.c_str()));
			}
		}
#endif	  // #if USE_USD_SDK

		return VariantNames;
	}

	bool FUsdVariantSet::HasAuthoredVariant(const FString& VariantName) const
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;

		pxr::UsdVariantSet UsdVariantSet = Impl->GetInner();
		if (UsdVariantSet.IsValid())
		{
			const std::string UsdVariantName(TCHAR_TO_UTF8(*VariantName));

			return UsdVariantSet.HasAuthoredVariant(UsdVariantName);
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	FString FUsdVariantSet::GetVariantSelection() const
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;

		pxr::UsdVariantSet UsdVariantSet = Impl->GetInner();
		if (UsdVariantSet.IsValid())
		{
			const std::string UsdVariantSelection = UsdVariantSet.GetVariantSelection();

			return FString(UTF8_TO_TCHAR(UsdVariantSelection.c_str()));
		}
#endif	  // #if USE_USD_SDK

		return FString{};
	}

	bool FUsdVariantSet::HasAuthoredVariantSelection(FString* Value) const
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;

		pxr::UsdVariantSet UsdVariantSet = Impl->GetInner();
		if (UsdVariantSet.IsValid())
		{
			std::string UsdValue;
			bool bResult = UsdVariantSet.HasAuthoredVariantSelection(&UsdValue);

			if (Value != nullptr)
			{
				*Value = UTF8_TO_TCHAR(UsdValue.c_str());
			}

			return bResult;
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	bool FUsdVariantSet::SetVariantSelection(const FString& VariantName)
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;

		pxr::UsdVariantSet UsdVariantSet = Impl->GetInner();
		if (UsdVariantSet.IsValid())
		{
			const std::string UsdVariantName(TCHAR_TO_UTF8(*VariantName));

			return UsdVariantSet.SetVariantSelection(UsdVariantName);
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	bool FUsdVariantSet::ClearVariantSelection()
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;

		pxr::UsdVariantSet UsdVariantSet = Impl->GetInner();
		if (UsdVariantSet.IsValid())
		{
			return UsdVariantSet.ClearVariantSelection();
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	bool FUsdVariantSet::BlockVariantSelection()
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;

		pxr::UsdVariantSet UsdVariantSet = Impl->GetInner();
		if (UsdVariantSet.IsValid())
		{
			return UsdVariantSet.BlockVariantSelection();
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	FUsdPrim FUsdVariantSet::GetPrim() const
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;

		pxr::UsdVariantSet UsdVariantSet = Impl->GetInner();
		if (UsdVariantSet.IsValid())
		{
			return FUsdPrim(UsdVariantSet.GetPrim());
		}
#endif	  // #if USE_USD_SDK

		return FUsdPrim{};
	}

	FString FUsdVariantSet::GetName() const
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;

		pxr::UsdVariantSet UsdVariantSet = Impl->GetInner();
		if (UsdVariantSet.IsValid())
		{
			const std::string UsdVariantName = UsdVariantSet.GetName();

			return FString(UTF8_TO_TCHAR(UsdVariantName.c_str()));
		}
#endif	  // #if USE_USD_SDK

		return FString{};
	}

	bool FUsdVariantSet::IsValid() const
	{
#if USE_USD_SDK
		return Impl->GetInner().IsValid();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	// FUsdVariantSets

	FUsdVariantSets::FUsdVariantSets()
	{
		Impl = MakeUnique<Internal::FUsdVariantSetsImpl>();
	}

	FUsdVariantSets::FUsdVariantSets(const FUsdVariantSets& Other)
	{
#if USE_USD_SDK
		Impl = MakeUnique<Internal::FUsdVariantSetsImpl>(Other.Impl->PxrUsdPrim.Get());
#endif	  // #if USE_USD_SDK
	}

	FUsdVariantSets::FUsdVariantSets(FUsdVariantSets&& Other) = default;

	FUsdVariantSets& FUsdVariantSets::operator=(const FUsdVariantSets& Other)
	{
#if USE_USD_SDK
		Impl = MakeUnique<Internal::FUsdVariantSetsImpl>(Other.Impl->PxrUsdPrim.Get());
#endif	  // #if USE_USD_SDK

		return *this;
	}

	FUsdVariantSets& FUsdVariantSets::operator=(FUsdVariantSets&& Other)
	{
		Impl = MoveTemp(Other.Impl);

		return *this;
	}

	FUsdVariantSets::~FUsdVariantSets()
	{
		Impl.Reset();
	}

	FUsdVariantSets::operator bool() const
	{
#if USE_USD_SDK
		return (bool)Impl->PxrUsdPrim.Get();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

#if USE_USD_SDK
	FUsdVariantSets::operator pxr::UsdVariantSets()
	{
		return Impl->GetInner();
	}

	FUsdVariantSets::operator const pxr::UsdVariantSets() const
	{
		return Impl->GetInner();
	}

	// Private constructor accessible by FUsdPrim.
	FUsdVariantSets::FUsdVariantSets(const pxr::UsdPrim& InPrim)
	{
		Impl = MakeUnique<Internal::FUsdVariantSetsImpl>(InPrim);
	}
#endif	  // #if USE_USD_SDK

	FUsdVariantSet FUsdVariantSets::AddVariantSet(const FString& VariantName, EUsdListPosition Position)
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;

		pxr::UsdVariantSets UsdVariantSets = Impl->GetInner();

		const std::string UsdVariantName(TCHAR_TO_UTF8(*VariantName));

		return FUsdVariantSet(UsdVariantSets.AddVariantSet(UsdVariantName, static_cast<pxr::UsdListPosition>(Position)));
#else
		return FUsdVariantSet{};
#endif	  // #if USE_USD_SDK
	}

	TArray<FString> FUsdVariantSets::GetNames() const
	{
		TArray<FString> VariantSetNames;

#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;

		pxr::UsdVariantSets UsdVariantSets = Impl->GetInner();

		const std::vector<std::string> UsdVariantSetNames = UsdVariantSets.GetNames();
		if (UsdVariantSetNames.empty())
		{
			return VariantSetNames;
		}

		VariantSetNames.Reserve(UsdVariantSetNames.size());

		for (const std::string& UsdVariantSetName : UsdVariantSetNames)
		{
			VariantSetNames.Emplace(UTF8_TO_TCHAR(UsdVariantSetName.c_str()));
		}
#endif	  // #if USE_USD_SDK

		return VariantSetNames;
	}

	FUsdVariantSet FUsdVariantSets::GetVariantSet(const FString& VariantSetName) const
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;

		pxr::UsdVariantSets UsdVariantSets = Impl->GetInner();

		const std::string UsdVariantSetName(TCHAR_TO_UTF8(*VariantSetName));

		return FUsdVariantSet(UsdVariantSets.GetVariantSet(UsdVariantSetName));
#else
		return FUsdVariantSet{};
#endif	  // #if USE_USD_SDK
	}

	bool FUsdVariantSets::HasVariantSet(const FString& VariantSetName) const
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;

		pxr::UsdVariantSets UsdVariantSets = Impl->GetInner();

		const std::string UsdVariantSetName(TCHAR_TO_UTF8(*VariantSetName));

		return UsdVariantSets.HasVariantSet(UsdVariantSetName);
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	FString FUsdVariantSets::GetVariantSelection(const FString& VariantSetName) const
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;

		pxr::UsdVariantSets UsdVariantSets = Impl->GetInner();

		const std::string UsdVariantSetName(TCHAR_TO_UTF8(*VariantSetName));

		const std::string UsdVariantSelection = UsdVariantSets.GetVariantSelection(UsdVariantSetName);

		return FString(UTF8_TO_TCHAR(UsdVariantSelection.c_str()));
#else
		return FString{};
#endif	  // #if USE_USD_SDK
	}

	bool FUsdVariantSets::SetSelection(const FString& VariantSetName, const FString& VariantName)
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;

		pxr::UsdVariantSets UsdVariantSets = Impl->GetInner();

		const std::string UsdVariantSetName(TCHAR_TO_UTF8(*VariantSetName));
		const std::string UsdVariantName(TCHAR_TO_UTF8(*VariantName));

		return UsdVariantSets.SetSelection(UsdVariantSetName, UsdVariantName);
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	TMap<FString, FString> FUsdVariantSets::GetAllVariantSelections() const
	{
		TMap<FString, FString> VariantSelections;

#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;

		pxr::UsdVariantSets UsdVariantSets = Impl->GetInner();

		const std::map<std::string, std::string> UsdVariantSelections = UsdVariantSets.GetAllVariantSelections();
		if (UsdVariantSelections.empty())
		{
			return VariantSelections;
		}

		VariantSelections.Reserve(UsdVariantSelections.size());

		for (const std::pair<const std::string, std::string>& UsdVariantSelection : UsdVariantSelections)
		{
			VariantSelections.Emplace(UTF8_TO_TCHAR(UsdVariantSelection.first.c_str()), UTF8_TO_TCHAR(UsdVariantSelection.second.c_str()));
		}
#endif	  // #if USE_USD_SDK

		return VariantSelections;
	}
}	 // namespace UE
