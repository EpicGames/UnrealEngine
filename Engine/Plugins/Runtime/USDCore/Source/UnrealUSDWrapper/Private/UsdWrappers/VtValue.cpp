// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdWrappers/VtValue.h"

#include "UnrealUSDWrapper.h"
#include "USDMemory.h"
#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdAttribute.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdStage.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/base/gf/matrix2f.h"
#include "pxr/base/gf/matrix3f.h"
#include "pxr/base/gf/matrix4f.h"
#include "pxr/base/vt/array.h"
#include "pxr/base/vt/value.h"
#include "pxr/usd/sdf/types.h"
#include "pxr/usd/usd/common.h"
#include "USDIncludesEnd.h"
#endif	  // #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
		class FVtValueImpl
		{
		public:
			FVtValueImpl() = default;

#if USE_USD_SDK

#if ENABLE_USD_DEBUG_PATH
			FString TypeName;
			FString StringifiedValue;
#endif
			explicit FVtValueImpl(const pxr::VtValue& InVtValue)
				: PxrVtValue(InVtValue)
			{
#if ENABLE_USD_DEBUG_PATH
				RefreshDebugTypes();
#endif
			}

			explicit FVtValueImpl(pxr::VtValue&& InVtValue)
				: PxrVtValue(MoveTemp(InVtValue))
			{
#if ENABLE_USD_DEBUG_PATH
				RefreshDebugTypes();
#endif
			}

#if ENABLE_USD_DEBUG_PATH
			void RefreshDebugTypes()
			{
				TypeName = PxrVtValue ? UTF8_TO_TCHAR(PxrVtValue->GetTypeName().c_str()) : TEXT("");
				StringifiedValue = PxrVtValue ? UTF8_TO_TCHAR(pxr::TfStringify(*PxrVtValue).c_str()) : TEXT("");
			}
#endif

			TUsdStore<pxr::VtValue> PxrVtValue;
#endif	  // #if USE_USD_SDK
		};
	}

	FVtValue::FVtValue()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FVtValueImpl>();
	}

	FVtValue::FVtValue(const FVtValue& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FVtValueImpl>(Other.Impl->PxrVtValue.Get());
#endif	  // #if USE_USD_SDK
	}

	FVtValue::FVtValue(FVtValue&& Other) = default;

	FVtValue& FVtValue::operator=(const FVtValue& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FVtValueImpl>(Other.Impl->PxrVtValue.Get());
#endif	  // #if USE_USD_SDK
		return *this;
	}

	FVtValue& FVtValue::operator=(FVtValue&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MoveTemp(Other.Impl);

		return *this;
	}

	FVtValue::~FVtValue()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	bool FVtValue::operator==(const FVtValue& Other) const
	{
#if USE_USD_SDK
		return Impl->PxrVtValue.Get() == Other.Impl->PxrVtValue.Get();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FVtValue::operator!=(const FVtValue& Other) const
	{
		return !(*this == Other);
	}

#if USE_USD_SDK
	FVtValue::FVtValue(const pxr::VtValue& InVtValue)
		: Impl(MakeUnique<Internal::FVtValueImpl>(InVtValue))
	{
	}

	FVtValue::FVtValue(pxr::VtValue&& InVtValue)
		: Impl(MakeUnique<Internal::FVtValueImpl>(MoveTemp(InVtValue)))
	{
	}

	FVtValue& FVtValue::operator=(const pxr::VtValue& InVtValue)
	{
		Impl = MakeUnique<Internal::FVtValueImpl>(InVtValue);
		return *this;
	}

	FVtValue& FVtValue::operator=(pxr::VtValue&& InVtValue)
	{
		Impl = MakeUnique<Internal::FVtValueImpl>(MoveTemp(InVtValue));
		return *this;
	}

	pxr::VtValue& FVtValue::GetUsdValue()
	{
		return Impl->PxrVtValue.Get();
	}

	const pxr::VtValue& FVtValue::GetUsdValue() const
	{
		return Impl->PxrVtValue.Get();
	}
#endif	  // #if USE_USD_SDK

	FString FVtValue::GetTypeName() const
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;

		return FString(UTF8_TO_TCHAR(Impl->PxrVtValue.Get().GetTypeName().c_str()));
#else
		return FString();
#endif	  // #if USE_USD_SDK
	}

	bool FVtValue::IsArrayValued() const
	{
#if USE_USD_SDK
		return Impl->PxrVtValue.Get().IsArrayValued();
#else
		return true;
#endif	  // #if USE_USD_SDK
	}

#if USE_USD_SDK
	// Utility to help map from a UE type to an USD type within the Get<T>/Set<T> functions
	template<typename T>
	struct USDTypeHelper
	{
	};

	template<>
	struct USDTypeHelper<FFloat16>
	{
		using USDType = pxr::GfHalf;
	};

	template<>
	struct USDTypeHelper<float>
	{
		using USDType = float;
	};

	template<>
	struct USDTypeHelper<double>
	{
		using USDType = double;
	};

	template<>
	struct USDTypeHelper<FSdfTimeCode>
	{
		using USDType = pxr::SdfTimeCode;
	};

	template<>
	struct USDTypeHelper<FQuat4d>
	{
		using USDType = pxr::GfQuatd;
		using VecType = pxr::GfVec3d;
	};

	template<>
	struct USDTypeHelper<FQuat4f>
	{
		using USDType = pxr::GfQuatf;
		using VecType = pxr::GfVec3f;
	};

	template<>
	struct USDTypeHelper<FQuat4h>
	{
		using USDType = pxr::GfQuath;
		using VecType = pxr::GfVec3h;
	};

	template<>
	struct USDTypeHelper<FVector2d>
	{
		using USDType = pxr::GfVec2d;
	};

	template<>
	struct USDTypeHelper<FVector2f>
	{
		using USDType = pxr::GfVec2f;
	};

	template<>
	struct USDTypeHelper<FVector2DHalf>
	{
		using USDType = pxr::GfVec2h;
	};

	template<>
	struct USDTypeHelper<FIntPoint>
	{
		using USDType = pxr::GfVec2i;
	};

	template<>
	struct USDTypeHelper<FVector3d>
	{
		using USDType = pxr::GfVec3d;
	};

	template<>
	struct USDTypeHelper<FVector3f>
	{
		using USDType = pxr::GfVec3f;
	};

	template<>
	struct USDTypeHelper<FVector3h>
	{
		using USDType = pxr::GfVec3h;
	};

	template<>
	struct USDTypeHelper<FIntVector>
	{
		using USDType = pxr::GfVec3i;
	};

	template<>
	struct USDTypeHelper<FVector4d>
	{
		using USDType = pxr::GfVec4d;
	};

	template<>
	struct USDTypeHelper<FVector4f>
	{
		using USDType = pxr::GfVec4f;
	};

	template<>
	struct USDTypeHelper<FVector4h>
	{
		using USDType = pxr::GfVec4h;
	};

	template<>
	struct USDTypeHelper<FIntRect>
	{
		using USDType = pxr::GfVec4i;
	};

	template<>
	struct USDTypeHelper<FMatrix44d>
	{
		using USDType = pxr::GfMatrix4d;
	};

	template<>
	struct USDTypeHelper<FMatrix44f>
	{
		using USDType = pxr::GfMatrix4f;
	};

	template<>
	struct USDTypeHelper<FLinearColor>
	{
		using USDType = pxr::GfVec4f;
	};

	template<>
	struct USDTypeHelper<FColor>
	{
		using USDType = pxr::GfVec4f;
	};
#endif	  // USE_USD_SDK

	template<typename T>
	T FVtValue::Get() const
	{
		T Result{};

#if USE_USD_SDK
		FScopedUsdAllocs Allocs;

		const pxr::VtValue& VtValue = GetUsdValue();

		if constexpr (std::is_same_v<T, int64>)
		{
			// Note: The Unreal int64 and uint64 are typedefs of "long long" and "unsigned long long".
			// On Windows / MSVC these end up with a type that matches int64_t and uint64_t. On Linux / Clang this is not the case.
			//
			// USD only has templates specialized for int64_t/uint64_t though, so if we use our typedef'd "long long"s here we will get
			// a compile error on Linux / Clang, so we have to go through the other type.
			Result = static_cast<T>(VtValue.Get<int64_t>());
		}
		else if constexpr (std::is_same_v<T, uint64>)
		{
			// See comment on the int64 case right above this
			Result = static_cast<T>(VtValue.Get<uint64_t>());
		}
		else if constexpr (std::is_same_v<T, FFloat16>)
		{
			if (VtValue.IsHolding<pxr::GfHalf>())
			{
				Result = FFloat16(VtValue.UncheckedGet<pxr::GfHalf>());
			}
		}
		else if constexpr (std::is_same_v<T, FSdfTimeCode>)
		{
			if (VtValue.IsHolding<pxr::SdfTimeCode>())
			{
				const pxr::SdfTimeCode& TimeCodeValue = VtValue.Get<pxr::SdfTimeCode>();
				Result = static_cast<T>(TimeCodeValue.GetValue());
			}
		}
		else if constexpr (std::is_same_v<T, FString>)
		{
			if (VtValue.IsHolding<std::string>())
			{
				const std::string& String = VtValue.UncheckedGet<std::string>();
				Result = UTF8_TO_TCHAR(String.c_str());
			}
			else
			{
				// As a convenience, let's use USD to stringify any attribute to a string, so we can do Get<FString> on anything
				std::string UsdStr = pxr::TfStringify(VtValue);
				Result = UTF8_TO_TCHAR(UsdStr.c_str());
			}
		}
		else if constexpr (std::is_same_v<T, FName>)
		{
			if (VtValue.IsHolding<pxr::TfToken>())
			{
				const pxr::TfToken& Token = VtValue.UncheckedGet<pxr::TfToken>();
				Result = UTF8_TO_TCHAR(Token.GetString().c_str());
			}
		}
		else if constexpr (std::is_same_v<T, FSdfAssetPath>)
		{
			if (VtValue.IsHolding<pxr::SdfAssetPath>())
			{
				const pxr::SdfAssetPath& AssetPath = VtValue.UncheckedGet<pxr::SdfAssetPath>();
				Result = FSdfAssetPath{
					UTF8_TO_TCHAR(AssetPath.GetAssetPath().c_str()),
					UTF8_TO_TCHAR(AssetPath.GetResolvedPath().c_str())
				};
			}
		}
		else if constexpr (std::is_same_v<T, FMatrix2D>)
		{
			if (VtValue.IsHolding<pxr::GfMatrix2d>())
			{
				const pxr::GfMatrix2d& Matrix = VtValue.UncheckedGet<pxr::GfMatrix2d>();

				Result.Row0.X = Matrix[0][0];
				Result.Row0.Y = Matrix[0][1];

				Result.Row1.X = Matrix[1][0];
				Result.Row1.Y = Matrix[1][1];
			}
		}
		else if constexpr (std::is_same_v<T, FMatrix3D>)
		{
			if (VtValue.IsHolding<pxr::GfMatrix3d>())
			{
				const pxr::GfMatrix3d& Matrix = VtValue.UncheckedGet<pxr::GfMatrix3d>();

				Result.Row0.X = Matrix[0][0];
				Result.Row0.Y = Matrix[0][1];
				Result.Row0.Z = Matrix[0][2];

				Result.Row1.X = Matrix[1][0];
				Result.Row1.Y = Matrix[1][1];
				Result.Row1.Z = Matrix[1][2];

				Result.Row2.X = Matrix[2][0];
				Result.Row2.Y = Matrix[2][1];
				Result.Row2.Z = Matrix[2][2];
			}
		}
		else if constexpr (std::is_same_v<T, FMatrix44d> || std::is_same_v<T, FMatrix44f>)
		{
			using USDType = typename USDTypeHelper<T>::USDType;

			if (VtValue.IsHolding<USDType>())
			{
				const USDType& Matrix = VtValue.UncheckedGet<USDType>();
				Result = T(
					Math::TPlane<typename T::FReal>(Matrix[0][0], Matrix[0][1], Matrix[0][2], Matrix[0][3]),
					Math::TPlane<typename T::FReal>(Matrix[1][0], Matrix[1][1], Matrix[1][2], Matrix[1][3]),
					Math::TPlane<typename T::FReal>(Matrix[2][0], Matrix[2][1], Matrix[2][2], Matrix[2][3]),
					Math::TPlane<typename T::FReal>(Matrix[3][0], Matrix[3][1], Matrix[3][2], Matrix[3][3])
				);
			}
		}
		else if constexpr (std::is_same_v<T, FQuat4d> || std::is_same_v<T, FQuat4f> || std::is_same_v<T, FQuat4h>)
		{
			using USDType = typename USDTypeHelper<T>::USDType;

			if (VtValue.IsHolding<USDType>())
			{
				const USDType& UsdValue = VtValue.UncheckedGet<USDType>();

				const auto& Imaginary = UsdValue.GetImaginary();
				Result = T{Imaginary[0], Imaginary[1], Imaginary[2], UsdValue.GetReal()};
			}
		}
		else if constexpr (std::is_same_v<T, FVector2d> ||		  //
						   std::is_same_v<T, FVector2f> ||		  //
						   std::is_same_v<T, FVector2DHalf> ||	  //
						   std::is_same_v<T, FIntPoint>)
		{
			using USDType = typename USDTypeHelper<T>::USDType;

			if (VtValue.IsHolding<USDType>())
			{
				const USDType& UsdValue = VtValue.UncheckedGet<USDType>();
				Result = T{UsdValue[0], UsdValue[1]};
			}
		}
		else if constexpr (std::is_same_v<T, FVector3d> ||	  //
						   std::is_same_v<T, FVector3f> ||	  //
						   std::is_same_v<T, FVector3h> ||	  //
						   std::is_same_v<T, FIntVector>)
		{
			using USDType = typename USDTypeHelper<T>::USDType;

			if (VtValue.IsHolding<USDType>())
			{
				const USDType& UsdValue = VtValue.UncheckedGet<USDType>();
				Result = T{UsdValue[0], UsdValue[1], UsdValue[2]};
			}
		}
		else if constexpr (std::is_same_v<T, FVector4d> ||	  //
						   std::is_same_v<T, FVector4f> ||	  //
						   std::is_same_v<T, FVector4h> ||	  //
						   std::is_same_v<T, FIntRect> ||	  //
						   std::is_same_v<T, FLinearColor>)
		{
			using USDType = typename USDTypeHelper<T>::USDType;

			if (VtValue.IsHolding<USDType>())
			{
				const USDType& UsdValue = VtValue.UncheckedGet<USDType>();
				Result = T{UsdValue[0], UsdValue[1], UsdValue[2], UsdValue[3]};
			}
		}
		else if constexpr (std::is_same_v<T, FColor>)
		{
			using USDType = typename USDTypeHelper<T>::USDType;

			if (VtValue.IsHolding<USDType>())
			{
				const USDType& UsdValue = VtValue.UncheckedGet<USDType>();

				const bool bSRGB = true;
				Result = FLinearColor{UsdValue[0], UsdValue[1], UsdValue[2], UsdValue[3]}.ToFColor(bSRGB);
			}
		}
		// This allows us to handle nested metadata dictionaries: In that case the VtValue will hold a std::map
		else if constexpr (std::is_same_v<T, TMap<FString, UE::FVtValue>>)
		{
			// This should be a std::map<std::string, pxr::VtValue>
			if (VtValue.IsHolding<pxr::VtDictionary>())
			{
				const pxr::VtDictionary& UsdValue = VtValue.UncheckedGet<pxr::VtDictionary>();

				Result.Empty(UsdValue.size());
				pxr::VtDictionary::const_iterator Iter = UsdValue.begin();
				for (; Iter != UsdValue.end(); ++Iter)
				{
					Result.Add(UTF8_TO_TCHAR(Iter->first.c_str()), UE::FVtValue{std::move(Iter->second)});
				}
			}
		}
		// See comments on the non-array case
		else if constexpr (std::is_same_v<T, TArray<int64>>)
		{
			if (VtValue.IsHolding<pxr::VtArray<int64_t>>())
			{
				const pxr::VtArray<int64_t>& UsdArray = VtValue.UncheckedGet<pxr::VtArray<int64_t>>();

				Result.SetNumUninitialized(UsdArray.size());

				static_assert(sizeof(int64) == sizeof(int64_t));
				FMemory::Memcpy(Result.GetData(), UsdArray.cdata(), Result.NumBytes());
			}
		}
		// See comments on the non-array case
		else if constexpr (std::is_same_v<T, TArray<uint64>>)
		{
			if (VtValue.IsHolding<pxr::VtArray<uint64_t>>())
			{
				const pxr::VtArray<uint64_t>& UsdArray = VtValue.UncheckedGet<pxr::VtArray<uint64_t>>();

				Result.SetNumUninitialized(UsdArray.size());

				static_assert(sizeof(uint64) == sizeof(uint64_t));
				FMemory::Memcpy(Result.GetData(), UsdArray.cdata(), Result.NumBytes());
			}
		}
		else if constexpr (std::is_same_v<T, TArray<FFloat16>>)
		{
			if (VtValue.IsHolding<pxr::VtArray<pxr::GfHalf>>())
			{
				const pxr::VtArray<pxr::GfHalf>& UsdArray = VtValue.UncheckedGet<pxr::VtArray<pxr::GfHalf>>();

				Result.SetNumUninitialized(UsdArray.size());
				for (int32 Index = 0; Index < UsdArray.size(); ++Index)
				{
					Result[Index] = FFloat16{UsdArray[Index]};
				}
			}
		}
		else if constexpr (std::is_same_v<T, TArray<FSdfTimeCode>>)
		{
			if (VtValue.IsHolding<pxr::VtArray<pxr::SdfTimeCode>>())
			{
				const pxr::VtArray<pxr::SdfTimeCode>& UsdArray = VtValue.UncheckedGet<pxr::VtArray<pxr::SdfTimeCode>>();

				Result.SetNumUninitialized(UsdArray.size());
				for (int32 Index = 0; Index < UsdArray.size(); ++Index)
				{
					Result[Index] = UsdArray[Index].GetValue();
				}
			}
		}
		else if constexpr (std::is_same_v<T, TArray<FString>>)
		{
			if (VtValue.IsHolding<pxr::VtArray<std::string>>())
			{
				const pxr::VtArray<std::string>& UsdArray = VtValue.UncheckedGet<pxr::VtArray<std::string>>();

				Result.SetNum(UsdArray.size());
				for (int32 Index = 0; Index < UsdArray.size(); ++Index)
				{
					Result[Index] = UTF8_TO_TCHAR(UsdArray[Index].c_str());
				}
			}
		}
		else if constexpr (std::is_same_v<T, TArray<FName>>)
		{
			if (VtValue.IsHolding<pxr::VtArray<pxr::TfToken>>())
			{
				const pxr::VtArray<pxr::TfToken>& UsdArray = VtValue.UncheckedGet<pxr::VtArray<pxr::TfToken>>();

				Result.SetNum(UsdArray.size());
				for (int32 Index = 0; Index < UsdArray.size(); ++Index)
				{
					Result[Index] = UTF8_TO_TCHAR(UsdArray[Index].GetString().c_str());
				}
			}
		}
		else if constexpr (std::is_same_v<T, TArray<FSdfAssetPath>>)
		{
			if (VtValue.IsHolding<pxr::VtArray<pxr::SdfAssetPath>>())
			{
				const pxr::VtArray<pxr::SdfAssetPath>& UsdArray = VtValue.UncheckedGet<pxr::VtArray<pxr::SdfAssetPath>>();

				Result.SetNum(UsdArray.size());
				for (int32 Index = 0; Index < UsdArray.size(); ++Index)
				{
					Result[Index] = FSdfAssetPath{
						UTF8_TO_TCHAR(UsdArray[Index].GetAssetPath().c_str()),
						UTF8_TO_TCHAR(UsdArray[Index].GetResolvedPath().c_str())
					};
				}
			}
		}
		else if constexpr (std::is_same_v<T, TArray<FMatrix2D>>)
		{
			if (VtValue.IsHolding<pxr::VtArray<pxr::GfMatrix2d>>())
			{
				const pxr::VtArray<pxr::GfMatrix2d>& UsdArray = VtValue.UncheckedGet<pxr::VtArray<pxr::GfMatrix2d>>();

				Result.SetNum(UsdArray.size());
				for (int32 Index = 0; Index < UsdArray.size(); ++Index)
				{
					const pxr::GfMatrix2d& UsdElement = UsdArray[Index];
					FMatrix2D& UEElement = Result[Index];

					UEElement.Row0.X = UsdElement[0][0];
					UEElement.Row0.Y = UsdElement[0][1];

					UEElement.Row1.X = UsdElement[1][0];
					UEElement.Row1.Y = UsdElement[1][1];
				}
			}
		}
		else if constexpr (std::is_same_v<T, TArray<FMatrix3D>>)
		{
			if (VtValue.IsHolding<pxr::VtArray<pxr::GfMatrix3d>>())
			{
				const pxr::VtArray<pxr::GfMatrix3d>& UsdArray = VtValue.UncheckedGet<pxr::VtArray<pxr::GfMatrix3d>>();

				Result.SetNum(UsdArray.size());
				for (int32 Index = 0; Index < UsdArray.size(); ++Index)
				{
					const pxr::GfMatrix3d& UsdElement = UsdArray[Index];
					FMatrix3D& UEElement = Result[Index];

					UEElement.Row0.X = UsdElement[0][0];
					UEElement.Row0.Y = UsdElement[0][1];
					UEElement.Row0.Z = UsdElement[0][2];

					UEElement.Row1.X = UsdElement[1][0];
					UEElement.Row1.Y = UsdElement[1][1];
					UEElement.Row1.Z = UsdElement[1][2];

					UEElement.Row2.X = UsdElement[2][0];
					UEElement.Row2.Y = UsdElement[2][1];
					UEElement.Row2.Z = UsdElement[2][2];
				}
			}
		}
		else if constexpr (std::is_same_v<T, TArray<FMatrix44d>> || std::is_same_v<T, TArray<FMatrix44f>>)
		{
			using UEElementType = typename T::ElementType;
			using UERealType = typename UEElementType::FReal;
			using USDElementType = typename USDTypeHelper<UEElementType>::USDType;
			using USDArrayType = pxr::VtArray<USDElementType>;

			if (VtValue.IsHolding<USDArrayType>())
			{
				const USDArrayType& UsdArray = VtValue.UncheckedGet<USDArrayType>();

				Result.SetNum(UsdArray.size());
				for (int32 Index = 0; Index < UsdArray.size(); ++Index)
				{
					const USDElementType& UsdElement = UsdArray[Index];

					Result[Index] = UEElementType(
						Math::TPlane<UERealType>(UsdElement[0][0], UsdElement[0][1], UsdElement[0][2], UsdElement[0][3]),
						Math::TPlane<UERealType>(UsdElement[1][0], UsdElement[1][1], UsdElement[1][2], UsdElement[1][3]),
						Math::TPlane<UERealType>(UsdElement[2][0], UsdElement[2][1], UsdElement[2][2], UsdElement[2][3]),
						Math::TPlane<UERealType>(UsdElement[3][0], UsdElement[3][1], UsdElement[3][2], UsdElement[3][3])
					);
				}
			}
		}
		else if constexpr (std::is_same_v<T, TArray<FQuat4d>> ||	//
						   std::is_same_v<T, TArray<FQuat4f>> ||	//
						   std::is_same_v<T, TArray<FQuat4h>>)
		{
			using UEElementType = typename T::ElementType;
			using USDElementType = typename USDTypeHelper<UEElementType>::USDType;
			using USDArrayType = pxr::VtArray<USDElementType>;

			if (VtValue.IsHolding<USDArrayType>())
			{
				const USDArrayType& UsdArray = VtValue.UncheckedGet<USDArrayType>();

				Result.SetNumUninitialized(UsdArray.size());
				for (int32 Index = 0; Index < UsdArray.size(); ++Index)
				{
					const USDElementType& USDElement = UsdArray[Index];

					const auto& Imaginary = USDElement.GetImaginary();
					Result[Index] = typename T::ElementType{Imaginary[0], Imaginary[1], Imaginary[2], USDElement.GetReal()};
				}
			}
		}
		else if constexpr (std::is_same_v<T, TArray<FVector2d>> ||		  //
						   std::is_same_v<T, TArray<FVector2f>> ||		  //
						   std::is_same_v<T, TArray<FVector2DHalf>> ||	  //
						   std::is_same_v<T, TArray<FIntPoint>>)
		{
			using UEElementType = typename T::ElementType;
			using USDElementType = typename USDTypeHelper<UEElementType>::USDType;
			using USDArrayType = pxr::VtArray<USDElementType>;

			if (VtValue.IsHolding<USDArrayType>())
			{
				const USDArrayType& UsdArray = VtValue.UncheckedGet<USDArrayType>();

				Result.SetNumUninitialized(UsdArray.size());
				for (int32 Index = 0; Index < UsdArray.size(); ++Index)
				{
					const USDElementType& USDElement = UsdArray[Index];
					Result[Index] = typename T::ElementType{USDElement[0], USDElement[1]};
				}
			}
		}
		else if constexpr (std::is_same_v<T, TArray<FVector3d>> ||	  //
						   std::is_same_v<T, TArray<FVector3f>> ||	  //
						   std::is_same_v<T, TArray<FVector3h>> ||	  //
						   std::is_same_v<T, TArray<FIntVector>>)
		{
			using UEElementType = typename T::ElementType;
			using USDElementType = typename USDTypeHelper<UEElementType>::USDType;
			using USDArrayType = pxr::VtArray<USDElementType>;

			if (VtValue.IsHolding<USDArrayType>())
			{
				const USDArrayType& UsdArray = VtValue.UncheckedGet<USDArrayType>();

				Result.SetNumUninitialized(UsdArray.size());
				for (int32 Index = 0; Index < UsdArray.size(); ++Index)
				{
					const USDElementType& USDElement = UsdArray[Index];
					Result[Index] = typename T::ElementType{USDElement[0], USDElement[1], USDElement[2]};
				}
			}
		}
		else if constexpr (std::is_same_v<T, TArray<FVector4d>> ||	  //
						   std::is_same_v<T, TArray<FVector4f>> ||	  //
						   std::is_same_v<T, TArray<FVector4h>> ||	  //
						   std::is_same_v<T, TArray<FIntRect>> ||	  //
						   std::is_same_v<T, TArray<FLinearColor>>)
		{
			using UEElementType = typename T::ElementType;
			using USDElementType = typename USDTypeHelper<UEElementType>::USDType;
			using USDArrayType = pxr::VtArray<USDElementType>;

			if (VtValue.IsHolding<USDArrayType>())
			{
				const USDArrayType& UsdArray = VtValue.UncheckedGet<USDArrayType>();

				Result.SetNumUninitialized(UsdArray.size());
				for (int32 Index = 0; Index < UsdArray.size(); ++Index)
				{
					const USDElementType& USDElement = UsdArray[Index];
					Result[Index] = typename T::ElementType{USDElement[0], USDElement[1], USDElement[2], USDElement[3]};
				}
			}
		}
		else if constexpr (std::is_same_v<T, TArray<FColor>>)
		{
			using UEElementType = typename T::ElementType;
			using USDElementType = typename USDTypeHelper<UEElementType>::USDType;
			using USDArrayType = pxr::VtArray<USDElementType>;

			if (VtValue.IsHolding<USDArrayType>())
			{
				const USDArrayType& UsdArray = VtValue.UncheckedGet<USDArrayType>();

				Result.SetNumUninitialized(UsdArray.size());
				for (int32 Index = 0; Index < UsdArray.size(); ++Index)
				{
					const USDElementType& USDElement = UsdArray[Index];

					const bool bSRGB = true;
					Result[Index] = FLinearColor{USDElement[0], USDElement[1], USDElement[2], USDElement[3]}.ToFColor(bSRGB);
				}
			}
		}
		// Fallback array case: Handles simple types like TArray<bool>, TArray<float>, TArray<double>, remaining integer types.
		else if constexpr (TIsTArray<T>::Value)
		{
			using UEElementType = typename T::ElementType;
			using USDArrayType = pxr::VtArray<UEElementType>;

			if (VtValue.IsHolding<USDArrayType>())
			{
				const USDArrayType& UsdArray = VtValue.Get<USDArrayType>();

				Result.SetNumUninitialized(UsdArray.size());
				FMemory::Memcpy(Result.GetData(), UsdArray.cdata(), Result.NumBytes());
			}
		}
		// Fallback case: Handles simple types like bool, float, double, remaining integer types
		else
		{
			Result = VtValue.Get<T>();
		}
#endif	  // #if USE_USD_SDK

		return Result;
	}

	template <typename T>
	void FVtValue::Set(const T& Value)
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;

		pxr::VtValue& VtValue = GetUsdValue();

		if constexpr (std::is_same_v<T, int64>)
		{
			VtValue = static_cast<int64_t>(Value);
		}
		else if constexpr (std::is_same_v<T, uint64>)
		{
			VtValue = static_cast<uint64_t>(Value);
		}
		else if constexpr (std::is_same_v<T, FFloat16>)
		{
			VtValue = pxr::GfHalf{Value.GetFloat()};
		}
		else if constexpr (std::is_same_v<T, FSdfTimeCode>)
		{
			VtValue = pxr::SdfTimeCode{Value.Time};
		}
		else if constexpr (std::is_same_v<T, FString>)
		{
			VtValue = TCHAR_TO_UTF8(*Value);
		}
		else if constexpr (std::is_same_v<T, FName>)
		{
			VtValue = pxr::TfToken{TCHAR_TO_UTF8(*Value.ToString())};
		}
		else if constexpr (std::is_same_v<T, FSdfAssetPath>)
		{
			VtValue = pxr::SdfAssetPath{TCHAR_TO_UTF8(*Value.AssetPath), TCHAR_TO_UTF8(*Value.ResolvedPath)};
		}
		else if constexpr (std::is_same_v<T, FMatrix2D>)
		{
			VtValue = pxr::GfMatrix2d{
				Value.Row0[0], Value.Row0[1],
				Value.Row1[0], Value.Row1[1]
			};
		}
		else if constexpr (std::is_same_v<T, FMatrix3D>)
		{
			VtValue = pxr::GfMatrix3d{
				Value.Row0[0], Value.Row0[1], Value.Row0[2],
				Value.Row1[0], Value.Row1[1], Value.Row1[2],
				Value.Row2[0], Value.Row2[1], Value.Row2[2]
			};
		}
		else if constexpr (std::is_same_v<T, FMatrix44d> || std::is_same_v<T, FMatrix44f>)
		{
			using USDType = typename USDTypeHelper<T>::USDType;

			VtValue = USDType{
				Value.M[0][0], Value.M[0][1], Value.M[0][2], Value.M[0][3],
				Value.M[1][0], Value.M[1][1], Value.M[1][2], Value.M[1][3],
				Value.M[2][0], Value.M[2][1], Value.M[2][2], Value.M[2][3],
				Value.M[3][0], Value.M[3][1], Value.M[3][2], Value.M[3][3]
			};
		}
		else if constexpr (std::is_same_v<T, FQuat4d> || std::is_same_v<T, FQuat4f> || std::is_same_v<T, FQuat4h>)
		{
			using USDType = typename USDTypeHelper<T>::USDType;
			using USDScalarType = typename USDType::ScalarType;

			VtValue = USDType{
				static_cast<USDScalarType>(Value.W),
				static_cast<USDScalarType>(Value.X),
				static_cast<USDScalarType>(Value.Y),
				static_cast<USDScalarType>(Value.Z)
			};
		}
		else if constexpr (std::is_same_v<T, FVector2d> ||		  //
						   std::is_same_v<T, FVector2f> ||		  //
						   std::is_same_v<T, FVector2DHalf> ||	  //
						   std::is_same_v<T, FIntPoint>)
		{
			using USDType = typename USDTypeHelper<T>::USDType;
			using USDScalarType = typename USDType::ScalarType;

			VtValue = USDType{
				static_cast<USDScalarType>(Value.X),
				static_cast<USDScalarType>(Value.Y)
			};
		}
		else if constexpr (std::is_same_v<T, FVector3d> ||	  //
						   std::is_same_v<T, FVector3f> ||	  //
						   std::is_same_v<T, FVector3h> ||	  //
						   std::is_same_v<T, FIntVector>)
		{
			using USDType = typename USDTypeHelper<T>::USDType;
			using USDScalarType = typename USDType::ScalarType;

			VtValue = USDType{
				static_cast<USDScalarType>(Value.X),
				static_cast<USDScalarType>(Value.Y),
				static_cast<USDScalarType>(Value.Z)
			};
		}
		else if constexpr (std::is_same_v<T, FVector4d> ||	  //
						   std::is_same_v<T, FVector4f> ||	  //
						   std::is_same_v<T, FVector4h>)
		{
			using USDType = typename USDTypeHelper<T>::USDType;
			using USDScalarType = typename USDType::ScalarType;

			VtValue = USDType{
				static_cast<USDScalarType>(Value.X),
				static_cast<USDScalarType>(Value.Y),
				static_cast<USDScalarType>(Value.Z),
				static_cast<USDScalarType>(Value.W)
			};
		}
		else if constexpr (std::is_same_v<T, FIntRect>)
		{
			VtValue = pxr::GfVec4i{Value.Min.X, Value.Min.Y, Value.Max.X, Value.Max.Y};
		}
		else if constexpr (std::is_same_v<T, FLinearColor>)
		{
			VtValue = pxr::GfVec4f{Value.R, Value.G, Value.B, Value.A};
		}
		else if constexpr (std::is_same_v<T, FColor>)
		{
			FLinearColor Linear{Value};
			VtValue = pxr::GfVec4f{Linear.R, Linear.G, Linear.B, Linear.A};
		}
		// This allows us to handle nested metadata dictionaries: In that case the VtValue should hold a std::map
		else if constexpr (std::is_same_v<T, TMap<FString, UE::FVtValue>>)
		{
			pxr::VtDictionary UsdValue;
			for (const TPair<FString, UE::FVtValue>& Pair : Value)
			{
				UsdValue.insert({
					TCHAR_TO_UTF8(*Pair.Key),
					Pair.Value.GetUsdValue()
				});
			}

			VtValue = UsdValue;
		}
		else if constexpr (std::is_same_v<T, TArray<int64>>)
		{
			pxr::VtArray<int64_t> UsdArray;
			UsdArray.resize(Value.Num());

			static_assert(sizeof(int64) == sizeof(int64_t));
			FMemory::Memcpy(UsdArray.data(), Value.GetData(), Value.NumBytes());

			VtValue = UsdArray;
		}
		else if constexpr (std::is_same_v<T, TArray<uint64>>)
		{
			pxr::VtArray<uint64_t> UsdArray;
			UsdArray.resize(Value.Num());

			static_assert(sizeof(uint64) == sizeof(uint64_t));
			FMemory::Memcpy(UsdArray.data(), Value.GetData(), Value.NumBytes());

			VtValue = UsdArray;
		}
		else if constexpr (std::is_same_v<T, TArray<FFloat16>>)
		{
			pxr::VtArray<pxr::GfHalf> UsdArray;
			UsdArray.resize(Value.Num());

			for (int32 Index = 0; Index < Value.Num(); ++Index)
			{
				UsdArray[Index] = Value[Index].GetFloat();
			}

			VtValue = UsdArray;
		}
		else if constexpr (std::is_same_v<T, TArray<FSdfTimeCode>>)
		{
			pxr::VtArray<pxr::SdfTimeCode> UsdArray;
			UsdArray.resize(Value.Num());

			for (int32 Index = 0; Index < Value.Num(); ++Index)
			{
				UsdArray[Index] = Value[Index].Time;
			}

			VtValue = UsdArray;
		}
		else if constexpr (std::is_same_v<T, TArray<FString>>)
		{
			pxr::VtArray<std::string> UsdArray;
			UsdArray.resize(Value.Num());

			for (int32 Index = 0; Index < Value.Num(); ++Index)
			{
				UsdArray[Index] = TCHAR_TO_UTF8(*Value[Index]);
			}

			VtValue = UsdArray;
		}
		else if constexpr (std::is_same_v<T, TArray<FName>>)
		{
			pxr::VtArray<pxr::TfToken> UsdArray;
			UsdArray.resize(Value.Num());

			for (int32 Index = 0; Index < Value.Num(); ++Index)
			{
				UsdArray[Index] = pxr::TfToken{TCHAR_TO_UTF8(*Value[Index].ToString())};
			}

			VtValue = UsdArray;
		}
		else if constexpr (std::is_same_v<T, TArray<FSdfAssetPath>>)
		{
			pxr::VtArray<pxr::SdfAssetPath> UsdArray;
			UsdArray.resize(Value.Num());

			for (int32 Index = 0; Index < Value.Num(); ++Index)
			{
				const FSdfAssetPath& Element = Value[Index];
				UsdArray[Index] = pxr::SdfAssetPath{
					TCHAR_TO_UTF8(*Element.AssetPath),
					TCHAR_TO_UTF8(*Element.ResolvedPath)
				};
			}

			VtValue = UsdArray;
		}
		else if constexpr (std::is_same_v<T, TArray<FMatrix2D>>)
		{
			pxr::VtArray<pxr::GfMatrix2d> UsdArray;
			UsdArray.resize(Value.Num());

			for (int32 Index = 0; Index < Value.Num(); ++Index)
			{
				const FMatrix2D& Element = Value[Index];
				UsdArray[Index] = pxr::GfMatrix2d{
					Element.Row0[0], Element.Row0[1],
					Element.Row1[0], Element.Row1[1]
				};
			}

			VtValue = UsdArray;
		}
		else if constexpr (std::is_same_v<T, TArray<FMatrix3D>>)
		{
			pxr::VtArray<pxr::GfMatrix3d> UsdArray;
			UsdArray.resize(Value.Num());

			for (int32 Index = 0; Index < Value.Num(); ++Index)
			{
				const FMatrix3D& Element = Value[Index];
				UsdArray[Index] = pxr::GfMatrix3d{
					Element.Row0[0], Element.Row0[1], Element.Row0[2],
					Element.Row1[0], Element.Row1[1], Element.Row1[2],
					Element.Row2[0], Element.Row2[1], Element.Row2[2]
				};
			}

			VtValue = UsdArray;
		}
		else if constexpr (std::is_same_v<T, TArray<FMatrix44d>> || std::is_same_v<T, TArray<FMatrix44f>>)
		{
			using UEElementType = typename T::ElementType;
			using UERealType = typename UEElementType::FReal;
			using USDElementType = typename USDTypeHelper<UEElementType>::USDType;
			using USDArrayType = pxr::VtArray<USDElementType>;

			USDArrayType UsdArray;
			UsdArray.resize(Value.Num());

			for (int32 Index = 0; Index < Value.Num(); ++Index)
			{
				const UEElementType& Element = Value[Index];
				UsdArray[Index] = USDElementType{
					Element.M[0][0], Element.M[0][1], Element.M[0][2], Element.M[0][3],
					Element.M[1][0], Element.M[1][1], Element.M[1][2], Element.M[1][3],
					Element.M[2][0], Element.M[2][1], Element.M[2][2], Element.M[2][3],
					Element.M[3][0], Element.M[3][1], Element.M[3][2], Element.M[3][3]
				};
			}

			VtValue = UsdArray;
		}
		else if constexpr (std::is_same_v<T, TArray<FQuat4d>> || std::is_same_v<T, TArray<FQuat4f>> || std::is_same_v<T, TArray<FQuat4h>>)
		{
			using UEElementType = typename T::ElementType;
			using USDElementType = typename USDTypeHelper<UEElementType>::USDType;
			using USDScalarType = typename USDElementType::ScalarType;
			using USDArrayType = pxr::VtArray<USDElementType>;

			USDArrayType UsdArray;
			UsdArray.resize(Value.Num());

			for (int32 Index = 0; Index < Value.Num(); ++Index)
			{
				const UEElementType& Element = Value[Index];
				UsdArray[Index] = USDElementType{
					static_cast<USDScalarType>(Element.W),
					static_cast<USDScalarType>(Element.X),
					static_cast<USDScalarType>(Element.Y),
					static_cast<USDScalarType>(Element.Z)
				};
			}

			VtValue = UsdArray;
		}
		else if constexpr (std::is_same_v<T, TArray<FVector2d>> ||		  //
						   std::is_same_v<T, TArray<FVector2f>> ||		  //
						   std::is_same_v<T, TArray<FVector2DHalf>> ||	  //
						   std::is_same_v<T, TArray<FIntPoint>>)
		{
			using UEElementType = typename T::ElementType;
			using USDElementType = typename USDTypeHelper<UEElementType>::USDType;
			using USDScalarType = typename USDElementType::ScalarType;
			using USDArrayType = pxr::VtArray<USDElementType>;

			USDArrayType UsdArray;
			UsdArray.resize(Value.Num());

			for (int32 Index = 0; Index < Value.Num(); ++Index)
			{
				const UEElementType& Element = Value[Index];
				UsdArray[Index] = USDElementType{
					static_cast<USDScalarType>(Element.X), 
					static_cast<USDScalarType>(Element.Y)
				};
			}

			VtValue = UsdArray;
		}
		else if constexpr (std::is_same_v<T, TArray<FVector3d>> ||		  //
						   std::is_same_v<T, TArray<FVector3f>> ||		  //
						   std::is_same_v<T, TArray<FVector3h>> ||	  //
						   std::is_same_v<T, TArray<FIntVector>>)
		{
			using UEElementType = typename T::ElementType;
			using USDElementType = typename USDTypeHelper<UEElementType>::USDType;
			using USDScalarType = typename USDElementType::ScalarType;
			using USDArrayType = pxr::VtArray<USDElementType>;

			USDArrayType UsdArray;
			UsdArray.resize(Value.Num());

			for (int32 Index = 0; Index < Value.Num(); ++Index)
			{
				const UEElementType& Element = Value[Index];
				UsdArray[Index] = USDElementType{
					static_cast<USDScalarType>(Element.X), 
					static_cast<USDScalarType>(Element.Y),
					static_cast<USDScalarType>(Element.Z)
				};
			}

			VtValue = UsdArray;
		}
		else if constexpr (std::is_same_v<T, TArray<FVector4d>> ||	  //
						   std::is_same_v<T, TArray<FVector4f>> ||	  //
						   std::is_same_v<T, TArray<FVector4h>>)
		{
			using UEElementType = typename T::ElementType;
			using USDElementType = typename USDTypeHelper<UEElementType>::USDType;
			using USDScalarType = typename USDElementType::ScalarType;
			using USDArrayType = pxr::VtArray<USDElementType>;

			USDArrayType UsdArray;
			UsdArray.resize(Value.Num());

			for (int32 Index = 0; Index < Value.Num(); ++Index)
			{
				const UEElementType& Element = Value[Index];
				UsdArray[Index] = USDElementType{
					static_cast<USDScalarType>(Element.X),
					static_cast<USDScalarType>(Element.Y),
					static_cast<USDScalarType>(Element.Z),
					static_cast<USDScalarType>(Element.W)
				};
			}

			VtValue = UsdArray;
		}
		else if constexpr (std::is_same_v<T, TArray<FIntRect>>)
		{
			pxr::VtArray<pxr::GfVec4i> UsdArray;
			UsdArray.resize(Value.Num());

			for (int32 Index = 0; Index < Value.Num(); ++Index)
			{
				const FIntRect& Element = Value[Index];
				UsdArray[Index] = pxr::GfVec4i{Element.Min.X, Element.Min.Y, Element.Max.X, Element.Max.Y};
			}

			VtValue = UsdArray;
		}
		else if constexpr (std::is_same_v<T, TArray<FLinearColor>>)
		{
			pxr::VtArray<pxr::GfVec4f> UsdArray;
			UsdArray.resize(Value.Num());

			for (int32 Index = 0; Index < Value.Num(); ++Index)
			{
				const FLinearColor& Element = Value[Index];
				UsdArray[Index] = pxr::GfVec4f{Element.R, Element.G, Element.B, Element.A};
			}

			VtValue = UsdArray;
		}
		else if constexpr (std::is_same_v<T, TArray<FColor>>)
		{
			pxr::VtArray<pxr::GfVec4f> UsdArray;
			UsdArray.resize(Value.Num());

			for (int32 Index = 0; Index < Value.Num(); ++Index)
			{
				const FLinearColor Element{Value[Index]};
				UsdArray[Index] = pxr::GfVec4f{Element.R, Element.G, Element.B, Element.A};
			}

			VtValue = UsdArray;
		}
		// Fallback array case: Handles simple types like TArray<bool>, TArray<float>, TArray<int32>, etc.
		else if constexpr (TIsTArray<T>::Value)
		{
			pxr::VtArray<typename T::ElementType> UsdArray;
			UsdArray.resize(Value.Num());
			FMemory::Memcpy(UsdArray.data(), Value.GetData(), Value.NumBytes());
			VtValue = UsdArray;
		}
		// Fallback single case: Handles simple types like bool, float, int32, etc.
		else
		{
			VtValue = Value;
		}

#if ENABLE_USD_DEBUG_PATH
		Impl->RefreshDebugTypes();
#endif

#endif	  // #if USE_USD_SDK
	}

	bool FVtValue::IsEmpty() const
	{
#if USE_USD_SDK
		return Impl->PxrVtValue.Get().IsEmpty();
#else
		return true;
#endif	  // #if USE_USD_SDK
	}
}	 // namespace UE

#define SPECIALIZE(X) \
	template UNREALUSDWRAPPER_API X UE::FVtValue::Get() const;\
	template UNREALUSDWRAPPER_API void UE::FVtValue::Set(const X&);
SPECIALIZE(bool);
SPECIALIZE(uint8);
SPECIALIZE(int32);
SPECIALIZE(uint32);
SPECIALIZE(int64);
SPECIALIZE(uint64);

SPECIALIZE(FFloat16);
SPECIALIZE(float);
SPECIALIZE(double);
SPECIALIZE(FSdfTimeCode);

SPECIALIZE(FString);
SPECIALIZE(FName);
SPECIALIZE(FSdfAssetPath);

SPECIALIZE(FMatrix2D);
SPECIALIZE(FMatrix3D);
SPECIALIZE(FMatrix44d);
SPECIALIZE(FMatrix44f);

SPECIALIZE(FQuat4d);
SPECIALIZE(FQuat4f);
SPECIALIZE(FQuat4h);

SPECIALIZE(FVector2d);
SPECIALIZE(FVector2f);
SPECIALIZE(FVector2DHalf);
SPECIALIZE(FIntPoint);

SPECIALIZE(FVector3d);
SPECIALIZE(FVector3f);
SPECIALIZE(FVector3h);
SPECIALIZE(FIntVector);

SPECIALIZE(FVector4d);
SPECIALIZE(FVector4f);
SPECIALIZE(FVector4h);
SPECIALIZE(FIntRect);
SPECIALIZE(FLinearColor); // Not strictly a USD basic data type but we support this for convenience / backwards compatibility
SPECIALIZE(FColor);

using FUsdMetadataMap = TMap<FString, UE::FVtValue>;  // Can't have a comma in the SPECIALIZE() argument list or the preprocessor thinks it's two arguments
SPECIALIZE(FUsdMetadataMap);

SPECIALIZE(TArray<bool>);
SPECIALIZE(TArray<uint8>);
SPECIALIZE(TArray<int32>);
SPECIALIZE(TArray<uint32>);
SPECIALIZE(TArray<int64>);
SPECIALIZE(TArray<uint64>);

SPECIALIZE(TArray<FFloat16>);
SPECIALIZE(TArray<float>);
SPECIALIZE(TArray<double>);
SPECIALIZE(TArray<FSdfTimeCode>);

SPECIALIZE(TArray<FString>);
SPECIALIZE(TArray<FName>);
SPECIALIZE(TArray<FSdfAssetPath>);

SPECIALIZE(TArray<FMatrix2D>);
SPECIALIZE(TArray<FMatrix3D>);
SPECIALIZE(TArray<FMatrix44d>);
SPECIALIZE(TArray<FMatrix44f>);

SPECIALIZE(TArray<FQuat4d>);
SPECIALIZE(TArray<FQuat4f>);
SPECIALIZE(TArray<FQuat4h>);

SPECIALIZE(TArray<FVector2d>);
SPECIALIZE(TArray<FVector2f>);
SPECIALIZE(TArray<FVector2DHalf>);
SPECIALIZE(TArray<FIntPoint>);

SPECIALIZE(TArray<FVector3d>);
SPECIALIZE(TArray<FVector3f>);
SPECIALIZE(TArray<FVector3h>);
SPECIALIZE(TArray<FIntVector>);

SPECIALIZE(TArray<FVector4d>);
SPECIALIZE(TArray<FVector4f>);
SPECIALIZE(TArray<FVector4h>);
SPECIALIZE(TArray<FIntRect>);
SPECIALIZE(TArray<FLinearColor>);
SPECIALIZE(TArray<FColor>);
#undef SPECIALIZE
