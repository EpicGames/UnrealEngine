// Copyright Epic Games, Inc. All Rights Reserved.

// HEADER_UNIT_SKIP - Special include

// clang-format off

#if USE_USD_SDK

#ifdef USD_INCLUDES_START

#pragma warning(pop)
#pragma pop_macro("check")

#if PLATFORM_LINUX && PLATFORM_EXCEPTIONS_DISABLED
	#undef try
	#undef catch
	#pragma pop_macro("try")
	#pragma pop_macro("catch")
#endif

// Boost needed _DEBUG defined when /RTCs build flag is enabled (Run Time Checks)
#if PLATFORM_WINDOWS && UE_BUILD_DEBUG
	#ifdef _DEBUG
		#undef _DEBUG
	#endif
#endif

THIRD_PARTY_INCLUDES_END

#undef USD_INCLUDES_START

#else

#if !defined(UE_DIRECT_HEADER_COMPILE)
	#error Mismatched USDIncludesEnd.h detected.
#endif

#endif // USD_INCLUDES_START

#endif // #if USE_USD_SDK

// clang-format on
