// Copyright Epic Games, Inc. All Rights Reserved.

// HEADER_UNIT_SKIP - Special include

// clang-format off

#if USE_USD_SDK

#ifndef USD_INCLUDES_START
	#define USD_INCLUDES_START
#else
	#error Nesting USDIncludesStart.h is not allowed!
#endif

#include "CoreMinimal.h"

THIRD_PARTY_INCLUDES_START

#pragma warning(push)
#pragma warning(disable: 4193) /* #pragma warning(pop): no matching '#pragma warning(push)', the pop is in USDIncludesEnd.h */
#pragma warning(disable: 4003) /* pxr/usd/sdf/fileFormat.h BOOST_PP_SEQ_DETAIL_IS_NOT_EMPTY during static analysis */
#pragma warning(disable: 5033) /* 'register' is no longer a supported storage class */
#pragma warning(disable: 6319)
#pragma warning(disable: 5232) /* Silence some warnings about operator==(const pxr::TfRefPtr<pxr::SdfLayer> &,const pxr::TfWeakPtr<pxr::SdfLayer> &) */
#pragma warning(disable: 4717) /* calling itself recursively within the USD SDK's weakPtrFacade.h that we get when compiling C++20 targets (UE-169305) */

#pragma push_macro("check")
#undef check // Boost function is named 'check' in boost\python\detail\convertible.hpp

// Boost needs _DEBUG defined when /RTCs build flag is enabled (Run Time Checks)
#if PLATFORM_WINDOWS && UE_BUILD_DEBUG
	#ifndef _DEBUG
		#define _DEBUG
	#endif
#endif

#if PLATFORM_WINDOWS
	#include "Windows/WindowsHWrapper.h"
#endif

// Clang emits a compile error if it finds try/catch blocks when exceptions are disabled (MSVC seems fine with it?)
#if PLATFORM_LINUX && PLATFORM_EXCEPTIONS_DISABLED
	#pragma push_macro("try")
	#pragma push_macro("catch")

	// We can define these out, but USD uses 'throw' as well, which is also not allowed with no-exceptions
	#define try
	#define catch(...) if(false)

	// We can't just "#define throw" because USD also uses "throw()" as an exception specification,
	// and that define would generate ill-formed code. The solution chosen here is to intercept and
	// redefine the only usage of 'throw' as a statement in the USD includes, in boost/python/errors.hpp.
	// Note that this will likely fail when the USD version is updated. The fix there would be to remove
	// this snippet and try to disable whatever 'throw' statements the new USD version has in some other way
	#ifndef ERRORS_DWA052500_H_
	#define ERRORS_DWA052500_H_
	#include <boost/python/detail/prefix.hpp>
	#include <boost/function/function0.hpp>
	namespace boost
	{
		namespace python
		{
			struct BOOST_PYTHON_DECL error_already_set
			{
				virtual ~error_already_set();
			};
			BOOST_PYTHON_DECL bool handle_exception_impl( function0<void> );
			BOOST_PYTHON_DECL void throw_error_already_set();
			template <class T> inline T* expect_non_null( T* x ) { return x; }
			BOOST_PYTHON_DECL PyObject* pytype_check( PyTypeObject* pytype, PyObject* source );
		}
	}
	#endif // ERRORS_DWA052500_H_
#endif // PLATFORM_LINUX && PLATFORM_EXCEPTIONS_DISABLED

#include "pxr/pxr.h"

#endif // #if USE_USD_SDK

// clang-format on
