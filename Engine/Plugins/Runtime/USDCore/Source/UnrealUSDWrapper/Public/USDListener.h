// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Containers/Map.h"
#include "Delegates/Delegate.h"

#include "USDSetUtils.h"
#include "UsdWrappers/ForwardDeclarations.h"
#include "UsdWrappers/VtValue.h"

#define UE_API UNREALUSDWRAPPER_API

class FUsdListenerImpl;

namespace UE
{
	class FSdfPath;
}

namespace UsdUtils
{
	/** Analogous to pxr::SdfChangeList::Entry::_Flags */
	struct FPrimChangeFlags
	{
		bool bDidChangeIdentifier : 1;
		bool bDidChangeResolvedPath : 1;
		bool bDidReplaceContent : 1;
		bool bDidReloadContent : 1;
		bool bDidReorderChildren : 1;
		bool bDidReorderProperties : 1;
		bool bDidRename : 1;
		bool bDidChangePrimVariantSets : 1;
		bool bDidChangePrimInheritPaths : 1;
		bool bDidChangePrimSpecializes : 1;
		bool bDidChangePrimReferences : 1;
		bool bDidChangeAttributeTimeSamples : 1;
		bool bDidChangeAttributeConnection : 1;
		bool bDidChangeRelationshipTargets : 1;
		bool bDidAddTarget : 1;
		bool bDidRemoveTarget : 1;
		bool bDidAddInertPrim : 1;
		bool bDidAddNonInertPrim : 1;
		bool bDidRemoveInertPrim : 1;
		bool bDidRemoveNonInertPrim : 1;
		bool bDidAddPropertyWithOnlyRequiredFields : 1;
		bool bDidAddProperty : 1;
		bool bDidRemovePropertyWithOnlyRequiredFields : 1;
		bool bDidRemoveProperty : 1;

		FPrimChangeFlags()
		{
			FMemory::Memset(this, 0, sizeof(*this));
		}
	};

	/**
	 * Analogous to pxr::SdfChangeList::Entry::InfoChange, describes a change to an attribute.
	 * Here we break off PropertyName and Field for simplicity
	 */
	struct FFieldChange
	{
		FString Field;						  // default, variability, timeSamples, etc.
		UE::FVtValue OldValue;				  // Can be empty when we create a new attribute opinion
		UE::FVtValue NewValue;				  // Can be empty when we clear an existing attribute opinion
	};
	using FAttributeChange = FFieldChange;	  // Renamed in 5.6 as this can refer to a relationship, or even prim metadata

	/** Analogous to pxr::SdfChangeList::SubLayerChangeType, describes a change to a sublayer */
	enum ESubLayerChangeType
	{
		SubLayerAdded,
		SubLayerRemoved,
		SubLayerOffset
	};

	/** Analogous to pxr::SdfChangeList::Entry, describes a generic change to an object */
	struct FSdfChangeListEntry
	{
		TArray<FFieldChange> FieldChanges;
		FPrimChangeFlags Flags;
		FString OldPath;		  // Empty if Flags.bDidRename is not set
		FString OldIdentifier;	  // Empty if Flags.bDIdChangeIdentifier is not set
		TArray<TPair<FString, ESubLayerChangeType>> SubLayerChanges;
	};

	using FObjectChangeNotice = FSdfChangeListEntry;	// Renamed in 5.3 as it is used for layer changes now too

	using FSdfChangeList = TArray<TPair<UE::FSdfPath, FSdfChangeListEntry>>;
	using FLayerToSdfChangeList = TArray<TPair<UE::FSdfLayerWeak, FSdfChangeList>>;

	/**
	 * Describes USD object changes by object path (an object can be a prim, a property, etc.)
	 */
	using FObjectChangesByPath = TCaseSensitiveStringMap<TArray<FSdfChangeListEntry>>;
}	 // namespace UsdUtils

/**
 * Registers to Usd Notices and emits events when the Usd Stage has changed
 */
class FUsdListener
{
public:
	UE_API FUsdListener();
	UE_API FUsdListener(const UE::FUsdStage& Stage);

	FUsdListener(const FUsdListener& Other) = delete;
	FUsdListener(FUsdListener&& Other) = delete;

	UE_API virtual ~FUsdListener();

	FUsdListener& operator=(const FUsdListener& Other) = delete;
	FUsdListener& operator=(FUsdListener&& Other) = delete;

	UE_API void Register(const UE::FUsdStage& Stage);

	// Increment/decrement the block counter
	UE_API void Block();
	UE_API void Unblock();
	UE_API bool IsBlocked() const;

	// Stage-specific events
	DECLARE_TS_MULTICAST_DELEGATE(FOnStageEditTargetChanged);
	UE_API FOnStageEditTargetChanged& GetOnStageEditTargetChanged();

	DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnSdfLayersChanged, const UsdUtils::FLayerToSdfChangeList&);
	UE_API FOnSdfLayersChanged& GetOnSdfLayersChanged();

	DECLARE_TS_MULTICAST_DELEGATE(FOnSdfLayerDirtinessChanged);
	UE_API FOnSdfLayerDirtinessChanged& GetOnSdfLayerDirtinessChanged();

	DECLARE_TS_MULTICAST_DELEGATE_TwoParams(
		FOnObjectsChanged,
		const UsdUtils::FObjectChangesByPath& /* InfoChanges */,
		const UsdUtils::FObjectChangesByPath& /* ResyncChanges */
	);
	UE_API FOnObjectsChanged& GetOnObjectsChanged();

private:
	TUniquePtr<FUsdListenerImpl> Impl;
};

class FScopedBlockNotices final
{
public:
	UE_API explicit FScopedBlockNotices(FUsdListener& InListener);
	UE_API ~FScopedBlockNotices();

	FScopedBlockNotices() = delete;
	FScopedBlockNotices(const FScopedBlockNotices&) = delete;
	FScopedBlockNotices(FScopedBlockNotices&&) = delete;
	FScopedBlockNotices& operator=(const FScopedBlockNotices&) = delete;
	FScopedBlockNotices& operator=(FScopedBlockNotices&&) = delete;

private:
	FUsdListener& Listener;
};

#undef UE_API
