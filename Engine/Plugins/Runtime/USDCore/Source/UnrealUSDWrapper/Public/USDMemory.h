// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/Platform.h"
#include "Misc/Optional.h"
#include "Misc/ScopeRWLock.h"
#include "Modules/Boilerplate/ModuleBoilerplate.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

#define UE_API UNREALUSDWRAPPER_API

LLM_DECLARE_TAG_API(Usd, UNREALUSDWRAPPER_API);

// Default to 0 in case the USD SDK is not used
// Also, on Linux/Mac, the USD SDK doesn't need the memory overloads to support UE allocators so the USD memory manager is not needed
#define USE_USD_MEMORY_MANAGER 0

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/pxr.h"

// See UnrealUSDWrapper.build.cs for the definition of this macro
#if ENABLE_USD_MEMORY_OVERLOADS

#ifndef PXR_RUNTIME_MEMORY_OVERLOADS_ENABLED
// The USD SDK was compiled without the support for UE memory overloads so the USD memory manager has to be enabled
#undef USE_USD_MEMORY_MANAGER
#define USE_USD_MEMORY_MANAGER 1
COMPILE_WARNING("The USD SDK was compiled without the memory overloads enabled. UE will use its USD Memory Manager instead.")
#endif // PXR_RUNTIME_MEMORY_OVERLOADS_ENABLED

#else

// USD memory overloads not enabled so the USD memory manager has to be enabled on Windows
#if PLATFORM_WINDOWS
#undef USE_USD_MEMORY_MANAGER
#define USE_USD_MEMORY_MANAGER 1

// On Windows the USD SDK is compiled with PXR_ENABLE_RUNTIME_MEMORY_OVERLOADS=ON, so this is just a sanity check
#ifdef PXR_RUNTIME_MEMORY_OVERLOADS_ENABLED
COMPILE_WARNING("The USD SDK was compiled with the memory overloads enabled but the UnrealUSDWrapper module was not. Make sure the memory overloads match between UE and the USD SDK.")
#endif // PXR_RUNTIME_MEMORY_OVERLOADS_ENABLED

#endif // PLATFORM_WINDOWS

#endif // ENABLE_USD_MEMORY_OVERLOADS

#include "USDIncludesEnd.h"
#endif	  // #if USE_USD_SDK

/**
 * This file is used to handle memory allocation issues when interacting with the USD SDK.
 *
 * Modules looking to use the USD SDK and the memory tools provided here need to use IMPLEMENT_MODULE_USD instead of the regular IMPLEMENT_MODULE.
 * The USD memory tools are only supported in non monolithic builds at this time, since it requires overriding new and delete per module.
 *
 * The USD SDK uses the shared C runtime allocator. This means that objects returned by the USD SDK might try to delete objects that were allocated
 * through the CRT. Since UE overrides new and delete per module, USD objects that try to call delete will end up freeing memory with the UE allocator
 * but the malloc was made using the CRT, leading in a crash.
 *
 * To go around this problem, modules using the USD SDK need special operators for new and delete. Those operators have the ability to redirect the
 * malloc or free calls to either the UE allocator or to the CRT allocator. The choice of the allocator is made in the FUsdMemoryManager.
 * FUsdMemoryManager manages a stack of active allocators per thread. Using ActivateAllocator and DeactivateAllocator, we can push and pop which
 * allocator is active on the calling thread.
 *
 * To simplify the workflow, TScopedAllocs is provided to make sure a certain block of code is bound to the right allocator.
 * FScopedUsdAllocs is a TScopedAllocs that activates the CRT allocator, while FScopedUnrealAllocs is the one that activates the UE allocator.
 * Since the UE allocator is the default one, FScopedUnrealAllocs is only needed inside a scope where the CRT allocator is active.
 *
 * Usage example:
 * {
 *		FScopedUsdAllocs UsdAllocs;
 *		std::vector<UsdAttribute> Attributes = Prim.GetAttributes();
 *		// do something with the USD attributes.
 * }
 *
 * TUsdStore is also provided to keep USD variables between different scopes (ie: in a class member variable).
 * It makes sure that the USD object is constructed, copied, moved and destroyed with calls to new and delete going through CRT allocator.
 *
 * Usage example:
 * TUsdStore< pxr::UsdPrim > RootPrim = UsdStage->GetPseudoRoot();
 *
 */

class FActiveAllocatorsStack;
class FTlsSlot;

enum class EUsdActiveAllocator
{
	/** Redirects operator new and delete to FMemory::Malloc and FMemory::Free */
	Unreal,
	/** Redirects operator new and delete to FMemory::SystemMalloc and FMemory::SystemFree */
	System
};

#if USE_USD_MEMORY_MANAGER

class FUsdMemoryManager
{
public:
	static UE_API void Initialize();
	static UE_API void Shutdown();

	/** Pushes Allocator on the stack of active allocators */
	static UE_API void ActivateAllocator(EUsdActiveAllocator Allocator);

	/** Pops Allocator from the stack of active allocators */
	static UE_API bool DeactivateAllocator(EUsdActiveAllocator Allocator);

	/** Redirects the call to malloc to the currently active allocator. */
	static UE_API void* Malloc(SIZE_T Count);

	/** Redirects the call to free to the currently active allocator. */
	static UE_API void Free(void* Original);

private:
	static UE_API FActiveAllocatorsStack& GetActiveAllocatorsStackForThread();

	/** Returns if the current active allocator is EActiveAllocator::System */
	static UE_API bool IsUsingSystemMalloc();

	static UE_API TOptional<FTlsSlot> ActiveAllocatorsStackTLS;

	struct FThreadSafeSet
	{
	public:
		void Add(void* Ptr)
		{
			FSubSet& SubSet = SubSets[GetTypeHash(Ptr) % BucketCount];
			FWriteScopeLock ScopeLock(SubSet.Lock);
			SubSet.Set.Add(Ptr);
		}

		bool Remove(void* Ptr)
		{
			FSubSet& SubSet = SubSets[GetTypeHash(Ptr) % BucketCount];
			FWriteScopeLock ScopeLock(SubSet.Lock);
			return SubSet.Set.Remove(Ptr) > 0;
		}

	private:
		struct FSubSet
		{
			FRWLock Lock;
			TSet<void*> Set;
		};

		static constexpr int32 BucketCount = 61; /* Prime number for better modulo distribution */
		FSubSet SubSets[BucketCount];
	};

	static UE_API FThreadSafeSet SystemAllocedPtrs;

	static UE_API FCriticalSection CriticalSection;
};

#endif // USE_USD_MEMORY_MANAGER

/**
 * Activates an allocator on construction and deactivates it on destruction
 */
template<EUsdActiveAllocator AllocatorType>
class TScopedAllocs final
{
public:
	TScopedAllocs()
	{
#if USE_USD_MEMORY_MANAGER

		// clang-format off
#if !FORCE_ANSI_ALLOCATOR && !IS_MONOLITHIC && !USD_MERGED_MODULES	  // If we're in a situation where we need the overriden allocators...
#ifndef SUPPRESS_PER_MODULE_INLINE_FILE	   // ...but we don't have this define (we can't check for whether IMPLEMENT_MODULE_USD is actually used, but
										   // hopefully this is a good enough proxy)
#ifndef DISABLE_USDMEMORY_WARNING	// This is not normally defined anywhere, but can be defined by the user to disable this check and allow
									// compilation anyway
COMPILE_WARNING("You should only include USDMemory.h when necessary. In order to use the allocators from USDMemory.h, make sure that your module uses 'IMPLEMENT_MODULE_USD' on your main module .cpp file, and that 'PrivateDefinitions.Add(\"SUPPRESS_PER_MODULE_INLINE_FILE\");' is added to your module's .Build.cs file. For more info, refer to the official USDImporter documentation, or the documentation comments at the start of USDMemory.h, as well as on the end of UnrealUSDWrapper.Build.cs. You can also disable this compile-time check by defining DISABLE_USDMEMORY_WARNING before including USDMemory.h.")
#endif
#endif
#endif
		// clang-format on

		FUsdMemoryManager::ActivateAllocator(AllocatorType);
#endif // USE_USD_MEMORY_MANAGER
	}

	~TScopedAllocs()
	{
#if USE_USD_MEMORY_MANAGER
		FUsdMemoryManager::DeactivateAllocator(AllocatorType);
#endif
	}

	TScopedAllocs(const TScopedAllocs<AllocatorType>& Other) = delete;
	TScopedAllocs(TScopedAllocs<AllocatorType>&& Other) = delete;

	TScopedAllocs& operator=(const TScopedAllocs<AllocatorType>& Other) = delete;
	TScopedAllocs& operator=(TScopedAllocs<AllocatorType>&& Other) = delete;
};

using FScopedUsdAllocs = TScopedAllocs<EUsdActiveAllocator::System>;
using FScopedUnrealAllocs = TScopedAllocs<EUsdActiveAllocator::Unreal>;

/**
 * Stores a USD object.
 * Ensures that its constructed, copied, moved and destroyed using the USD allocator.
 */
template<typename UsdObjectType>
class TUsdStore final
{
public:
	TUsdStore()
	{
		StoredUsdObject = TOptional<UsdObjectType>{};	 // Force init with current allocator

		{
			FScopedUsdAllocs UsdAllocs;
			StoredUsdObject.Emplace();	  // Create UsdObjectType with USD allocator
		}
	}

	TUsdStore(const TUsdStore<UsdObjectType>& Other)
	{
		FScopedUsdAllocs UsdAllocs;
		StoredUsdObject.Emplace(Other.StoredUsdObject.GetValue());
	}

	TUsdStore(TUsdStore<UsdObjectType>&& Other)
	{
		FScopedUsdAllocs UsdAllocs;
		StoredUsdObject = MoveTemp(Other.StoredUsdObject);
	}

	TUsdStore<UsdObjectType>& operator=(const TUsdStore<UsdObjectType>& Other)
	{
		FScopedUsdAllocs UsdAllocs;
		StoredUsdObject.Emplace(Other.StoredUsdObject.GetValue());
		return *this;
	}

	TUsdStore<UsdObjectType>& operator=(TUsdStore<UsdObjectType>&& Other)
	{
		FScopedUsdAllocs UsdAllocs;
		StoredUsdObject = MoveTemp(Other.StoredUsdObject);
		return *this;
	}

	TUsdStore(const UsdObjectType& UsdObject)
	{
		FScopedUsdAllocs UsdAllocs;
		StoredUsdObject = UsdObject;
	}

	TUsdStore(UsdObjectType&& UsdObject)
	{
		FScopedUsdAllocs UsdAllocs;
		StoredUsdObject = MoveTemp(UsdObject);
	}

	TUsdStore<UsdObjectType>& operator=(const UsdObjectType& UsdObject)
	{
		FScopedUsdAllocs UsdAllocs;
		StoredUsdObject = UsdObject;
		return *this;
	}

	TUsdStore<UsdObjectType>& operator=(UsdObjectType&& UsdObject)
	{
		FScopedUsdAllocs UsdAllocs;
		StoredUsdObject = MoveTemp(UsdObject);
		return *this;
	}

	explicit operator bool() const
	{
		return StoredUsdObject.IsSet();
	}

	~TUsdStore()
	{
		FScopedUsdAllocs UsdAllocs;
		StoredUsdObject.Reset();	// Destroy UsdObjectType with USD allocator
	}

	UsdObjectType& Get()
	{
		return StoredUsdObject.GetValue();
	}

	const UsdObjectType& Get() const
	{
		return StoredUsdObject.GetValue();
	}

	UsdObjectType& operator*()
	{
		return Get();
	}

	const UsdObjectType& operator*() const
	{
		return Get();
	}

	UsdObjectType* operator->()
	{
		return &Get();
	}

	const UsdObjectType* operator->() const
	{
		return &Get();
	}

private:
	TOptional<UsdObjectType> StoredUsdObject;
};

template<typename UsdObjectType, typename... ArgTypes>
TUsdStore<UsdObjectType> MakeUsdStore(ArgTypes&&... Args)
{
	FScopedUsdAllocs UsdAllocs;
	return TUsdStore<UsdObjectType>(UsdObjectType(Forward<ArgTypes>(Args)...));
}

// MakeShared version that makes sure that the SharedPointer allocs are made with the Unreal allocator
template<typename ObjectType, typename... ArgTypes>
TSharedRef<ObjectType> MakeSharedUnreal(ArgTypes&&... Args)
{
	FScopedUnrealAllocs UnrealAllocs;
	return MakeShared<ObjectType>(Forward<ArgTypes>(Args)...);
}

// See comment on UnrealUSDWrapper.Build.cs to understand why we disable these for monolithic builds (everything will still work,
// they're just unnecessary)
// clang-format off
#if USE_USD_MEMORY_MANAGER && !FORCE_ANSI_ALLOCATOR && !IS_MONOLITHIC && !USD_MERGED_MODULES
	#define REPLACEMENT_OPERATOR_NEW_AND_DELETE_USD \
		OPERATOR_NEW_MSVC_PRAGMA void* operator new  ( size_t Size                        ) OPERATOR_NEW_THROW_SPEC      { return FUsdMemoryManager::Malloc( Size ); } \
		OPERATOR_NEW_MSVC_PRAGMA void* operator new[]( size_t Size                        ) OPERATOR_NEW_THROW_SPEC      { return FUsdMemoryManager::Malloc( Size ); } \
		OPERATOR_NEW_MSVC_PRAGMA void* operator new  ( size_t Size, const std::nothrow_t& ) OPERATOR_NEW_NOTHROW_SPEC    { return FUsdMemoryManager::Malloc( Size ); } \
		OPERATOR_NEW_MSVC_PRAGMA void* operator new[]( size_t Size, const std::nothrow_t& ) OPERATOR_NEW_NOTHROW_SPEC    { return FUsdMemoryManager::Malloc( Size ); } \
		void operator delete  ( void* Ptr )                                                 OPERATOR_DELETE_THROW_SPEC   { return FUsdMemoryManager::Free( Ptr ); } \
		void operator delete[]( void* Ptr )                                                 OPERATOR_DELETE_THROW_SPEC   { return FUsdMemoryManager::Free( Ptr ); } \
		void operator delete  ( void* Ptr, const std::nothrow_t& )                          OPERATOR_DELETE_NOTHROW_SPEC { return FUsdMemoryManager::Free( Ptr ); } \
		void operator delete[]( void* Ptr, const std::nothrow_t& )                          OPERATOR_DELETE_NOTHROW_SPEC { return FUsdMemoryManager::Free( Ptr ); } \
		void operator delete  ( void* Ptr, size_t Size )                                    OPERATOR_DELETE_THROW_SPEC   { return FUsdMemoryManager::Free( Ptr ); } \
		void operator delete[]( void* Ptr, size_t Size )                                    OPERATOR_DELETE_THROW_SPEC   { return FUsdMemoryManager::Free( Ptr ); } \
		void operator delete  ( void* Ptr, size_t Size, const std::nothrow_t& )             OPERATOR_DELETE_NOTHROW_SPEC { return FUsdMemoryManager::Free( Ptr ); } \
		void operator delete[]( void* Ptr, size_t Size, const std::nothrow_t& )             OPERATOR_DELETE_NOTHROW_SPEC { return FUsdMemoryManager::Free( Ptr ); }
#else
	#define REPLACEMENT_OPERATOR_NEW_AND_DELETE_USD
#endif

#if USE_USD_MEMORY_MANAGER && !FORCE_ANSI_ALLOCATOR && !IS_MONOLITHIC && !USD_MERGED_MODULES
	#define IMPLEMENT_MODULE_USD( ModuleImplClass, ModuleName ) \
		\
		/**/ \
		/* InitializeModule function, called by module manager after this module's DLL has been loaded */ \
		/**/ \
		/* @return	Returns an instance of this module */ \
		/**/ \
		static IModuleInterface* Initialize##ModuleName##Module() \
		{ \
			return new ModuleImplClass(); \
		} \
		static FModuleInitializerEntry ModuleName##InitializerEntry(TEXT(#ModuleName), Initialize##ModuleName##Module, TEXT(UE_MODULE_NAME)); \
		/* Forced reference to this function is added by the linker to check that each module uses IMPLEMENT_MODULE */ \
		extern "C" void IMPLEMENT_MODULE_##ModuleName() { } \
		extern "C" DLLEXPORT void ThisIsAnUnrealEngineModule() {} \
		UE_VISUALIZERS_HELPERS \
		REPLACEMENT_OPERATOR_NEW_AND_DELETE_USD
#else
	#define IMPLEMENT_MODULE_USD( ModuleImplClass, ModuleName ) IMPLEMENT_MODULE( ModuleImplClass, ModuleName )
#endif
// clang-format on

#undef UE_API
