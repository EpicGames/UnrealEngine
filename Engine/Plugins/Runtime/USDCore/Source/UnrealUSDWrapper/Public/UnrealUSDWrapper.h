// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Engine/EngineTypes.h"
#include "Modules/ModuleInterface.h"
#include "Templates/Tuple.h"
#include "UObject/ObjectMacros.h"

#include "UsdWrappers/ForwardDeclarations.h"

#include <memory>
#include <string>
#include <vector>

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"
#endif	  // #if USE_USD_SDK

#include "UnrealUSDWrapper.generated.h"

// We are capped to 4 because the geometry cache and skeletal mesh + morph target shaders have a hard-coded limit of 4 texture coordinates, so our
// material instances can't cross that limit or else they wouldn't compile when assigned to those meshes
#define USD_PREVIEW_SURFACE_MAX_UV_SETS 4
#define UNUSED_UV_INDEX USD_PREVIEW_SURFACE_MAX_UV_SETS

#if USE_USD_SDK
PXR_NAMESPACE_OPEN_SCOPE
	class GfMatrix4d;
	class SdfPath;
	class TfToken;
	class UsdAttribute;
	class UsdGeomMesh;
	class UsdPrim;
	class UsdStage;
	class UsdStageCache;

	template<typename T>
	class TfRefPtr;
	template<typename T>
	class TfWeakPtr;

	using UsdStageRefPtr = TfRefPtr<UsdStage>;
	using UsdStageWeakPtr = TfWeakPtr<UsdStage>;

	using SdfLayerRefPtr = TfRefPtr<SdfLayer>;
	using SdfLayerWeakPtr = TfWeakPtr<SdfLayer>;
PXR_NAMESPACE_CLOSE_SCOPE
#endif	  // #if USE_USD_SDK

class IUsdPrim;

namespace UE
{
	class FUsdAttribute;
}

enum class EUsdInterpolationMethod
{
	/** Each element in a buffer maps directly to a specific vertex */
	Vertex,
	/** Each element in a buffer maps to a specific face/vertex pair */
	FaceVarying,
	/** Each vertex on a face is the same value */
	Uniform,
	/** Single value */
	Constant
};

enum class EUsdGeomOrientation
{
	/** Right handed coordinate system */
	RightHanded,
	/** Left handed coordinate system */
	LeftHanded,
};

enum class EUsdSubdivisionScheme
{
	None,
	CatmullClark,
	Loop,
	Bilinear,
};

UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EUsdPurpose : int32
{
	Default = 0 UMETA(Hidden),
	Proxy = 1,
	Render = 2,
	Guide = 4
};
ENUM_CLASS_FLAGS(EUsdPurpose);

UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EUsdDefaultKind : int32
{
	None = 0 UMETA(Hidden),
	Model = 1,
	Component = 2,
	Group = 4,
	Assembly = 8,
	Subcomponent = 16
};
ENUM_CLASS_FLAGS(EUsdDefaultKind);

/** Corresponds to pxr::UsdLoadPolicy, refer to the USD SDK documentation */
UENUM()
enum class EUsdLoadPolicy : uint8
{
	UsdLoadWithDescendants,		 // Load a prim plus all its descendants.
	UsdLoadWithoutDescendants	 // Load a prim by itself with no descendants.
};

UENUM()
enum class EUsdInitialLoadSet : uint8
{
	LoadAll,
	LoadNone
};

UENUM()
enum class EUsdInterpolationType : uint8
{
	Held,
	Linear
};

UENUM()
enum class EUsdRootMotionHandling : uint8
{
	// Use for the root bone just its regular joint animation as described on the SkelAnimation prim.
	NoAdditionalRootMotion,

	// Use the transform animation from the SkelRoot prim in addition to the root bone joint animation as
	// described on the SkelAnimation prim. Note that the SkelRoot prim's Sequencer transform track will no longer
	// contain the transform animation data used in this manner, so as to not apply the animation twice.
	UseMotionFromSkelRoot,

	// Use the transform animation from the Skeleton prim in addition to the root bone joint animation as
	// described on the SkelAnimation prim.
	UseMotionFromSkeleton
};

UENUM()
enum class EUsdCollisionType : uint8
{
	// No approximation so equivalent to CTF_UseComplexAsSimple
	None,

	// Breaks up concave meshes into multiple, simple convex collider primitives
	ConvexDecomposition,

	// Single convex hull primitive around the mesh
	ConvexHull,

	// Single sphere collider
	Sphere,

	// Single cube collider
	Cube,

	// Behaves the same as ConvexDecomposition in Unreal, it's just listed separately as it's a separate value in USD
	MeshSimplification,

	// Single capsule collider
	Capsule,

	// Use a separate, dedicated collider mesh
	CustomMesh
};

/** Stage option on how to handle geometry caches in stage workflow */
UENUM()
enum class EGeometryCacheImport : uint8
{
	// Never imported (no persistent assets); always streamed from the USD stage
	Never,

	// Imported on stage load; played back from the persistent assets
	OnLoad,

	// Imported on save; geometry caches are streamed from the stage until they are saved.
	// Afterwards they are played back from the persistent assets
	OnSave
};

/** Corresponds to pxr::UsdListPosition, refer to the USD SDK documentation */
UENUM()
enum class EUsdListPosition : uint8
{
	// The position at the front of the prepend list.
	// An item added at this position will, after composition is applied,
	// be stronger than other items prepended in this layer, and stronger
	// than items added by weaker layers.
	FrontOfPrependList,

	// The position at the back of the prepend list.
	// An item added at this position will, after composition is applied,
	// be weaker than other items prepended in this layer, but stronger
	// than items added by weaker layers.
	BackOfPrependList,

	// The position at the front of the append list.
	// An item added at this position will, after composition is applied,
	// be stronger than other items appended in this layer, and stronger
	// than items added by weaker layers.
	FrontOfAppendList,

	// The position at the back of the append list.
	// An item added at this position will, after composition is applied,
	// be weaker than other items appended in this layer, but stronger
	// than items added by weaker layers.
	BackOfAppendList
};

/**
 * Corresponds to pxr::GfMatrix2d. We don't expose any methods though, this is just to facilitate reading/writing
 * these types from USD.
 */
USTRUCT(BlueprintType)
struct FMatrix2D
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD")
	FVector2D Row0 = FVector2D(1.0, 0.0);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD")
	FVector2D Row1 = FVector2D(0.0, 1.0);

	bool operator==(const FMatrix2D& Other) const = default;
};

/**
 * Corresponds to pxr::GfMatrix3d. We don't expose any methods though, this is just to facilitate reading/writing
 * these types from USD.
 */
USTRUCT(BlueprintType)
struct FMatrix3D
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD")
	FVector Row0 = FVector(1.0, 0.0, 0.0);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD")
	FVector Row1 = FVector(0.0, 1.0, 0.0);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD")
	FVector Row2 = FVector(0.0, 0.0, 1.0);

	bool operator==(const FMatrix3D& Other) const = default;
};

struct FSdfTimeCode
{
	double Time;

	UNREALUSDWRAPPER_API FSdfTimeCode(double Time = 0.0);
	bool operator==(const FSdfTimeCode& Other) const = default;
	UNREALUSDWRAPPER_API operator double() const;
};

struct FSdfAssetPath
{
	FString AssetPath;
	FString ResolvedPath;

	UNREALUSDWRAPPER_API FSdfAssetPath(const FString& AssetPath = {}, const FString& ResolvedPath = {});
	bool operator==(const FSdfAssetPath& Other) const = default;
	UNREALUSDWRAPPER_API operator FString() const;
};

struct FQuat4h
{
	FFloat16 X;
	FFloat16 Y;
	FFloat16 Z;
	FFloat16 W;

	UNREALUSDWRAPPER_API FQuat4h(float X = 0.0f, float Y = 0.0f, float Z = 0.0f, float W = 0.0f);
	UNREALUSDWRAPPER_API FQuat4h(const FQuat4f& Quat);
	bool operator==(const FQuat4h& Other) const = default;
	UNREALUSDWRAPPER_API operator FQuat4f() const;
};

struct FVector3h
{
	FFloat16 X;
	FFloat16 Y;
	FFloat16 Z;

	UNREALUSDWRAPPER_API FVector3h(float X = 0.0f, float Y = 0.0f, float Z = 0.0f);
	UNREALUSDWRAPPER_API FVector3h(const FVector3f& Vec);
	bool operator==(const FVector3h& Other) const = default;
	UNREALUSDWRAPPER_API operator FVector3f() const;
};

struct FVector4h
{
	FFloat16 X;
	FFloat16 Y;
	FFloat16 Z;
	FFloat16 W;

	UNREALUSDWRAPPER_API FVector4h(float X = 0.0f, float Y = 0.0f, float Z = 0.0f, float W = 0.0f);
	UNREALUSDWRAPPER_API FVector4h(const FVector4f& Vec);
	bool operator==(const FVector4h& Other) const = default;
	UNREALUSDWRAPPER_API operator FVector4f() const;
};

class IUnrealUSDWrapperModule : public IModuleInterface
{
};

class UnrealUSDWrapper
{
public:
#if USE_USD_SDK
	UNREALUSDWRAPPER_API static double GetDefaultTimeCode();
#endif	  // #if USE_USD_SDK

	/**
	 * Registers all USD plug-ins discovered at PathToPlugInfo.
	 * @return An array containing the names of any newly registered plugins.
	 */
	UNREALUSDWRAPPER_API static TArray<FString> RegisterPlugins(const FString& PathToPlugInfo);

	/**
	 * Registers all USD plug-ins discovered in any of PathsToPlugInfo.
	 * @return An array containing the names of any newly registered plugins.
	 */
	UNREALUSDWRAPPER_API static TArray<FString> RegisterPlugins(const TArray<FString>& PathsToPlugInfo);

	/**
	 * Returns the file extensions of all file formats supported by USD.
	 *
	 * These include the extensions for the file formats built into USD as
	 * well as those for any other file formats introduced through the USD
	 * plugin system (e.g. "abc" for the Alembic file format of the usdAbc
	 * plugin).
	 */
	UNREALUSDWRAPPER_API static TArray<FString> GetAllSupportedFileFormats();

	/**
	 * Returns the file extensions that are native to USD.
	 *
	 * These are the extensions for the file formats built into USD (i.e. "usd", "usda", "usdc", and "usdz").
	 */
	UNREALUSDWRAPPER_API static TArray<FString> GetNativeFileFormats();
	UNREALUSDWRAPPER_API static void GetNativeFileFormats(TArray<FString>& OutTextFormats, TArray<FString>& OutPossiblyBinaryFormats);

	/**
	 * Internally used by all the export factories, this will query the supported export USD file formats and fill out
	 * OutFormatExtensions with entries like "usd", "usda", "usdc", etc., and OutFormatDescriptions with entries like
	 * "Universal Scene Description binary file" or "Universal Scene Description text file"
	 */
	UNREALUSDWRAPPER_API static void AddUsdExportFileFormatDescriptions(TArray<FString>& OutFormatExtensions, TArray<FString>& OutFormatDescriptions);
	/**
	 * Internally used by all the import factories, this will query the supported export USD file formats and fill out
	 * OutFormats with entries like "usda; Universal Scene Description text files", "usdc; Universal Scene Description binary files", etc.
	 */
	UNREALUSDWRAPPER_API static void AddUsdImportFileFormatDescriptions(TArray<FString>& OutFormats);

	/**
	 * Opens a USD stage from a file on disk or existing layers, with a population mask or not.
	 * @param Identifier - Path to a file that the USD SDK can open (or the identifier of a root layer), which will become the root layer of the new
	 *stage
	 * @param RootLayer - Existing root layer to use for the new stage, instead of reading it from disk
	 * @param SessionLayer - Existing session layer to use for the new stage, instead of creating a new one
	 * @param InitialLoadSet - How to handle USD payloads when opening this stage
	 * @param PopulationMask - List of prim paths to import, following the USD population mask rules
	 * @param bUseStageCache - If true, and the stage is already opened in the stage cache (or the layers are already loaded in the registry) then
	 *						   the file reading may be skipped, and the existing stage returned. When false, the stage and all its referenced layers
	 *						   will be re-read anew, and the stage will not be added to the stage cache.
	 * @param bForceReloadLayersFromDisk - USD layers are always cached in the layer registry, so trying to reopen an
	 *                                     already opened layer will just fetch it from memory (potentially with
	 *                                     edits). If this is true, all local layers used by the stage will be force-
	 *                                     reloaded from disk
	 * @return The opened stage, which may be invalid.
	 */
	UNREALUSDWRAPPER_API static UE::FUsdStage OpenStage(
		const TCHAR* Identifier,
		EUsdInitialLoadSet InitialLoadSet,
		bool bUseStageCache = true,
		bool bForceReloadLayersFromDisk = false
	);
	UNREALUSDWRAPPER_API static UE::FUsdStage OpenStage(
		UE::FSdfLayer RootLayer,
		UE::FSdfLayer SessionLayer,
		EUsdInitialLoadSet InitialLoadSet,
		bool bUseStageCache = true,
		bool bForceReloadLayersFromDisk = false
	);
	UNREALUSDWRAPPER_API static UE::FUsdStage OpenMaskedStage(
		const TCHAR* Identifier,
		EUsdInitialLoadSet InitialLoadSet,
		const TArray<FString>& PopulationMask,
		bool bForceReloadLayersFromDisk = false
	);
	UNREALUSDWRAPPER_API static UE::FUsdStage OpenMaskedStage(
		UE::FSdfLayer RootLayer,
		UE::FSdfLayer SessionLayer,
		EUsdInitialLoadSet InitialLoadSet,
		const TArray<FString>& PopulationMask,
		bool bForceReloadLayersFromDisk = false
	);

	/** Creates a new USD root layer file, opens it as a new stage and returns that stage */
	UNREALUSDWRAPPER_API static UE::FUsdStage NewStage(const TCHAR* FilePath);

	/** Creates a new memory USD root layer, opens it as a new stage and returns that stage */
	UNREALUSDWRAPPER_API static UE::FUsdStage NewStage();

	/**
	 * Get the singleton, persistent stage used as a clipboard for prim cut/copy/paste operations.
	 * WARNING: This stage may remain open indefinitely! If you use this directly, be aware that there may be unintended
	 * consequences (e.g. a sublayer added to this stage may never fully close). When invoking a paste or clear of the
	 * clipboard stage when it is unknown whether a previous cut or copy was done, setting the bCreateIfNeeded parameter
	 * to false will avoid needlessly creating the stage.
	 */
	UNREALUSDWRAPPER_API static UE::FUsdStage GetClipboardStage(bool bCreateIfNeeded = true);

	/** Returns all the stages that are currently opened in the USD utils stage cache, shared between C++ and Python */
	UNREALUSDWRAPPER_API static TArray<UE::FUsdStage> GetAllStagesFromCache();

	/** Removes the stage from the stage cache. See UsdStageCache::Erase. */
	UNREALUSDWRAPPER_API static void EraseStageFromCache(const UE::FUsdStage& Stage);

	/**
	 * Set the directories that will be used as the default search path by USD's default resolver during asset resolution.
	 *
	 * Each directory in the search path should be an absolute path. If it is not, it will be anchored to the current working directory.
	 *
	 * Note that the default search path must be set before the first invocation of USD's resolver system, so this function
	 * must be called before that to have any effect.
	 */
	UNREALUSDWRAPPER_API static void SetDefaultResolverDefaultSearchPath(const TArray<FDirectoryPath>& SearchPath);

	/** Starts listening to error/warning/log messages emitted by USD */
	UE_DEPRECATED(5.6, "Use RegisterDiagnosticDelegate from USDErrorUtils.h.")
	UNREALUSDWRAPPER_API static void SetupDiagnosticDelegate();

	/** Stops listening to error/warning/log messages emitted by USD */
	UE_DEPRECATED(5.6, "Use UnregisterDiagnosticDelegate from USDErrorUtils.h.")
	UNREALUSDWRAPPER_API static void ClearDiagnosticDelegate();
};

class IUsdPrim
{
public:
#if USE_USD_SDK
	static UNREALUSDWRAPPER_API bool IsValidPrimName(const FString& Name, FText& OutReason);

	static UNREALUSDWRAPPER_API EUsdPurpose GetPurpose(const pxr::UsdPrim& Prim, bool bComputed = true);

	static UNREALUSDWRAPPER_API bool HasGeometryData(const pxr::UsdPrim& Prim);
	static UNREALUSDWRAPPER_API bool HasGeometryDataOrLODVariants(const pxr::UsdPrim& Prim);
	static UNREALUSDWRAPPER_API int GetNumLODs(const pxr::UsdPrim& Prim);

	static UNREALUSDWRAPPER_API bool IsKindChildOf(const pxr::UsdPrim& Prim, const std::string& InBaseKind);
	static UNREALUSDWRAPPER_API pxr::TfToken GetKind(const pxr::UsdPrim& Prim);
	static UNREALUSDWRAPPER_API bool SetKind(const pxr::UsdPrim& Prim, const pxr::TfToken& Kind);
	static UNREALUSDWRAPPER_API bool ClearKind(const pxr::UsdPrim& Prim);

	static UNREALUSDWRAPPER_API pxr::GfMatrix4d GetLocalTransform(const pxr::UsdPrim& Prim);
	static UNREALUSDWRAPPER_API pxr::GfMatrix4d GetLocalToWorldTransform(const pxr::UsdPrim& Prim);
	static UNREALUSDWRAPPER_API pxr::GfMatrix4d GetLocalToWorldTransform(const pxr::UsdPrim& Prim, double Time);
	static UNREALUSDWRAPPER_API pxr::GfMatrix4d GetLocalToWorldTransform(const pxr::UsdPrim& Prim, double Time, const pxr::SdfPath& AbsoluteRootPath);
	static UNREALUSDWRAPPER_API bool HasTransform(const pxr::UsdPrim& Prim);

	static UNREALUSDWRAPPER_API bool SetActiveLODIndex(const pxr::UsdPrim& Prim, int LODIndex);

	static UNREALUSDWRAPPER_API EUsdGeomOrientation GetGeometryOrientation(const pxr::UsdGeomMesh& Mesh);
	static UNREALUSDWRAPPER_API EUsdGeomOrientation GetGeometryOrientation(const pxr::UsdGeomMesh& Mesh, double Time);
#endif	  // #if USE_USD_SDK
};

namespace UnrealIdentifiers
{
#if USE_USD_SDK
	extern UNREALUSDWRAPPER_API const pxr::TfToken LOD;
	extern UNREALUSDWRAPPER_API const pxr::TfToken LodSubtreeAPI;
	extern UNREALUSDWRAPPER_API const pxr::TfToken UnrealLodSubtreeLevels;

	extern UNREALUSDWRAPPER_API const pxr::TfToken SubdivisionAPI;
	extern UNREALUSDWRAPPER_API const pxr::TfToken SubdivisionLevel;

	extern UNREALUSDWRAPPER_API const FString StaticMeshBuildSettingsAPI;
	extern UNREALUSDWRAPPER_API const FString NaniteBuildSettingsAPI;

	/* Attribute name when assigning Unreal materials to UsdGeomMeshes */
	extern UNREALUSDWRAPPER_API const pxr::TfToken MaterialAssignment;
	extern UNREALUSDWRAPPER_API const pxr::TfToken Unreal;

	extern UNREALUSDWRAPPER_API const pxr::TfToken UnrealNaniteOverride;
	extern UNREALUSDWRAPPER_API const pxr::TfToken UnrealNaniteOverrideEnable;
	extern UNREALUSDWRAPPER_API const pxr::TfToken UnrealNaniteOverrideDisable;

	extern UNREALUSDWRAPPER_API const pxr::TfToken LiveLinkAPI;
	extern UNREALUSDWRAPPER_API const pxr::TfToken ControlRigAPI;
	extern UNREALUSDWRAPPER_API const pxr::TfToken UnrealAnimBlueprintPath;
	extern UNREALUSDWRAPPER_API const pxr::TfToken UnrealLiveLinkSubjectName;
	extern UNREALUSDWRAPPER_API const pxr::TfToken UnrealLiveLinkEnabled;
	extern UNREALUSDWRAPPER_API const pxr::TfToken UnrealControlRigPath;
	extern UNREALUSDWRAPPER_API const pxr::TfToken UnrealUseFKControlRig;
	extern UNREALUSDWRAPPER_API const pxr::TfToken UnrealControlRigReduceKeys;
	extern UNREALUSDWRAPPER_API const pxr::TfToken UnrealControlRigReductionTolerance;

	extern UNREALUSDWRAPPER_API const pxr::TfToken SparseVolumeTextureAPI;
	extern UNREALUSDWRAPPER_API const pxr::TfToken UnrealSVTMappedFields;
	extern UNREALUSDWRAPPER_API const pxr::TfToken UnrealSVTMappedMaterialParameters;
	extern UNREALUSDWRAPPER_API const pxr::TfToken UnrealSVTMappedGridComponents;
	extern UNREALUSDWRAPPER_API const pxr::TfToken UnrealSVTMappedAttributeChannels;
	extern UNREALUSDWRAPPER_API const pxr::TfToken UnrealSVTAttributesADataType;
	extern UNREALUSDWRAPPER_API const pxr::TfToken UnrealSVTAttributesBDataType;

	extern UNREALUSDWRAPPER_API const pxr::TfToken DiffuseColor;
	extern UNREALUSDWRAPPER_API const pxr::TfToken EmissiveColor;
	extern UNREALUSDWRAPPER_API const pxr::TfToken Metallic;
	extern UNREALUSDWRAPPER_API const pxr::TfToken Roughness;
	extern UNREALUSDWRAPPER_API const pxr::TfToken Opacity;
	extern UNREALUSDWRAPPER_API const pxr::TfToken OpacityMode;
	extern UNREALUSDWRAPPER_API const pxr::TfToken OpacityThreshold;
	extern UNREALUSDWRAPPER_API const pxr::TfToken Normal;
	extern UNREALUSDWRAPPER_API const pxr::TfToken Specular;
	extern UNREALUSDWRAPPER_API const pxr::TfToken SpecularColor;
	extern UNREALUSDWRAPPER_API const pxr::TfToken UseSpecularWorkflow;
	extern UNREALUSDWRAPPER_API const pxr::TfToken ClearCoat;
	extern UNREALUSDWRAPPER_API const pxr::TfToken ClearCoatRoughness;
	extern UNREALUSDWRAPPER_API const pxr::TfToken Anisotropy;
	extern UNREALUSDWRAPPER_API const pxr::TfToken Tangent;
	extern UNREALUSDWRAPPER_API const pxr::TfToken SubsurfaceColor;
	extern UNREALUSDWRAPPER_API const pxr::TfToken Displacement;
	extern UNREALUSDWRAPPER_API const pxr::TfToken Occlusion;
	extern UNREALUSDWRAPPER_API const pxr::TfToken Refraction;

	extern UNREALUSDWRAPPER_API const pxr::TfToken Xform;

	// Tokens used mostly for shade material conversion
	extern UNREALUSDWRAPPER_API const pxr::TfToken Surface;
	extern UNREALUSDWRAPPER_API const pxr::TfToken St;
	extern UNREALUSDWRAPPER_API const pxr::TfToken Varname;
	extern UNREALUSDWRAPPER_API const pxr::TfToken Scale;
	extern UNREALUSDWRAPPER_API const pxr::TfToken Rotation;
	extern UNREALUSDWRAPPER_API const pxr::TfToken Translation;
	extern UNREALUSDWRAPPER_API const pxr::TfToken In;
	extern UNREALUSDWRAPPER_API const pxr::TfToken Result;
	extern UNREALUSDWRAPPER_API const pxr::TfToken File;
	extern UNREALUSDWRAPPER_API const pxr::TfToken WrapS;
	extern UNREALUSDWRAPPER_API const pxr::TfToken WrapT;
	extern UNREALUSDWRAPPER_API const pxr::TfToken Repeat;
	extern UNREALUSDWRAPPER_API const pxr::TfToken Mirror;
	extern UNREALUSDWRAPPER_API const pxr::TfToken Clamp;
	extern UNREALUSDWRAPPER_API const pxr::TfToken Fallback;
	extern UNREALUSDWRAPPER_API const pxr::TfToken R;
	extern UNREALUSDWRAPPER_API const pxr::TfToken RGB;
	extern UNREALUSDWRAPPER_API const pxr::TfToken RawColorSpaceToken;
	extern UNREALUSDWRAPPER_API const pxr::TfToken SRGBColorSpaceToken;
	extern UNREALUSDWRAPPER_API const pxr::TfToken SourceColorSpaceToken;

	// Tokens copied from usdImaging, because at the moment it's all we need from it
	extern UNREALUSDWRAPPER_API const pxr::TfToken UsdPreviewSurface;
	extern UNREALUSDWRAPPER_API const pxr::TfToken UsdTransform2d;
	extern UNREALUSDWRAPPER_API const pxr::TfToken UsdPrimvarReader_float2;
	extern UNREALUSDWRAPPER_API const pxr::TfToken UsdPrimvarReader_float3;
	extern UNREALUSDWRAPPER_API const pxr::TfToken UsdUVTexture;

	// Token used to indicate that a material parsed from a material prim should use world space normals
	extern UNREALUSDWRAPPER_API const pxr::TfToken WorldSpaceNormals;

	// Normals and points can also be primvars and have indices, but there is no defined tokens for them
	extern UNREALUSDWRAPPER_API const pxr::TfToken PrimvarsNormals;
	extern UNREALUSDWRAPPER_API const pxr::TfToken PrimvarsPoints;

	extern UNREALUSDWRAPPER_API const pxr::TfToken GroomAPI;
	extern UNREALUSDWRAPPER_API const pxr::TfToken GroomBindingAPI;
	extern UNREALUSDWRAPPER_API const pxr::TfToken UnrealGroomToBind;
	extern UNREALUSDWRAPPER_API const pxr::TfToken UnrealGroomReferenceMesh;

	extern UNREALUSDWRAPPER_API const pxr::TfToken UnrealContentPath;
	extern UNREALUSDWRAPPER_API const pxr::TfToken UnrealAssetType;
	extern UNREALUSDWRAPPER_API const pxr::TfToken UnrealExportTime;
	extern UNREALUSDWRAPPER_API const pxr::TfToken UnrealEngineVersion;

	extern UNREALUSDWRAPPER_API const pxr::TfToken UnrealCollapsingAPI;
	extern UNREALUSDWRAPPER_API const pxr::TfToken UnrealCollapsingAttr;
	extern UNREALUSDWRAPPER_API const pxr::TfToken CollapsingAllow;
	extern UNREALUSDWRAPPER_API const pxr::TfToken CollapsingDefault;
	extern UNREALUSDWRAPPER_API const pxr::TfToken CollapsingNever;

	extern UNREALUSDWRAPPER_API const pxr::TfToken NaniteAssemblyRootAPI;
	extern UNREALUSDWRAPPER_API const pxr::TfToken NaniteAssemblyExternalRefAPI;
	extern UNREALUSDWRAPPER_API const pxr::TfToken NaniteAssemblySkelBindingAPI;
	extern UNREALUSDWRAPPER_API const pxr::TfToken UnrealNaniteAssemblyMeshType;
	extern UNREALUSDWRAPPER_API const pxr::TfToken UnrealNaniteAssemblyMeshAssetPath;
	extern UNREALUSDWRAPPER_API const pxr::TfToken UnrealNaniteAssemblySkeleton;
	extern UNREALUSDWRAPPER_API const pxr::TfToken PrimvarsUnrealNaniteAssemblyBindJoints;
	extern UNREALUSDWRAPPER_API const pxr::TfToken PrimvarsUnrealNaniteAssemblyBindJointWeights;
	extern UNREALUSDWRAPPER_API const pxr::TfToken NaniteAssemblyStaticMesh;
	extern UNREALUSDWRAPPER_API const pxr::TfToken NaniteAssemblySkeletalMesh;

#endif	  // #if USE_USD_SDK

	extern UNREALUSDWRAPPER_API const TCHAR* LayerSavedComment;
	extern UNREALUSDWRAPPER_API const TCHAR* TwoSidedMaterialSuffix;

	extern UNREALUSDWRAPPER_API const TCHAR* Invisible;
	extern UNREALUSDWRAPPER_API const TCHAR* Inherited;
	extern UNREALUSDWRAPPER_API const TCHAR* IdentifierPrefix;

	// The prim name we always use when exporting Skeletons inside SkelRoots to USD
	extern UNREALUSDWRAPPER_API const TCHAR* ExportedSkeletonPrimName;

	// USceneComponent properties
	extern UNREALUSDWRAPPER_API FName TransformPropertyName;
	extern UNREALUSDWRAPPER_API FName HiddenInGamePropertyName;
	extern UNREALUSDWRAPPER_API FName HiddenPropertyName;

	// UCameraComponent properties
	extern UNREALUSDWRAPPER_API FName AspectRatioPropertyName;
	extern UNREALUSDWRAPPER_API FName OrthoWidthPropertyName;
	extern UNREALUSDWRAPPER_API FName OrthoFarClipPlanePropertyName;
	extern UNREALUSDWRAPPER_API FName OrthoNearClipPlanePropertyName;

	// UCineCameraComponent properties
	extern UNREALUSDWRAPPER_API FName CurrentFocalLengthPropertyName;
	extern UNREALUSDWRAPPER_API FName ManualFocusDistancePropertyName;
	extern UNREALUSDWRAPPER_API FName CurrentAperturePropertyName;
	extern UNREALUSDWRAPPER_API FName SensorWidthPropertyName;
	extern UNREALUSDWRAPPER_API FName SensorHeightPropertyName;
	extern UNREALUSDWRAPPER_API FName SensorHorizontalOffsetPropertyName;
	extern UNREALUSDWRAPPER_API FName SensorVerticalOffsetPropertyName;
	extern UNREALUSDWRAPPER_API FName ExposureCompensationPropertyName;
	extern UNREALUSDWRAPPER_API FName ProjectionModePropertyName;
	extern UNREALUSDWRAPPER_API FName CustomNearClipppingPlanePropertyName;
	extern UNREALUSDWRAPPER_API FName DepthOfFieldFstopPropertyName;
	extern UNREALUSDWRAPPER_API FName CameraISOPropertyName;
	extern UNREALUSDWRAPPER_API FName CameraShutterSpeedPropertyName;

	// LightComponentBase properties
	extern UNREALUSDWRAPPER_API FName IntensityPropertyName;
	extern UNREALUSDWRAPPER_API FName LightColorPropertyName;

	// LightComponent properties
	extern UNREALUSDWRAPPER_API FName UseTemperaturePropertyName;
	extern UNREALUSDWRAPPER_API FName TemperaturePropertyName;

	// URectLightComponent properties
	extern UNREALUSDWRAPPER_API FName SourceWidthPropertyName;
	extern UNREALUSDWRAPPER_API FName SourceHeightPropertyName;

	// UPointLightComponent properties
	extern UNREALUSDWRAPPER_API FName SourceRadiusPropertyName;

	// USpotLightComponent properties
	extern UNREALUSDWRAPPER_API FName OuterConeAnglePropertyName;
	extern UNREALUSDWRAPPER_API FName InnerConeAnglePropertyName;

	// UDirectionalLightComponent properties
	extern UNREALUSDWRAPPER_API FName LightSourceAnglePropertyName;

	// Material purpose tokens that we convert from USD, if available
	extern UNREALUSDWRAPPER_API FString MaterialAllPurpose;
	extern UNREALUSDWRAPPER_API FString MaterialAllPurposeText;	   // Text to show on UI for "allPurpose", as its value is actually the empty string
	extern UNREALUSDWRAPPER_API FString MaterialPreviewPurpose;
	extern UNREALUSDWRAPPER_API FString MaterialFullPurpose;

	extern UNREALUSDWRAPPER_API FString PrimvarsDisplayColor;
	extern UNREALUSDWRAPPER_API FString PrimvarsDisplayOpacity;
	extern UNREALUSDWRAPPER_API FString DoubleSided;

	// Tokens from UsdGeomModelAPI that we need to reference from the UsdStageActor
	extern UNREALUSDWRAPPER_API FString ModelDrawMode;
	extern UNREALUSDWRAPPER_API FString ModelApplyDrawMode;

	// The character used to separate property namespaces (usually just ':')
	extern UNREALUSDWRAPPER_API FString UsdNamespaceDelimiter;

	extern UNREALUSDWRAPPER_API const FName UniversalRenderContext;	   // Likely the empty string
	extern UNREALUSDWRAPPER_API const FName UnrealRenderContext;
	extern UNREALUSDWRAPPER_API const FName MaterialXRenderContext;
	extern UNREALUSDWRAPPER_API const FName MdlRenderContext;

	extern UNREALUSDWRAPPER_API const FString UniversalRenderContextDisplayString;	  // "universal"

	// This marks a generated scene component as corresponding to a PointInstancer prim.
	// It is mainly used to let us know to cleanup any ISM/HISM child component whenever we're cleaning up the scene component
	extern UNREALUSDWRAPPER_API const FName PointInstancerTag;

}	 // namespace UnrealIdentifiers

struct FUsdDelegates
{
	DECLARE_MULTICAST_DELEGATE_OneParam(FUsdImportDelegate, FString /* FilePath */);
	static UNREALUSDWRAPPER_API FUsdImportDelegate OnPreUsdImport;
	static UNREALUSDWRAPPER_API FUsdImportDelegate OnPostUsdImport;
};
