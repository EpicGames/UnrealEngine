// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class UsdStage;
	class SdfLayer;

	template<typename T>
	class TfRefPtr;
	template<typename T>
	class TfWeakPtr;

	using UsdStageRefPtr = TfRefPtr<UsdStage>;
	using UsdStageWeakPtr = TfWeakPtr<UsdStage>;

	using SdfLayerRefPtr = TfRefPtr<SdfLayer>;
	using SdfLayerWeakPtr = TfWeakPtr<SdfLayer>;
PXR_NAMESPACE_CLOSE_SCOPE

namespace UE
{
	template<typename PtrType>
	class FUsdStageBase;
	using FUsdStage = FUsdStageBase<pxr::UsdStageRefPtr>;
	using FUsdStageWeak = FUsdStageBase<pxr::UsdStageWeakPtr>;

	template<typename PtrType>
	class FSdfLayerBase;
	using FSdfLayer = FSdfLayerBase<pxr::SdfLayerRefPtr>;
	using FSdfLayerWeak = FSdfLayerBase<pxr::SdfLayerWeakPtr>;
}

#else	  // When the USD SDK is not available
namespace UE
{
	// We use these types so that we can keep the same FUsdStage, etc. type aliases
	// and not change the rest of the code. It's the bare minimum to be able to
	// be tested for validity (and always fail).
	class FDummyPtrBase
	{
	public:
		explicit operator bool() const
		{
			return false;
		}

		bool operator==(const FDummyPtrBase&) const
		{
			return false;
		}
	};

	// We use separate ref and weak types to prevent multiple symbol
	// definitions on the wrapper copy constructor/assignment operators.
	class FDummyWeakPtrType;

	class FDummyRefPtrType : public FDummyPtrBase
	{
	public:
		FDummyRefPtrType(){};
		FDummyRefPtrType(const FDummyWeakPtrType&){};
	};

	class FDummyWeakPtrType : public FDummyPtrBase
	{
	public:
		FDummyWeakPtrType(){};
		FDummyWeakPtrType(const FDummyRefPtrType&){};
	};

	template<typename PtrType>
	class FUsdStageBase;
	using FUsdStage = FUsdStageBase<FDummyRefPtrType>;
	using FUsdStageWeak = FUsdStageBase<FDummyWeakPtrType>;

	template<typename PtrType>
	class FSdfLayerBase;
	using FSdfLayer = FSdfLayerBase<FDummyRefPtrType>;
	using FSdfLayerWeak = FSdfLayerBase<FDummyWeakPtrType>;
}	 // namespace UE
#endif	  // #if USE_USD_SDK
