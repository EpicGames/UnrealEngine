// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UniquePtr.h"
#include "UsdWrappers/ForwardDeclarations.h"

#if USE_USD_SDK

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class PcpLayerStack;

	template<typename T>
	class TfRefPtr;

	using PcpLayerStackRefPtr = TfRefPtr<PcpLayerStack>;
PXR_NAMESPACE_CLOSE_SCOPE

#endif	  // #if USE_USD_SDK

namespace UE
{
	class FPcpLayerStackIdentifier;
	struct FSdfLayerOffset;
	namespace Internal
	{
		class FPcpLayerStackImpl;
	}

	/**
	 * Minimal pxr::PcpLayerStackRefPtr wrapper for Unreal that can be used from no-rtti modules.
	 */
	class UNREALUSDWRAPPER_API FPcpLayerStack
	{
	public:
		FPcpLayerStack();

		FPcpLayerStack(const FPcpLayerStack& Other);
		FPcpLayerStack(FPcpLayerStack&& Other);
		~FPcpLayerStack();

		FPcpLayerStack& operator=(const FPcpLayerStack& Other);
		FPcpLayerStack& operator=(FPcpLayerStack&& Other);

		bool operator==(const FPcpLayerStack& Other) const;
		bool operator!=(const FPcpLayerStack& Other) const;

		explicit operator bool() const;

		// Auto conversion from/to pxr::PcpLayerStackRefPtr
	public:
#if USE_USD_SDK
		explicit FPcpLayerStack(const pxr::PcpLayerStackRefPtr& InPcpLayerStack);
		explicit FPcpLayerStack(pxr::PcpLayerStackRefPtr&& InPcpLayerStack);
		FPcpLayerStack& operator=(const pxr::PcpLayerStackRefPtr& InPcpLayerStack);
		FPcpLayerStack& operator=(pxr::PcpLayerStackRefPtr&& InPcpLayerStack);

		operator pxr::PcpLayerStackRefPtr&();
		operator const pxr::PcpLayerStackRefPtr&() const;
#endif	  // #if USE_USD_SDK

		  // Wrapped pxr::PcpLayerStack functions, refer to the USD SDK documentation
	public:
		FPcpLayerStackIdentifier GetIdentifier() const;
		TArray<FSdfLayer> GetLayers() const;
		FSdfLayerOffset GetLayerOffsetForLayer(const FSdfLayer& Layer) const;
		FSdfLayerOffset GetLayerOffsetForLayer(const FSdfLayerWeak& Layer) const;
		bool HasLayer(const FSdfLayer& Layer) const;
		bool HasLayer(const FSdfLayerWeak& Layer) const;

	private:
		TUniquePtr<Internal::FPcpLayerStackImpl> Impl;
	};
}	 // namespace UE
