// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UniquePtr.h"
#include "UsdWrappers/ForwardDeclarations.h"

#if USE_USD_SDK

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class PcpLayerStackIdentifier;
PXR_NAMESPACE_CLOSE_SCOPE

#endif	  // #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
		class FPcpLayerStackIdentifierImpl;
	}

	/**
	 * Minimal pxr::PcpLayerStackIdentifier wrapper for Unreal that can be used from no-rtti modules.
	 */
	class UNREALUSDWRAPPER_API FPcpLayerStackIdentifier
	{
	public:
		FPcpLayerStackIdentifier();

		FPcpLayerStackIdentifier(const FPcpLayerStackIdentifier& Other);
		FPcpLayerStackIdentifier(FPcpLayerStackIdentifier&& Other);
		~FPcpLayerStackIdentifier();

		FPcpLayerStackIdentifier& operator=(const FPcpLayerStackIdentifier& Other);
		FPcpLayerStackIdentifier& operator=(FPcpLayerStackIdentifier&& Other);

		bool operator==(const FPcpLayerStackIdentifier& Other) const;
		bool operator!=(const FPcpLayerStackIdentifier& Other) const;

		explicit operator bool() const;

		// Auto conversion from/to pxr::PcpLayerStackIdentifier
	public:
#if USE_USD_SDK
		explicit FPcpLayerStackIdentifier(const pxr::PcpLayerStackIdentifier& InPcpLayerStackIdentifier);
		explicit FPcpLayerStackIdentifier(pxr::PcpLayerStackIdentifier&& InPcpLayerStackIdentifier);
		FPcpLayerStackIdentifier& operator=(const pxr::PcpLayerStackIdentifier& InPcpLayerStackIdentifier);
		FPcpLayerStackIdentifier& operator=(pxr::PcpLayerStackIdentifier&& InPcpLayerStackIdentifier);

		operator pxr::PcpLayerStackIdentifier&();
		operator const pxr::PcpLayerStackIdentifier&() const;
#endif	  // #if USE_USD_SDK

		  // Wrapped pxr::PcpLayerStackIdentifier functions, refer to the USD SDK documentation
	public:
		FSdfLayerWeak RootLayer() const;
		FSdfLayerWeak SessionLayer() const;

	private:
		TUniquePtr<Internal::FPcpLayerStackIdentifierImpl> Impl;
	};
}	 // namespace UE
