// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UniquePtr.h"

#if USE_USD_SDK

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class PcpMapExpression;
PXR_NAMESPACE_CLOSE_SCOPE

#endif	  // #if USE_USD_SDK

namespace UE
{
	struct FSdfLayerOffset;
	namespace Internal
	{
		class FPcpMapExpressionImpl;
	}

	/**
	 * Minimal pxr::PcpMapExpression wrapper for Unreal that can be used from no-rtti modules.
	 */
	class UNREALUSDWRAPPER_API FPcpMapExpression
	{
	public:
		FPcpMapExpression();

		FPcpMapExpression(const FPcpMapExpression& Other);
		FPcpMapExpression(FPcpMapExpression&& Other);
		~FPcpMapExpression();

		FPcpMapExpression& operator=(const FPcpMapExpression& Other);
		FPcpMapExpression& operator=(FPcpMapExpression&& Other);

		// Auto conversion from/to pxr::PcpMapExpression
	public:
#if USE_USD_SDK
		explicit FPcpMapExpression(const pxr::PcpMapExpression& InPcpMapExpression);
		explicit FPcpMapExpression(pxr::PcpMapExpression&& InPcpMapExpression);
		FPcpMapExpression& operator=(const pxr::PcpMapExpression& InPcpMapExpression);
		FPcpMapExpression& operator=(pxr::PcpMapExpression&& InPcpMapExpression);

		operator pxr::PcpMapExpression&();
		operator const pxr::PcpMapExpression&() const;
#endif	  // #if USE_USD_SDK

		  // Wrapped pxr::PcpMapExpression functions, refer to the USD SDK documentation
	public:
		FSdfLayerOffset GetTimeOffset() const;

	private:
		TUniquePtr<Internal::FPcpMapExpressionImpl> Impl;
	};
}	 // namespace UE
