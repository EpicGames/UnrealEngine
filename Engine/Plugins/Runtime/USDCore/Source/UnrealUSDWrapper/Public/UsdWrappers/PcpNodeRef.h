// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UniquePtr.h"

#if USE_USD_SDK

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class PcpNodeRef;
PXR_NAMESPACE_CLOSE_SCOPE

#endif	  // #if USE_USD_SDK

namespace UE
{
	class FSdfPath;
	class FPcpLayerStack;
	class FPcpMapExpression;
	namespace Internal
	{
		class FPcpNodeRefImpl;
	}

	/** Analogous to pxr::PcpArcType */
	enum class EPcpArcType
	{
		PcpArcTypeRoot,
		PcpArcTypeInherit,
		PcpArcTypeVariant,
		PcpArcTypeRelocate,
		PcpArcTypeReference,
		PcpArcTypePayload,
		PcpArcTypeSpecialize,

		PcpNumArcTypes
	};

	/**
	 * Minimal pxr::PcpNodeRef wrapper for Unreal that can be used from no-rtti modules.
	 */
	class UNREALUSDWRAPPER_API FPcpNodeRef
	{
	public:
		FPcpNodeRef();

		FPcpNodeRef(const FPcpNodeRef& Other);
		FPcpNodeRef(FPcpNodeRef&& Other);
		~FPcpNodeRef();

		FPcpNodeRef& operator=(const FPcpNodeRef& Other);
		FPcpNodeRef& operator=(FPcpNodeRef&& Other);

		bool operator==(const FPcpNodeRef& Other) const;
		bool operator!=(const FPcpNodeRef& Other) const;

		explicit operator bool() const;

		// Auto conversion from/to pxr::PcpNodeRef
	public:
#if USE_USD_SDK
		explicit FPcpNodeRef(const pxr::PcpNodeRef& InPcpNodeRef);
		explicit FPcpNodeRef(pxr::PcpNodeRef&& InPcpNodeRef);
		FPcpNodeRef& operator=(const pxr::PcpNodeRef& InPcpNodeRef);
		FPcpNodeRef& operator=(pxr::PcpNodeRef&& InPcpNodeRef);

		operator pxr::PcpNodeRef&();
		operator const pxr::PcpNodeRef&() const;
#endif	  // #if USE_USD_SDK

		  // Wrapped pxr::PcpNodeRef functions, refer to the USD SDK documentation
	public:
		EPcpArcType GetArcType() const;
		FPcpNodeRef GetParentNode() const;
		FPcpMapExpression GetMapToParent() const;
		FPcpMapExpression GetMapToRoot() const;
		FSdfPath GetPath() const;
		FPcpLayerStack GetLayerStack() const;

	private:
		TUniquePtr<Internal::FPcpNodeRefImpl> Impl;
	};
}	 // namespace UE
