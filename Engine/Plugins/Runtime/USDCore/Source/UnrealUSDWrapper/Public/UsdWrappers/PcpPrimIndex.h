// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UniquePtr.h"

#if USE_USD_SDK

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class PcpPrimIndex;
PXR_NAMESPACE_CLOSE_SCOPE

#endif	  // #if USE_USD_SDK

namespace UE
{
	class FPcpNodeRef;
	namespace Internal
	{
		class FPcpPrimIndexImpl;
	}

	/**
	 * Minimal pxr::PcpPrimIndex wrapper for Unreal that can be used from no-rtti modules.
	 */
	class UNREALUSDWRAPPER_API FPcpPrimIndex
	{
	public:
		FPcpPrimIndex();

		FPcpPrimIndex(const FPcpPrimIndex& Other);
		FPcpPrimIndex(FPcpPrimIndex&& Other);
		~FPcpPrimIndex();

		FPcpPrimIndex& operator=(const FPcpPrimIndex& Other);
		FPcpPrimIndex& operator=(FPcpPrimIndex&& Other);

		explicit operator bool() const;

		// Auto conversion from/to pxr::PcpPrimIndex
	public:
#if USE_USD_SDK
		explicit FPcpPrimIndex(const pxr::PcpPrimIndex& InPcpPrimIndex);
		explicit FPcpPrimIndex(pxr::PcpPrimIndex&& InPcpPrimIndex);
		FPcpPrimIndex& operator=(const pxr::PcpPrimIndex& InPcpPrimIndex);
		FPcpPrimIndex& operator=(pxr::PcpPrimIndex&& InPcpPrimIndex);

		operator pxr::PcpPrimIndex&();
		operator const pxr::PcpPrimIndex&() const;
#endif	  // #if USE_USD_SDK

		  // Wrapped pxr::PcpPrimIndex functions, refer to the USD SDK documentation
	public:
		bool IsValid() const;
		FPcpNodeRef GetRootNode() const;
		FString DumpToString(bool bIncludeInheritOriginInfo = true, bool bIncludeMaps = true) const;

	private:
		TUniquePtr<Internal::FPcpPrimIndexImpl> Impl;
	};
}	 // namespace UE
