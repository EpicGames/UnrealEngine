// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UniquePtr.h"

#if USE_USD_SDK

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class SdfAttributeSpec;
	template<class T>
	class SdfHandle;
	using SdfAttributeSpecHandle = SdfHandle<SdfAttributeSpec>;
PXR_NAMESPACE_CLOSE_SCOPE

#endif	  // #if USE_USD_SDK

namespace UE
{
	class FVtValue;

	namespace Internal
	{
		class FSdfAttributeSpecImpl;
	}

	/**
	 * Minimal pxr::SdfAttributeSpecHandle wrapper for Unreal that can be used from no-rtti modules.
	 */
	class UNREALUSDWRAPPER_API FSdfAttributeSpec
	{
	public:
		FSdfAttributeSpec();

		FSdfAttributeSpec(const FSdfAttributeSpec& Other);
		FSdfAttributeSpec(FSdfAttributeSpec&& Other);
		~FSdfAttributeSpec();

		FSdfAttributeSpec& operator=(const FSdfAttributeSpec& Other);
		FSdfAttributeSpec& operator=(FSdfAttributeSpec&& Other);

		bool operator==(const FSdfAttributeSpec& Other) const;
		bool operator!=(const FSdfAttributeSpec& Other) const;

		explicit operator bool() const;

		// Auto conversion from/to pxr::SdfAttributeSpecHandle
	public:
#if USE_USD_SDK
		explicit FSdfAttributeSpec(const pxr::SdfAttributeSpecHandle& InSdfAttributeSpecHandle);
		explicit FSdfAttributeSpec(pxr::SdfAttributeSpecHandle&& InSdfAttributeSpecHandle);
		FSdfAttributeSpec& operator=(const pxr::SdfAttributeSpecHandle& InSdfAttributeSpecHandle);
		FSdfAttributeSpec& operator=(pxr::SdfAttributeSpecHandle&& InSdfAttributeSpecHandle);

		operator pxr::SdfAttributeSpecHandle&();
		operator const pxr::SdfAttributeSpecHandle&() const;
#endif	  // #if USE_USD_SDK

		  // Wrapped pxr::SdfAttributeSpec functions, refer to the USD SDK documentation
	public:
		UE::FVtValue GetDefaultValue() const;
		bool SetDefaultValue(const UE::FVtValue& Value);

		FString GetName() const;

	private:
		TUniquePtr<Internal::FSdfAttributeSpecImpl> Impl;
	};
}	 // namespace UE
