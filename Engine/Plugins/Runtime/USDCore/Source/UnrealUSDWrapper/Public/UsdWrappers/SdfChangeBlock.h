// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UniquePtr.h"

#define UE_API UNREALUSDWRAPPER_API

namespace UE
{
	namespace Internal
	{
		class FSdfChangeBlockImpl;
	}

	/**
	 * Minimal pxr::SdfChangeBlock wrapper for Unreal that can be used from no-rtti modules.
	 */
	class FSdfChangeBlock final
	{
	public:
		UE_API FSdfChangeBlock();
		UE_API ~FSdfChangeBlock();

	private:
		TUniquePtr<Internal::FSdfChangeBlockImpl> Impl;
	};
}

#undef UE_API
