// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"

#include "UsdWrappers/ForwardDeclarations.h"
#include "UsdWrappers/SdfPath.h"

#define UE_API UNREALUSDWRAPPER_API

namespace UE
{
	class FSdfPrimSpec;

	namespace Internal
	{
		template<typename PtrType>
		class FSdfLayerImpl;
	}

	struct FSdfLayerOffset
	{
		FSdfLayerOffset() = default;

		FSdfLayerOffset(double InOffset, double InScale)
			: Offset(InOffset)
			, Scale(InScale)
		{
		}

		double Offset = 0.0;
		double Scale = 1.0;
	};

	struct FSdfPayload
	{
		FSdfPayload(const FString& InAssetPath = {}, const FSdfPath& InPrimPath = {}, const FSdfLayerOffset& InLayerOffset = {})
			: AssetPath(InAssetPath)
			, PrimPath(InPrimPath)
			, LayerOffset(InLayerOffset)
		{
		}

		FString AssetPath;
		FSdfPath PrimPath;
		FSdfLayerOffset LayerOffset;
	};

	// In the USD API, pxr::SdfPayload and pxr::SdfReference are very similar
	// except that the latter includes support for a custom data dictionary.
	// The UE wrapping of SdfReference does not currently include support for
	// that custom data, leaving the FSdfReference wrapper identical to the
	// FSdfPayload wrapper. We keep them as distinct types though to better
	// mimic the USD API and in case support for custom data dictionaries on
	// references is added to the FSdfReference wrapper in the future.
	struct FSdfReference
	{
		FSdfReference(const FString& InAssetPath = {}, const FSdfPath& InPrimPath = {}, const FSdfLayerOffset& InLayerOffset = {})
			: AssetPath(InAssetPath)
			, PrimPath(InPrimPath)
			, LayerOffset(InLayerOffset)
		{
		}

		FString AssetPath;
		FSdfPath PrimPath;
		FSdfLayerOffset LayerOffset;
	};

	/**
	 * Minimal pxr::SdfLayer pointer wrapper for Unreal that can be used from no-rtti modules.
	 * Use the aliases FSdfLayer and FSdfLayerWeak instead (defined on ForwardDeclarations.h)
	 */
	template<typename PtrType>
	class FSdfLayerBase
	{
	public:
		FSdfLayerBase();

		FSdfLayerBase(const FSdfLayer& Other);
		FSdfLayerBase(FSdfLayer&& Other);
		FSdfLayerBase(const FSdfLayerWeak& Other);
		FSdfLayerBase(FSdfLayerWeak&& Other);

		~FSdfLayerBase();

		FSdfLayerBase& operator=(const FSdfLayer& Other);
		FSdfLayerBase& operator=(FSdfLayer&& Other);
		FSdfLayerBase& operator=(const FSdfLayerWeak& Other);
		FSdfLayerBase& operator=(FSdfLayerWeak&& Other);

		// We have to implement a templated comparison operator, or else we get ambigous
		// conversions on the Mac targets when comparing FSdfLayer and FSdfLayerWeak
		template<typename OtherPtrType>
		bool operator==(const FSdfLayerBase<OtherPtrType>& Other) const;

		explicit operator bool() const;

		friend UNREALUSDWRAPPER_API uint32 GetTypeHash(const FSdfLayerWeak& Layer);

		// Auto conversion from/to PtrType. We use concrete pointer types here
		// because we should also be able to convert between them
	public:
#if USE_USD_SDK
		explicit FSdfLayerBase(const pxr::SdfLayerRefPtr& InSdfLayer);
		explicit FSdfLayerBase(pxr::SdfLayerRefPtr&& InSdfLayer);
		explicit FSdfLayerBase(const pxr::SdfLayerWeakPtr& InSdfLayer);
		explicit FSdfLayerBase(pxr::SdfLayerWeakPtr&& InSdfLayer);

		FSdfLayerBase& operator=(const pxr::SdfLayerRefPtr& InSdfLayer);
		FSdfLayerBase& operator=(pxr::SdfLayerRefPtr&& InSdfLayer);
		FSdfLayerBase& operator=(const pxr::SdfLayerWeakPtr& InSdfLayer);
		FSdfLayerBase& operator=(pxr::SdfLayerWeakPtr&& InSdfLayer);

		// We can provide reference cast operators for the type we do have, but
		// need to settle on providing copy cast operators for the other types
		operator PtrType&();
		operator const PtrType&() const;

		operator pxr::SdfLayerRefPtr() const;
		operator pxr::SdfLayerWeakPtr() const;
#endif	  // #if USE_USD_SDK

		  // Wrapped pxr::SdfLayer functions, refer to the USD SDK documentation
	public:
		FString GetComment() const;
		void SetComment(const TCHAR* Comment) const;

		void TransferContent(const FSdfLayer& SourceLayer);

		static TSet<FSdfLayerWeak> GetLoadedLayers();
		static FSdfLayer FindOrOpen(const TCHAR* Identifier, const TMap<FString, FString>& FileFormatArguments = {});
		static FSdfLayer CreateNew(const TCHAR* Identifier, const TMap<FString, FString>& FileFormatArguments = {});

		TMap<FString, FString> GetFileFormatArguments() const;

		bool Save(bool bForce = false) const;

		TSet<FString> GetCompositionAssetDependencies() const;

		bool UpdateCompositionAssetDependency(const TCHAR* OldAssetPath, const TCHAR* NewAssetPath = nullptr);

		FString GetRealPath() const;
		FString GetIdentifier() const;
		FString GetDisplayName() const;

		FString ComputeAbsolutePath(const FString& AssetPath) const;

		bool IsDirty() const;
		bool IsEmpty() const;
		bool IsAnonymous() const;

		bool Export(const TCHAR* Filename, const FString& Comment = {}, const TMap<FString, FString>& FileFormatArguments = {}) const;

		void Clear();

		void Reload(bool bForce = false);

		bool HasStartTimeCode() const;
		double GetStartTimeCode() const;
		void SetStartTimeCode(double TimeCode);

		bool HasEndTimeCode() const;
		double GetEndTimeCode() const;
		void SetEndTimeCode(double TimeCode);

		bool HasTimeCodesPerSecond() const;
		double GetTimeCodesPerSecond() const;
		void SetTimeCodesPerSecond(double TimeCodesPerSecond);

		bool HasFramesPerSecond() const;
		double GetFramesPerSecond() const;
		void SetFramesPerSecond(double FramesPerSecond);

		TArray<FString> GetSubLayerPaths() const;
		void SetSubLayerPaths(const TArray<FString>& NewPaths);
		int64 GetNumSubLayerPaths() const;
		void InsertSubLayerPath(const FString& Path, int32 Index = -1);
		void RemoveSubLayerPath(int32 Index);
		TArray<FSdfLayerOffset> GetSubLayerOffsets() const;
		FSdfLayerOffset GetSubLayerOffset(int32 Index) const;
		void SetSubLayerOffset(const FSdfLayerOffset& Offset, int32 Index);

		bool HasSpec(const FSdfPath& Path) const;

		FSdfPrimSpec GetPseudoRoot() const;
		FSdfPrimSpec GetPrimAtPath(const FSdfPath& Path) const;

		TSet<double> ListTimeSamplesForPath(const FSdfPath& Path) const;
		void EraseTimeSample(const FSdfPath& Path, double Time);

		bool IsMuted() const;
		void SetMuted(bool bMuted);

	private:
		friend FSdfLayer;
		friend FSdfLayerWeak;

		TUniquePtr<Internal::FSdfLayerImpl<PtrType>> Impl;
	};

	/**
	 * Wrapper for global functions in pxr/usd/sdf/layerUtils.h
	 */
	class FSdfLayerUtils
	{
	public:
		static UE_API FString SdfComputeAssetPathRelativeToLayer(const FSdfLayer& Anchor, const TCHAR* AssetPath);
	};
}	 // namespace UE

#undef UE_API
