// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class SdfPath;
PXR_NAMESPACE_CLOSE_SCOPE
#endif	  // #if USE_USD_SDK

#define UE_API UNREALUSDWRAPPER_API

namespace UE
{
	namespace Internal
	{
		class FSdfPathImpl;
	}

	/**
	 * Minimal pxr::SdfPath wrapper for Unreal that can be used from no-rtti modules.
	 */
	class FSdfPath
	{
	public:
		UE_API FSdfPath();
		UE_API explicit FSdfPath(const TCHAR* InPath);

		UE_API FSdfPath(const FSdfPath& Other);
		UE_API FSdfPath(FSdfPath&& Other);

		UE_API FSdfPath& operator=(const FSdfPath& Other);
		UE_API FSdfPath& operator=(FSdfPath&& Other);

		UE_API ~FSdfPath();

		UE_API bool operator==(const FSdfPath& Other) const;
		UE_API bool operator!=(const FSdfPath& Other) const;
		UE_API bool operator<(const FSdfPath& Other) const;
		UE_API bool operator<=(const FSdfPath& Other) const;

		friend UE_API uint32 GetTypeHash(const UE::FSdfPath& Path);
		friend UE_API FArchive& operator<<(FArchive& Ar, FSdfPath& Path);

		// Auto conversion from/to pxr::SdfPath
	public:
#if USE_USD_SDK
		UE_API explicit FSdfPath(const pxr::SdfPath& InSdfPath);
		UE_API explicit FSdfPath(pxr::SdfPath&& InSdfPath);
		UE_API FSdfPath& operator=(const pxr::SdfPath& InSdfPath);
		UE_API FSdfPath& operator=(pxr::SdfPath&& InSdfPath);

		UE_API operator pxr::SdfPath&();
		UE_API operator const pxr::SdfPath&() const;
#endif	  // #if USE_USD_SDK

		  // Wrapped pxr::SdfPath functions, refer to the USD SDK documentation
	public:
		static UE_API const FSdfPath& AbsoluteRootPath();

		// Returns true if PathString is a valid path string, meaning that passing the string
		// to the FSdfPath constructor will result in a valid, non-empty FSdfPath.
		// Otherwise, returns false and, if OutErrorMessage is not null, sets it to the parse error.
		static UE_API bool IsValidPathString(const TCHAR* PathString, FString* OutErrorMessage = nullptr);

		UE_API bool IsEmpty() const noexcept;

		UE_API bool IsAbsoluteRootPath() const;
		UE_API bool IsPrimPath() const;
		UE_API bool IsAbsoluteRootOrPrimPath() const;
		UE_API bool IsPrimPropertyPath() const;
		UE_API bool IsPropertyPath() const;
		UE_API bool IsRelationalAttributePath() const;
		UE_API bool IsTargetPath() const;
		UE_API FString GetName() const;
		UE_API FString GetElementString() const;
		UE_API FSdfPath GetAbsoluteRootOrPrimPath() const;

		UE_API FSdfPath ReplaceName(const TCHAR* NewLeafName) const;
		UE_API FSdfPath ReplacePrefix(const UE::FSdfPath& OldPrefix, 
			                          const UE::FSdfPath& NewPrefix, 
			                          bool FixTargetPaths = true) const;

		UE_API FSdfPath GetParentPath() const;
		UE_API FSdfPath GetPrimPath() const;
		UE_API FSdfPath AppendPath(const UE::FSdfPath& NewRelativeSuffix) const;
		UE_API FSdfPath AppendChild(const TCHAR* ChildName) const;
		UE_API FSdfPath AppendProperty(FName PropertyName) const;

		UE_API FSdfPath StripAllVariantSelections() const;

		UE_API FString GetString() const;

		UE_API TArray<FSdfPath> GetPrefixes() const;
		UE_API int32 GetPathElementCount() const;

		UE_API bool HasPrefix(const UE::FSdfPath& Prefix) const;

		UE_API FSdfPath MakeAbsolutePath(const FSdfPath& Anchor) const;
		UE_API FSdfPath MakeRelativePath(const FSdfPath& Anchor) const;

	private:
		TUniquePtr<Internal::FSdfPathImpl> Impl;
	};
}	 // namespace UE

#undef UE_API