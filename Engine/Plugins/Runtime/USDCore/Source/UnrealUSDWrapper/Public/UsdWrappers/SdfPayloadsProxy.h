// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UniquePtr.h"

#if USE_USD_SDK

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class SdfPayloadTypePolicy;
	template<typename T>
	class SdfListEditorProxy;
	using SdfPayloadsProxy = SdfListEditorProxy<SdfPayloadTypePolicy>;
PXR_NAMESPACE_CLOSE_SCOPE

#endif	  // #if USE_USD_SDK

namespace UE
{
	struct FSdfPayload;
	namespace Internal
	{
		class FSdfPayloadsProxyImpl;
	}

	/**
	 * Minimal pxr::SdfPayloadsProxy wrapper for Unreal that can be used from no-rtti modules.
	 */
	class UNREALUSDWRAPPER_API FSdfPayloadsProxy
	{
	public:
		FSdfPayloadsProxy();

		FSdfPayloadsProxy(const FSdfPayloadsProxy& Other);
		FSdfPayloadsProxy(FSdfPayloadsProxy&& Other);
		~FSdfPayloadsProxy();

		FSdfPayloadsProxy& operator=(const FSdfPayloadsProxy& Other);
		FSdfPayloadsProxy& operator=(FSdfPayloadsProxy&& Other);

		explicit operator bool() const;

		// Auto conversion from/to pxr::SdfPayloadsProxy
	public:
#if USE_USD_SDK
		explicit FSdfPayloadsProxy(const pxr::SdfPayloadsProxy& InSdfPayloadsProxy);
		explicit FSdfPayloadsProxy(pxr::SdfPayloadsProxy&& InSdfPayloadsProxy);
		FSdfPayloadsProxy& operator=(const pxr::SdfPayloadsProxy& InSdfPayloadsProxy);
		FSdfPayloadsProxy& operator=(pxr::SdfPayloadsProxy&& InSdfPayloadsProxy);

		operator pxr::SdfPayloadsProxy&();
		operator const pxr::SdfPayloadsProxy&() const;
#endif	  // #if USE_USD_SDK

		  // Wrapped pxr::SdfPayloadsProxy functions, refer to the USD SDK documentation
	public:
		TArray<FSdfPayload> GetAppliedItems() const;

	private:
		TUniquePtr<Internal::FSdfPayloadsProxyImpl> Impl;
	};
}	 // namespace UE
