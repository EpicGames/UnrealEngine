// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"

#include "UsdWrappers/ForwardDeclarations.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class SdfPrimSpec;
	template<class T>
	class SdfHandle;
	using SdfPrimSpecHandle = SdfHandle<SdfPrimSpec>;
PXR_NAMESPACE_CLOSE_SCOPE
#endif	  // #if USE_USD_SDK

namespace UE
{
	class FSdfAttributeSpec;
	class FSdfPath;
	class FSdfPayloadsProxy;
	class FSdfReferencesProxy;

	namespace Internal
	{
		class FSdfPrimSpecImpl;
	}

	/** Corresponds to pxr::SdfSpecType, refer to the USD SDK documentation */
	enum class ESdfSpecType
	{
		SdfSpecTypeUnknown = 0,
		SdfSpecTypeAttribute,
		SdfSpecTypeConnection,
		SdfSpecTypeExpression,
		SdfSpecTypeMapper,
		SdfSpecTypeMapperArg,
		SdfSpecTypePrim,
		SdfSpecTypePseudoRoot,
		SdfSpecTypeRelationship,
		SdfSpecTypeRelationshipTarget,
		SdfSpecTypeVariant,
		SdfSpecTypeVariantSet,

		SdfNumSpecTypes
	};

	/**
	 * Minimal pxr::SdfPrimSpecHandle wrapper for Unreal that can be used from no-rtti modules.
	 */
	class UNREALUSDWRAPPER_API FSdfPrimSpec
	{
	public:
		FSdfPrimSpec();

		FSdfPrimSpec(const FSdfPrimSpec& Other);
		FSdfPrimSpec(FSdfPrimSpec&& Other);
		~FSdfPrimSpec();

		FSdfPrimSpec& operator=(const FSdfPrimSpec& Other);
		FSdfPrimSpec& operator=(FSdfPrimSpec&& Other);

		bool operator==(const FSdfPrimSpec& Other) const;
		bool operator!=(const FSdfPrimSpec& Other) const;

		explicit operator bool() const;

		// Auto conversion from/to pxr::SdfPrimSpecHandle
	public:
#if USE_USD_SDK
		explicit FSdfPrimSpec(const pxr::SdfPrimSpecHandle& InSdfPrimSpec);
		explicit FSdfPrimSpec(pxr::SdfPrimSpecHandle&& InSdfPrimSpec);
		FSdfPrimSpec& operator=(const pxr::SdfPrimSpecHandle& InSdfPrimSpec);
		FSdfPrimSpec& operator=(pxr::SdfPrimSpecHandle&& InSdfPrimSpec);

		operator pxr::SdfPrimSpecHandle&();
		operator const pxr::SdfPrimSpecHandle&() const;
#endif	  // #if USE_USD_SDK

		  // Wrapped pxr::SdfPrimSpecHandle functions, refer to the USD SDK documentation
	public:
		ESdfSpecType GetSpecType() const;
		FSdfLayerWeak GetLayer() const;
		FSdfPath GetPath() const;

		FSdfPrimSpec GetRealNameParent() const;
		bool RemoveNameChild(const FSdfPrimSpec& Child);

		UE::FSdfAttributeSpec GetAttributeAtPath(const UE::FSdfPath& Path) const;

		FSdfPayloadsProxy GetPayloadList() const;
		FSdfReferencesProxy GetReferenceList() const;

		FName GetTypeName() const;
		FName GetName() const;

	private:
		TUniquePtr<Internal::FSdfPrimSpecImpl> Impl;
	};
}	 // namespace UE
