// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UniquePtr.h"

#if USE_USD_SDK

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class SdfReferenceTypePolicy;
	template<typename T>
	class SdfListEditorProxy;
	using SdfReferencesProxy = SdfListEditorProxy<SdfReferenceTypePolicy>;
PXR_NAMESPACE_CLOSE_SCOPE

#endif	  // #if USE_USD_SDK

namespace UE
{
	struct FSdfReference;
	namespace Internal
	{
		class FSdfReferencesProxyImpl;
	}

	/**
	 * Minimal pxr::SdfReferencesProxy wrapper for Unreal that can be used from no-rtti modules.
	 */
	class UNREALUSDWRAPPER_API FSdfReferencesProxy
	{
	public:
		FSdfReferencesProxy();

		FSdfReferencesProxy(const FSdfReferencesProxy& Other);
		FSdfReferencesProxy(FSdfReferencesProxy&& Other);
		~FSdfReferencesProxy();

		FSdfReferencesProxy& operator=(const FSdfReferencesProxy& Other);
		FSdfReferencesProxy& operator=(FSdfReferencesProxy&& Other);

		explicit operator bool() const;

		// Auto conversion from/to pxr::SdfReferencesProxy
	public:
#if USE_USD_SDK
		explicit FSdfReferencesProxy(const pxr::SdfReferencesProxy& InSdfReferencesProxy);
		explicit FSdfReferencesProxy(pxr::SdfReferencesProxy&& InSdfReferencesProxy);
		FSdfReferencesProxy& operator=(const pxr::SdfReferencesProxy& InSdfReferencesProxy);
		FSdfReferencesProxy& operator=(pxr::SdfReferencesProxy&& InSdfReferencesProxy);

		operator pxr::SdfReferencesProxy&();
		operator const pxr::SdfReferencesProxy&() const;
#endif	  // #if USE_USD_SDK

		  // Wrapped pxr::SdfReferencesProxy functions, refer to the USD SDK documentation
	public:
		TArray<FSdfReference> GetAppliedItems() const;

	private:
		TUniquePtr<Internal::FSdfReferencesProxyImpl> Impl;
	};
}	 // namespace UE
