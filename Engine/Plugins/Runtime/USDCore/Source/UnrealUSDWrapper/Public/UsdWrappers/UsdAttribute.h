// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Optional.h"
#include "Templates/UniquePtr.h"
#include "Containers/StringView.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class UsdAttribute;
	class UsdProperty;
PXR_NAMESPACE_CLOSE_SCOPE
#endif	  // #if USE_USD_SDK

namespace UE
{
	class FSdfPath;
	class FUsdPrim;
	class FVtValue;
	class FUsdResolveInfo;

	namespace Internal
	{
		class FUsdAttributeImpl;
	}

	/**
	 * Minimal pxr::UsdAttribute wrapper for Unreal that can be used from no-rtti modules.
	 */
	class UNREALUSDWRAPPER_API FUsdAttribute
	{
	public:
		FUsdAttribute();

		FUsdAttribute(const FUsdAttribute& Other);
		FUsdAttribute(FUsdAttribute&& Other);
		~FUsdAttribute();

		FUsdAttribute& operator=(const FUsdAttribute& Other);
		FUsdAttribute& operator=(FUsdAttribute&& Other);

		bool operator==(const FUsdAttribute& Other) const;
		bool operator!=(const FUsdAttribute& Other) const;

		explicit operator bool() const;

		// Auto conversion from/to pxr::UsdAttribute
	public:
#if USE_USD_SDK
		explicit FUsdAttribute(const pxr::UsdAttribute& InUsdAttribute);
		explicit FUsdAttribute(pxr::UsdAttribute&& InUsdAttribute);
		FUsdAttribute& operator=(const pxr::UsdAttribute& InUsdAttribute);
		FUsdAttribute& operator=(pxr::UsdAttribute&& InUsdAttribute);

		operator pxr::UsdAttribute&();
		operator const pxr::UsdAttribute&() const;

		// Also define the conversion operators directly to the base class as we can't "forward declare the class
		// inheritance", but it is nice to be able to pass an UE::FUsdAttribute to a function that takes an
		// pxr::UsdProperty directly
		operator pxr::UsdProperty&();
		operator const pxr::UsdProperty&() const;
#endif	  // #if USE_USD_SDK

		  // Wrapped pxr::UsdObject functions, refer to the USD SDK documentation
	public:
		bool GetMetadata(const TCHAR* Key, UE::FVtValue& Value) const;
		bool HasMetadata(const TCHAR* Key) const;
		bool SetMetadata(const TCHAR* Key, const UE::FVtValue& Value) const;
		bool ClearMetadata(const TCHAR* Key) const;

		// Wrapped pxr::UsdAttribute functions, refer to the USD SDK documentation
	public:
		FName GetName() const;
		FName GetBaseName() const;
		FName GetTypeName() const;
		FString GetCPPTypeName() const;

		bool GetTimeSamples(TArray<double>& Times) const;
		size_t GetNumTimeSamples() const;

		bool HasValue() const;
		bool HasAuthoredValue() const;
		bool HasFallbackValue() const;

		bool ValueMightBeTimeVarying() const;

		bool Get(UE::FVtValue& Value, TOptional<double> Time = {}) const;

		FUsdResolveInfo GetResolveInfo(double Time) const;

		bool Set(const UE::FVtValue& Value, TOptional<double> Time = {}) const;

		template <typename T>
		bool Get(T& Value, TOptional<double> Time = {}) const;

		bool Clear() const;
		bool ClearAtTime(double Time) const;

		bool ClearConnections() const;

		static bool GetUnionedTimeSamples(const TArray<UE::FUsdAttribute>& Attrs, TArray<double>& OutTimes);

		FSdfPath GetPath() const;
		FUsdPrim GetPrim() const;

	private:
		TUniquePtr<Internal::FUsdAttributeImpl> Impl;
	};
}	 // namespace UE

namespace UsdUtils
{
	// Returns the attribute value for the given prim and attribute name. If the attribute doesn't exist, it returns the default value for the type.
	template<typename ValueType>
	UNREALUSDWRAPPER_API ValueType GetAttributeValue(const UE::FUsdPrim& Prim, FStringView AttributeName, TOptional<double> Time = {});

	// Outputs the authored attribute value for the given prim and attribute name, and returns true.
	// If the attribute doesn't exist, returns false and leaves OutValue unassigned.
	// If the attribute does exist but has no authored opinion, returns false but assigns OutValue with the attribute's fallback value, if possible.
	template<typename ValueType>
	UNREALUSDWRAPPER_API bool GetAuthoredAttributeValue(const UE::FUsdPrim& InPrim, FStringView InAttributeName, ValueType& OutValue, TOptional<double> InTime = {});
}