// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UsdWrappers/ForwardDeclarations.h"

#include "Templates/UniquePtr.h"

#define UE_API UNREALUSDWRAPPER_API

namespace UE
{
	namespace Internal
	{
		class FUsdEditContextImpl;
	}

	/**
	 * Minimal pxr::UsdEditContext wrapper for Unreal that can be used from no-rtti modules.
	 */
	class FUsdEditContext final
	{
		FUsdEditContext(const FUsdEditContext& Other) = delete;
		FUsdEditContext& operator=(const FUsdEditContext& Other) = delete;

	public:
		UE_API explicit FUsdEditContext(const FUsdStageWeak& Stage);
		UE_API FUsdEditContext(const FUsdStageWeak& Stage, const FSdfLayer& EditTarget);
		UE_API ~FUsdEditContext();

	private:
		TUniquePtr<Internal::FUsdEditContextImpl> Impl;
	};
}

#undef UE_API
