// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/CriticalSection.h"
#include "Templates/UniquePtr.h"

#if USE_USD_SDK

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class UsdPrim;
	class UsdGeomBBoxCache;
PXR_NAMESPACE_CLOSE_SCOPE

#endif	  // #if USE_USD_SDK

enum class EUsdPurpose : int32;

namespace UE
{
	namespace Internal
	{
		class FUsdGeomBBoxCacheImpl;
	}

	/**
	 * Minimal pxr::UsdGeomBBoxCache wrapper for Unreal that can be used from no-rtti modules.
	 * Unlike the other wrapper types, FUsdGeomBBoxCache contains its own mutex in order to try
	 * and make its usage somewhat thread-safe. The wrapped functions will use this mutex already,
	 * and the conversion operators that return the underlying type will check that the mutex is
	 * locked before returning (the caller is expected to lock the mutex before using the operators).
	 */
	class FUsdGeomBBoxCache
	{
	public:
		UNREALUSDWRAPPER_API FUsdGeomBBoxCache(double Time, EUsdPurpose IncludedPurposeFlags, bool bUseExtentsHint = false, bool bIgnoreVisibility = false);
		UNREALUSDWRAPPER_API FUsdGeomBBoxCache(const FUsdGeomBBoxCache& Other);
		UNREALUSDWRAPPER_API FUsdGeomBBoxCache(FUsdGeomBBoxCache&& Other);

		UNREALUSDWRAPPER_API FUsdGeomBBoxCache& operator=(const FUsdGeomBBoxCache& Other);
		UNREALUSDWRAPPER_API FUsdGeomBBoxCache& operator=(FUsdGeomBBoxCache&& Other);

		UNREALUSDWRAPPER_API ~FUsdGeomBBoxCache();

		// Auto conversion from/to pxr::UsdGeomBBoxCache
	public:
#if USE_USD_SDK
		UNREALUSDWRAPPER_API explicit FUsdGeomBBoxCache(const pxr::UsdGeomBBoxCache& InUsdGeomBBoxCache);
		UNREALUSDWRAPPER_API explicit FUsdGeomBBoxCache(pxr::UsdGeomBBoxCache&& InUsdGeomBBoxCache);

		UNREALUSDWRAPPER_API operator pxr::UsdGeomBBoxCache&();
		UNREALUSDWRAPPER_API operator const pxr::UsdGeomBBoxCache&() const;
#endif	  // #if USE_USD_SDK

		  // Wrapped pxr::UsdGeomBBoxCache functions, refer to the USD SDK documentation
	public:
		UNREALUSDWRAPPER_API void Clear();

		UNREALUSDWRAPPER_API void SetIncludedPurposes(EUsdPurpose IncludedPurposeFlags);
		UNREALUSDWRAPPER_API EUsdPurpose GetIncludedPurposes() const;

		UNREALUSDWRAPPER_API bool GetUseExtentsHint() const;
		UNREALUSDWRAPPER_API bool GetIgnoreVisibility() const;

		UNREALUSDWRAPPER_API void SetTime(double UsdTimeCode);
		UNREALUSDWRAPPER_API double GetTime() const;

	public:
		// The underlying pxr::UsdGeomBBoxCache object is not thread-safe, so we have this.
		// This mutex will be internally locked for all of the wrapped function calls.
		// It is exposed so that it can be locked by the user in case it is desirable to use
		// the underlying pxr::UsdGeomBBoxCache type directly
		mutable FRWLock Lock;

	private:
		TUniquePtr<Internal::FUsdGeomBBoxCacheImpl> Impl;
	};
}	 // namespace UE
