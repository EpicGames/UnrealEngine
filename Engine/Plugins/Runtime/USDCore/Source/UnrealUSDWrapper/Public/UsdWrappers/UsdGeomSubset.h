// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UniquePtr.h"
#include "UsdWrappers/UsdAttribute.h"

#if USE_USD_SDK
PXR_NAMESPACE_OPEN_SCOPE
	class UsdGeomSubset;
PXR_NAMESPACE_CLOSE_SCOPE
#endif	  // #if USE_USD_SDK

namespace UE
{
	class FUsdPrim;

	namespace Internal
	{
		class FUsdGeomSubsetImpl;
	}

	/**
	 * Minimal pxr::UsdGeomSubset wrapper for Unreal that can be used from no-rtti modules.
	 */
	class UNREALUSDWRAPPER_API FUsdGeomSubset
	{
	public:
		FUsdGeomSubset();
		FUsdGeomSubset(const FUsdPrim& UsdPrim);

		FUsdGeomSubset(const FUsdGeomSubset& Other);
		FUsdGeomSubset(FUsdGeomSubset&& Other);
		~FUsdGeomSubset();

		FUsdGeomSubset& operator=(const FUsdGeomSubset& Other);
		FUsdGeomSubset& operator=(FUsdGeomSubset&& Other);

		explicit operator bool() const;

		// Auto conversion from/to pxr::FUsdGeomSubset
	public:
#if USE_USD_SDK
		explicit FUsdGeomSubset(const pxr::UsdGeomSubset& InUsdGeomSubset);
		explicit FUsdGeomSubset(pxr::UsdGeomSubset&& InUsdGeomSubset);
		FUsdGeomSubset& operator=(const pxr::UsdGeomSubset& InUsdGeomSubset);
		FUsdGeomSubset& operator=(pxr::UsdGeomSubset&& InUsdGeomSubset);

		operator pxr::UsdGeomSubset&();
		operator const pxr::UsdGeomSubset&() const;
#endif	  // #if USE_USD_SDK

		// Wrapped pxr::UsdGeomSubset functions, refer to the USD SDK documentation
	public:
		FUsdPrim GetPrim() const;

		FUsdAttribute GetElementTypeAttr() const;

		FUsdAttribute GetFamilyNameAttr() const;

		FUsdAttribute GetIndicesAttr() const;

	private:
		TUniquePtr<Internal::FUsdGeomSubsetImpl> Impl;
	};
}	 // namespace UE
