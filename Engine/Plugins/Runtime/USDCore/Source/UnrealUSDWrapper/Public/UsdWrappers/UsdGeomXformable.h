// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UsdTyped.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class UsdPrim;
	class UsdGeomXformable;
PXR_NAMESPACE_CLOSE_SCOPE
#endif	  // #if USE_USD_SDK

namespace UE
{
	class FUsdAttribute;

	namespace Internal
	{
		class FUsdGeomXformableImpl;
	}

	/**
	 * Minimal pxr::UsdGeomXformable wrapper for Unreal that can be used from no-rtti modules.
	 */
	class UNREALUSDWRAPPER_API FUsdGeomXformable : public FUsdTyped
	{
	public:
		FUsdGeomXformable();
		FUsdGeomXformable(const FUsdGeomXformable& Other);
		FUsdGeomXformable(FUsdGeomXformable&& Other);

		explicit FUsdGeomXformable(const FUsdPrim& Prim);

		FUsdGeomXformable& operator=(const FUsdGeomXformable& Other);
		FUsdGeomXformable& operator=(FUsdGeomXformable&& Other);

		~FUsdGeomXformable();

		explicit operator bool() const;

		// Auto conversion from/to pxr::UsdGeomXformable
	public:
#if USE_USD_SDK
		explicit FUsdGeomXformable(const pxr::UsdGeomXformable& InUsdGeomXformable);
		explicit FUsdGeomXformable(pxr::UsdGeomXformable&& InUsdGeomXformable);
		explicit FUsdGeomXformable(const pxr::UsdPrim& Prim);

		operator pxr::UsdGeomXformable&();
		operator const pxr::UsdGeomXformable&() const;
#endif	  // #if USE_USD_SDK

		  // Wrapped pxr::UsdGeomXformable functions, refer to the USD SDK documentation
	public:
		bool GetResetXformStack() const;

		bool TransformMightBeTimeVarying() const;
		bool GetTimeSamples(TArray<double>* Times) const;

		FUsdAttribute GetXformOpOrderAttr() const;
		bool ClearXformOpOrder() const;

	private:
		TUniquePtr<Internal::FUsdGeomXformableImpl> Impl;
	};
}	 // namespace UE
