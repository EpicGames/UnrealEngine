// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Templates/UniquePtr.h"

#include "UnrealUSDWrapper.h"
#include "UsdWrappers/SdfLayer.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class UsdPayloads;
	class UsdPrim;
PXR_NAMESPACE_CLOSE_SCOPE
#endif	  // #if USE_USD_SDK

namespace UE
{
	class FSdfPath;
	class FUsdPrim;

	namespace Internal
	{
		class FUsdPayloadsImpl;
	}

	/**
	 * Minimal pxr::UsdPayloads wrapper for Unreal that can be used from no-rtti modules.
	 */
	class UNREALUSDWRAPPER_API FUsdPayloads
	{
	public:
		FUsdPayloads();
		FUsdPayloads(const FUsdPayloads& Other);
		FUsdPayloads(FUsdPayloads&& Other);

		FUsdPayloads& operator=(const FUsdPayloads& Other);
		FUsdPayloads& operator=(FUsdPayloads&& Other);

		~FUsdPayloads();

		explicit operator bool() const;

#if USE_USD_SDK
		// Auto conversion from/to pxr::UsdPayloads
		explicit FUsdPayloads(const pxr::UsdPayloads& InUsdPayloads);
		explicit FUsdPayloads(pxr::UsdPayloads&& InUsdPayloads);
		FUsdPayloads& operator=(const pxr::UsdPayloads& InUsdPayloads);
		FUsdPayloads& operator=(pxr::UsdPayloads&& InUsdPayloads);

		operator pxr::UsdPayloads();
		operator const pxr::UsdPayloads() const;

	private:
		// FUsdPayloads instances are only constructable by instances of
		// FUsdPrim.
		// Use FUsdPrim::GetPayloads() to construct an instance.
		explicit FUsdPayloads(const pxr::UsdPrim& InPrim);
#endif	  // #if USE_USD_SDK

		  // Wrapped pxr::UsdPayloads functions, refer to the USD SDK documentation
	public:
		bool AddPayload(const FSdfPayload& Payload, EUsdListPosition Position = EUsdListPosition::BackOfPrependList);

		bool AddPayload(
			const FString& Identifier,
			const FSdfPath& PrimPath,
			const FSdfLayerOffset& LayerOffset = {},
			EUsdListPosition Position = EUsdListPosition::BackOfPrependList
		);

		bool AddPayload(
			const FString& Identifier,
			const FSdfLayerOffset& LayerOffset = {},
			EUsdListPosition Position = EUsdListPosition::BackOfPrependList
		);

		bool AddInternalPayload(
			const FSdfPath& PrimPath,
			const FSdfLayerOffset& LayerOffset = {},
			EUsdListPosition Position = EUsdListPosition::BackOfPrependList
		);

		bool RemovePayload(const FSdfPayload& Payload);

		bool ClearPayloads();

		bool SetPayloads(const TArray<FSdfPayload>& Items);

		FUsdPrim GetPrim() const;

	private:
		TUniquePtr<Internal::FUsdPayloadsImpl> Impl;

		friend class FUsdPrim;
	};
}	 // namespace UE
