// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Misc/Optional.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"

#include "UnrealUSDWrapper.h"
#include "UsdWrappers/ForwardDeclarations.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class UsdPrim;
PXR_NAMESPACE_CLOSE_SCOPE
#endif	  // #if USE_USD_SDK

namespace UE
{
	class FPcpPrimIndex;
	class FSdfPath;
	class FSdfPrimSpec;
	class FUsdAttribute;
	class FUsdPayloads;
	class FUsdReferences;
	class FUsdRelationship;
	class FUsdVariantSet;
	class FUsdVariantSets;
	class FVtValue;

	namespace Internal
	{
		class FUsdPrimImpl;
	}

	/** Corresponds to pxr::SdfSpecifier, refer to the USD SDK documentation */
	enum class ESdfSpecifier
	{
		Def,	  // Defines a concrete prim
		Over,	  // Overrides an existing prim
		Class,	  // Defines an abstract prim
		Num		  // The number of specifiers
	};

	/**
	 * Minimal pxr::UsdPrim wrapper for Unreal that can be used from no-rtti modules.
	 */
	class UNREALUSDWRAPPER_API FUsdPrim
	{
	public:
		FUsdPrim();

		FUsdPrim(const FUsdPrim& Other);
		FUsdPrim(FUsdPrim&& Other);
		~FUsdPrim();

		FUsdPrim& operator=(const FUsdPrim& Other);
		FUsdPrim& operator=(FUsdPrim&& Other);

		bool operator==(const FUsdPrim& Other) const;
		bool operator!=(const FUsdPrim& Other) const;

		explicit operator bool() const;

		// Auto conversion from/to pxr::UsdPrim
	public:
#if USE_USD_SDK
		explicit FUsdPrim(const pxr::UsdPrim& InUsdPrim);
		explicit FUsdPrim(pxr::UsdPrim&& InUsdPrim);
		FUsdPrim& operator=(const pxr::UsdPrim& InUsdPrim);
		FUsdPrim& operator=(pxr::UsdPrim&& InUsdPrim);

		operator pxr::UsdPrim&();
		operator const pxr::UsdPrim&() const;
#endif	  // #if USE_USD_SDK

		  // Wrapped pxr::UsdObject functions, refer to the USD SDK documentation
	public:
		bool GetMetadata(const TCHAR* Key, UE::FVtValue& Value) const;
		bool HasMetadata(const TCHAR* Key) const;
		bool SetMetadata(const TCHAR* Key, const UE::FVtValue& Value) const;
		bool ClearMetadata(const TCHAR* Key) const;
		TMap<FString, UE::FVtValue> GetAllMetadata() const;
		TMap<FString, UE::FVtValue> GetAllAuthoredMetadata() const;

		FString GetDisplayName() const;
		bool SetDisplayName(const FString& Name) const;
		bool ClearDisplayName() const;
		bool HasAuthoredDisplayName() const;

		// Wrapped pxr::UsdPrim functions, refer to the USD SDK documentation
	public:
		bool SetSpecifier(ESdfSpecifier Specifier);

		bool IsActive() const;
		bool SetActive(bool bActive);

		bool IsValid() const;
		bool IsPseudoRoot() const;
		bool IsModel() const;
		bool IsDefined() const;
		bool IsGroup() const;

		TArray<FName> GetAppliedSchemas() const;

		bool IsA(FName SchemaIdentifier) const;
		bool HasAPI(FName SchemaIdentifier) const;
		bool HasAPI(FName SchemaIdentifier, FName InstanceName) const;

		bool CanApplyAPI(FName SchemaIdentifier, FString* OutWhyNot = nullptr) const;
		bool CanApplyAPI(FName SchemaIdentifier, FName InstanceName, FString* OutWhyNot = nullptr) const;

		bool ApplyAPI(FName SchemaIdentifier) const;
		bool ApplyAPI(FName SchemaIdentifier, FName InstanceName) const;

		bool RemoveAPI(FName SchemaIdentifier) const;
		bool RemoveAPI(FName SchemaIdentifier, FName InstanceName) const;

		const FSdfPath GetPrimPath() const;
		FUsdStage GetStage() const;

		FName GetName() const;

		FName GetTypeName() const;
		bool SetTypeName(FName TypeName) const;
		bool ClearTypeName() const;
		bool HasAuthoredTypeName() const;

		FUsdPrim GetParent() const;

		TArray<FUsdPrim> GetChildren() const;
		TArray<FUsdPrim> GetFilteredChildren(bool bTraverseInstanceProxies) const;

		FUsdVariantSets GetVariantSets() const;
		FUsdVariantSet GetVariantSet(const FString& VariantSetName) const;
		bool HasVariantSets() const;

		FUsdPayloads GetPayloads() const;
		bool HasAuthoredPayloads() const;
		bool IsLoaded() const;
		void Load(EUsdLoadPolicy Policy = EUsdLoadPolicy::UsdLoadWithDescendants);
		void Unload();

		FUsdReferences GetReferences() const;
		bool HasAuthoredReferences() const;

		TArray<FSdfPrimSpec> GetPrimStack() const;

		FPcpPrimIndex GetPrimIndex() const;
		FPcpPrimIndex ComputeExpandedPrimIndex() const;

		bool RemoveProperty(FName PropName) const;

		FUsdAttribute CreateAttribute(const TCHAR* AttrName, FName TypeName) const;
		TArray<FUsdAttribute> GetAttributes() const;
		FUsdAttribute GetAttribute(const TCHAR* AttrName) const;
		bool HasAttribute(const TCHAR* AttrName) const;

		FUsdRelationship CreateRelationship(const TCHAR* RelName, bool bCustom = true) const;
		TArray<FUsdRelationship> GetRelationships() const;
		FUsdRelationship GetRelationship(const TCHAR* RelationshipName) const;
		bool HasRelationship(const TCHAR* RelName) const;

		bool IsInstanceable() const;
		bool SetInstanceable(bool bInstanceable) const;
		bool ClearInstanceable() const;
		bool HasAuthoredInstanceable() const;
		bool IsInstance() const;
		bool IsInstanceProxy() const;
		bool IsPrototype() const;
		bool IsInPrototype() const;
		FUsdPrim GetPrototype() const;
		FUsdPrim GetPrimInPrototype() const;
		TArray<FUsdPrim> GetInstances() const;
		static bool IsPrototypePath(const FSdfPath& Path);
		static bool IsPathInPrototype(const FSdfPath& Path);

	private:
		TUniquePtr<Internal::FUsdPrimImpl> Impl;
	};
}	 // namespace UE
