// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Templates/UniquePtr.h"

#include "UnrealUSDWrapper.h"
#include "UsdWrappers/SdfLayer.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class UsdPrim;
	class UsdReferences;
PXR_NAMESPACE_CLOSE_SCOPE
#endif	  // #if USE_USD_SDK

namespace UE
{
	class FSdfPath;
	class FUsdPrim;

	namespace Internal
	{
		class FUsdReferencesImpl;
	}

	/**
	 * Minimal pxr::UsdReferences wrapper for Unreal that can be used from no-rtti modules.
	 */
	class UNREALUSDWRAPPER_API FUsdReferences
	{
	public:
		FUsdReferences();
		FUsdReferences(const FUsdReferences& Other);
		FUsdReferences(FUsdReferences&& Other);

		FUsdReferences& operator=(const FUsdReferences& Other);
		FUsdReferences& operator=(FUsdReferences&& Other);

		~FUsdReferences();

		explicit operator bool() const;

#if USE_USD_SDK
		// Auto conversion from/to pxr::UsdReferences
		explicit FUsdReferences(const pxr::UsdReferences& InUsdReferences);
		explicit FUsdReferences(pxr::UsdReferences&& InUsdReferences);
		FUsdReferences& operator=(const pxr::UsdReferences& InUsdReferences);
		FUsdReferences& operator=(pxr::UsdReferences&& InUsdReferences);

		operator pxr::UsdReferences();
		operator const pxr::UsdReferences() const;

	private:
		// FUsdReferences instances are only constructable by instances of
		// FUsdPrim.
		// Use FUsdPrim::GetReferences() to construct an instance.
		explicit FUsdReferences(const pxr::UsdPrim& InPrim);
#endif	  // #if USE_USD_SDK

		  // Wrapped pxr::UsdReferences functions, refer to the USD SDK documentation
	public:
		bool AddReference(const FSdfReference& Reference, EUsdListPosition Position = EUsdListPosition::BackOfPrependList);

		bool AddReference(
			const FString& Identifier,
			const FSdfPath& PrimPath,
			const FSdfLayerOffset& LayerOffset = {},
			EUsdListPosition Position = EUsdListPosition::BackOfPrependList
		);

		bool AddReference(
			const FString& Identifier,
			const FSdfLayerOffset& LayerOffset = {},
			EUsdListPosition Position = EUsdListPosition::BackOfPrependList
		);

		bool AddInternalReference(
			const FSdfPath& PrimPath,
			const FSdfLayerOffset& LayerOffset = {},
			EUsdListPosition Position = EUsdListPosition::BackOfPrependList
		);

		bool RemoveReference(const FSdfReference& Reference);

		bool ClearReferences();

		bool SetReferences(const TArray<FSdfReference>& Items);

		FUsdPrim GetPrim() const;

	private:
		TUniquePtr<Internal::FUsdReferencesImpl> Impl;

		friend class FUsdPrim;
	};
}	 // namespace UE
