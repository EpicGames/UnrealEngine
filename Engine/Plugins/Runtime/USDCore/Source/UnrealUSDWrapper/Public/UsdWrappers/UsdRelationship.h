// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UniquePtr.h"
#include "UnrealUSDWrapper.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class UsdRelationship;
	class UsdProperty;
PXR_NAMESPACE_CLOSE_SCOPE
#endif	  // #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
		class FUsdRelationshipImpl;
	}
	class FSdfPath;
	class FVtValue;

	/**
	 * Minimal pxr::UsdRelationship wrapper for Unreal that can be used from no-rtti modules.
	 */
	class UNREALUSDWRAPPER_API FUsdRelationship
	{
	public:
		FUsdRelationship();

		FUsdRelationship(const FUsdRelationship& Other);
		FUsdRelationship(FUsdRelationship&& Other);
		~FUsdRelationship();

		FUsdRelationship& operator=(const FUsdRelationship& Other);
		FUsdRelationship& operator=(FUsdRelationship&& Other);

		bool operator==(const FUsdRelationship& Other) const;
		bool operator!=(const FUsdRelationship& Other) const;

		explicit operator bool() const;

		// Auto conversion from/to pxr::UsdRelationship
	public:
#if USE_USD_SDK
		explicit FUsdRelationship(const pxr::UsdRelationship& InUsdRelationship);
		explicit FUsdRelationship(pxr::UsdRelationship&& InUsdRelationship);
		FUsdRelationship& operator=(const pxr::UsdRelationship& InUsdRelationship);
		FUsdRelationship& operator=(pxr::UsdRelationship&& InUsdRelationship);

		operator pxr::UsdRelationship&();
		operator const pxr::UsdRelationship&() const;

		// Also define the conversion operators directly to the base class as we can't "forward declare the class
		// inheritance", but it is nice to be able to pass an UE::FUsdRelationship to a function that takes an
		// pxr::UsdProperty directly
		operator pxr::UsdProperty&();
		operator const pxr::UsdProperty&() const;
#endif	  // #if USE_USD_SDK

		  // Wrapped pxr::UsdObject functions, refer to the USD SDK documentation
	public:
		bool GetMetadata(const TCHAR* Key, UE::FVtValue& Value) const;
		bool HasMetadata(const TCHAR* Key) const;
		bool SetMetadata(const TCHAR* Key, const UE::FVtValue& Value) const;
		bool ClearMetadata(const TCHAR* Key) const;

		// Wrapped pxr::UsdRelationship functions, refer to the USD SDK documentation
	public:
		bool SetTargets(const TArray<UE::FSdfPath>& Targets) const;
		bool ClearTargets(bool bRemoveSpec) const;
		bool GetTargets(TArray<UE::FSdfPath>& Targets) const;

	private:
		TUniquePtr<Internal::FUsdRelationshipImpl> Impl;
	};
}	 // namespace UE
