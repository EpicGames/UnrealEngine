// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/UniquePtr.h"

#if USE_USD_SDK

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class UsdResolveInfo;
PXR_NAMESPACE_CLOSE_SCOPE

#endif	  // #if USE_USD_SDK

namespace UE
{
	class FPcpNodeRef;
	namespace Internal
	{
		class FUsdResolveInfoImpl;
	}

	/**
	 * Minimal pxr::UsdResolveInfo wrapper for Unreal that can be used from no-rtti modules.
	 */
	class UNREALUSDWRAPPER_API FUsdResolveInfo
	{
	public:
		FUsdResolveInfo();
		FUsdResolveInfo(const FUsdResolveInfo& Other);
		FUsdResolveInfo(FUsdResolveInfo&& Other);

		~FUsdResolveInfo();

		// Auto conversion from/to pxr::UsdResolveInfo
	public:
#if USE_USD_SDK
		explicit FUsdResolveInfo(const pxr::UsdResolveInfo& InResolveInfo);
		explicit FUsdResolveInfo(pxr::UsdResolveInfo&& InResolveInfo);

		operator pxr::UsdResolveInfo&();
		operator const pxr::UsdResolveInfo&() const;
#endif	  // #if USE_USD_SDK

		  // Wrapped pxr::UsdResolveInfo functions, refer to the USD SDK documentation
	public:
		FPcpNodeRef GetNode() const;

	private:
		TUniquePtr<Internal::FUsdResolveInfoImpl> Impl;
	};
}	 // namespace UE
