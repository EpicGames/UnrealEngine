// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/Optional.h"
#include "Templates/UniquePtr.h"

#if USE_USD_SDK

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class UsdPrim;
	class UsdSkelAnimQuery;
PXR_NAMESPACE_CLOSE_SCOPE

#endif	  // #if USE_USD_SDK

namespace UE
{
	class FUsdPrim;

	namespace Internal
	{
		class FUsdSkelAnimQueryImpl;
	}

	/**
	 * Minimal pxr::UsdSkelAnimQuery wrapper for Unreal that can be used from no-rtti modules.
	 */
	class UNREALUSDWRAPPER_API FUsdSkelAnimQuery
	{
	public:
		FUsdSkelAnimQuery();
		FUsdSkelAnimQuery(const FUsdSkelAnimQuery& Other);
		FUsdSkelAnimQuery(FUsdSkelAnimQuery&& Other);
		~FUsdSkelAnimQuery();

		FUsdSkelAnimQuery& operator=(const FUsdSkelAnimQuery& Other);
		FUsdSkelAnimQuery& operator=(FUsdSkelAnimQuery&& Other);

		explicit operator bool() const;

		// Auto conversion from/to pxr::UsdSkelAnimQuery
	public:
#if USE_USD_SDK
		explicit FUsdSkelAnimQuery(const pxr::UsdSkelAnimQuery& InUsdSkelAnimQuery);
		explicit FUsdSkelAnimQuery(pxr::UsdSkelAnimQuery&& InUsdSkelAnimQuery);

		operator pxr::UsdSkelAnimQuery&();
		operator const pxr::UsdSkelAnimQuery&() const;
#endif

		// Wrapped pxr::UsdSkelAnimQuery functions, refer to the USD SDK documentation
	public:
		bool IsValid() const;

		UE::FUsdPrim GetPrim() const;

		bool ComputeBlendShapeWeights(TArray<float>& Weights, TOptional<double> TimeCode = {}) const;

		bool GetJointTransformTimeSamples(TArray<double>& TimeCodes) const;
		bool GetBlendShapeWeightTimeSamples(TArray<double>& TimeCodes) const;

		TArray<FString> GetJointOrder() const;
		TArray<FString> GetBlendShapeOrder() const;

	private:
		TUniquePtr<Internal::FUsdSkelAnimQueryImpl> Impl;
	};
}	 // namespace UE
