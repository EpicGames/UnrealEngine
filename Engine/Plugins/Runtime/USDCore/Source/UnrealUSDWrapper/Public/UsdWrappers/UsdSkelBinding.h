// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/UniquePtr.h"

#if USE_USD_SDK

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class UsdPrim;
	class UsdSkelBinding;
PXR_NAMESPACE_CLOSE_SCOPE

#endif	  // #if USE_USD_SDK

namespace UE
{
	class FUsdPrim;
	class FUsdSkelSkinningQuery;

	namespace Internal
	{
		class FUsdSkelBindingImpl;
	}

	/**
	 * Minimal pxr::UsdSkelBinding wrapper for Unreal that can be used from no-rtti modules.
	 */
	class UNREALUSDWRAPPER_API FUsdSkelBinding
	{
	public:
		FUsdSkelBinding();
		FUsdSkelBinding(const FUsdSkelBinding& Other);
		FUsdSkelBinding(FUsdSkelBinding&& Other);
		~FUsdSkelBinding();

		FUsdSkelBinding& operator=(const FUsdSkelBinding& Other);
		FUsdSkelBinding& operator=(FUsdSkelBinding&& Other);

		// Auto conversion from/to pxr::UsdSkelBinding
	public:
#if USE_USD_SDK
		explicit FUsdSkelBinding(const pxr::UsdSkelBinding& InUsdSkelBinding);
		explicit FUsdSkelBinding(pxr::UsdSkelBinding&& InUsdSkelBinding);

		operator pxr::UsdSkelBinding&();
		operator const pxr::UsdSkelBinding&() const;
#endif

		// Wrapped pxr::UsdSkelBinding functions, refer to the USD SDK documentation
	public:
		UE::FUsdPrim GetSkeleton() const;

		TArray<UE::FUsdSkelSkinningQuery> GetSkinningTargets() const;

	private:
		TUniquePtr<Internal::FUsdSkelBindingImpl> Impl;
	};
}	 // namespace UE
