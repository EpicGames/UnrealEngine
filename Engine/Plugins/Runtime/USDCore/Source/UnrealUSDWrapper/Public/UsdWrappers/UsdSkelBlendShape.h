// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UsdTyped.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class UsdPrim;
	class UsdSkelBlendShape;
PXR_NAMESPACE_CLOSE_SCOPE
#endif	  // #if USE_USD_SDK

namespace UE
{
	class FUsdSkelInbetweenShape;

	namespace Internal
	{
		class FUsdSkelBlendShapeImpl;
	}

	/**
	 * Minimal pxr::UsdSkelBlendShape wrapper for Unreal that can be used from no-rtti modules.
	 */
	class UNREALUSDWRAPPER_API FUsdSkelBlendShape : public FUsdTyped
	{
	public:
		FUsdSkelBlendShape();
		FUsdSkelBlendShape(const FUsdPrim& Prim);
		FUsdSkelBlendShape(const FUsdSkelBlendShape& Other);
		FUsdSkelBlendShape(FUsdSkelBlendShape&& Other);
		~FUsdSkelBlendShape();

		FUsdSkelBlendShape& operator=(const FUsdSkelBlendShape& Other);
		FUsdSkelBlendShape& operator=(FUsdSkelBlendShape&& Other);

		explicit operator bool() const;

		// Auto conversion from/to pxr::UsdSkelBlendShape
	public:
#if USE_USD_SDK
		explicit FUsdSkelBlendShape(const pxr::UsdSkelBlendShape& InUsdSkelBlendShape);
		explicit FUsdSkelBlendShape(pxr::UsdSkelBlendShape&& InUsdSkelBlendShape);
		explicit FUsdSkelBlendShape(const pxr::UsdPrim& Prim);

		operator pxr::UsdSkelBlendShape&();
		operator const pxr::UsdSkelBlendShape&() const;
#endif

		// Wrapped pxr::UsdSkelBlendShape functions, refer to the USD SDK documentation
	public:
		FUsdSkelInbetweenShape GetInbetween(const FString& Name) const;
		TArray<FUsdSkelInbetweenShape> GetInbetweens() const;

	private:
		TUniquePtr<Internal::FUsdSkelBlendShapeImpl> Impl;
	};
}	 // namespace UE
