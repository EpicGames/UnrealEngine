// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/UniquePtr.h"

#if USE_USD_SDK

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class UsdPrim;
	class UsdSkelBlendShapeQuery;
PXR_NAMESPACE_CLOSE_SCOPE

#endif	  // #if USE_USD_SDK

namespace UE
{
	class FUsdPrim;
	class FUsdSkelBlendShape;

	namespace Internal
	{
		class FUsdSkelBlendShapeQueryImpl;
	}

	/**
	 * Minimal pxr::UsdSkelBlendShapeQuery wrapper for Unreal that can be used from no-rtti modules.
	 */
	class UNREALUSDWRAPPER_API FUsdSkelBlendShapeQuery
	{
	public:
		FUsdSkelBlendShapeQuery();
		FUsdSkelBlendShapeQuery(const UE::FUsdPrim& SkelBindingAPIPrim);
		FUsdSkelBlendShapeQuery(const FUsdSkelBlendShapeQuery& Other);
		FUsdSkelBlendShapeQuery(FUsdSkelBlendShapeQuery&& Other);
		~FUsdSkelBlendShapeQuery();

		FUsdSkelBlendShapeQuery& operator=(const FUsdSkelBlendShapeQuery& Other);
		FUsdSkelBlendShapeQuery& operator=(FUsdSkelBlendShapeQuery&& Other);

		explicit operator bool() const;

		// Auto conversion from/to pxr::UsdSkelBlendShapeQuery
	public:
#if USE_USD_SDK
		explicit FUsdSkelBlendShapeQuery(const pxr::UsdSkelBlendShapeQuery& InUsdSkelBlendShapeQuery);
		explicit FUsdSkelBlendShapeQuery(pxr::UsdSkelBlendShapeQuery&& InUsdSkelBlendShapeQuery);

		operator pxr::UsdSkelBlendShapeQuery&();
		operator const pxr::UsdSkelBlendShapeQuery&() const;
#endif

		// Wrapped pxr::UsdSkelBlendShapeQuery functions, refer to the USD SDK documentation
	public:
		bool IsValid() const;

		UE::FUsdPrim GetPrim() const;

		size_t GetNumBlendShapes() const;

		UE::FUsdSkelBlendShape GetBlendShape(size_t BlendShapeIndex) const;

	private:
		TUniquePtr<Internal::FUsdSkelBlendShapeQueryImpl> Impl;
	};
}	 // namespace UE
