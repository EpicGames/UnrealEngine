// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/UniquePtr.h"

#if USE_USD_SDK

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class UsdPrim;
	class UsdSkelCache;
PXR_NAMESPACE_CLOSE_SCOPE

#endif	  // #if USE_USD_SDK

namespace UE
{
	class FUsdPrim;
	class FUsdSkelSkeletonQuery;
	class FUsdSkelAnimQuery;
	class FUsdSkelSkinningQuery;
	class FUsdSkelBinding;

	namespace Internal
	{
		class FUsdSkelCacheImpl;
	}

	/**
	 * Minimal pxr::UsdSkelCache wrapper for Unreal that can be used from no-rtti modules.
	 */
	class UNREALUSDWRAPPER_API FUsdSkelCache
	{
	public:
		FUsdSkelCache();
		FUsdSkelCache(const FUsdSkelCache& Other);
		FUsdSkelCache(FUsdSkelCache&& Other);
		~FUsdSkelCache();

		FUsdSkelCache& operator=(const FUsdSkelCache& Other);
		FUsdSkelCache& operator=(FUsdSkelCache&& Other);

		// Auto conversion from/to pxr::UsdSkelCache
	public:
#if USE_USD_SDK
		explicit FUsdSkelCache(const pxr::UsdSkelCache& InUsdSkelCache);
		explicit FUsdSkelCache(pxr::UsdSkelCache&& InUsdSkelCache);

		operator pxr::UsdSkelCache&();
		operator const pxr::UsdSkelCache&() const;
#endif

		// Wrapped pxr::UsdSkelCache functions, refer to the USD SDK documentation
	public:
		void Clear();

		bool Populate(const UE::FUsdPrim& SkelRootPrim, bool bTraverseInstanceProxies);

		UE::FUsdSkelSkeletonQuery GetSkelQuery(const UE::FUsdPrim& SkeletonPrim) const;

		UE::FUsdSkelAnimQuery GetAnimQuery(const UE::FUsdPrim& SkelAnimationPrim) const;

		UE::FUsdSkelSkinningQuery GetSkinningQuery(const UE::FUsdPrim& SkinnedPrim) const;

		bool ComputeSkelBindings(const UE::FUsdPrim& InSkelRootPrim, TArray<UE::FUsdSkelBinding>& OutBindings, bool bTraverseInstanceProxies) const;

		bool ComputeSkelBinding(
			const UE::FUsdPrim& InSkelRootPrim,
			const UE::FUsdPrim& InSkeletonPrim,
			UE::FUsdSkelBinding& OutBinding,
			bool bTraverseInstanceProxies
		) const;

	private:
		TUniquePtr<Internal::FUsdSkelCacheImpl> Impl;
	};
}	 // namespace UE
