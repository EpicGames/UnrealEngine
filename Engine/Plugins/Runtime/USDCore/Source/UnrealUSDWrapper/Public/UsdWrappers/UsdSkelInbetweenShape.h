// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/UniquePtr.h"

#if USE_USD_SDK

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class UsdPrim;
	class UsdSkelInbetweenShape;
PXR_NAMESPACE_CLOSE_SCOPE

#endif	  // #if USE_USD_SDK

namespace UE
{
	class FUsdPrim;
	class FUsdAttribute;

	namespace Internal
	{
		class FUsdSkelInbetweenShapeImpl;
	}

	/**
	 * Minimal pxr::UsdSkelInbetweenShape wrapper for Unreal that can be used from no-rtti modules.
	 */
	class UNREALUSDWRAPPER_API FUsdSkelInbetweenShape
	{
	public:
		FUsdSkelInbetweenShape();
		FUsdSkelInbetweenShape(const FUsdSkelInbetweenShape& Other);
		FUsdSkelInbetweenShape(FUsdSkelInbetweenShape&& Other);
		~FUsdSkelInbetweenShape();

		FUsdSkelInbetweenShape& operator=(const FUsdSkelInbetweenShape& Other);
		FUsdSkelInbetweenShape& operator=(FUsdSkelInbetweenShape&& Other);

		explicit operator bool() const;

		// Auto conversion from/to pxr::UsdSkelInbetweenShape
	public:
#if USE_USD_SDK
		explicit FUsdSkelInbetweenShape(const pxr::UsdSkelInbetweenShape& InUsdSkelInbetweenShape);
		explicit FUsdSkelInbetweenShape(pxr::UsdSkelInbetweenShape&& InUsdSkelInbetweenShape);

		operator pxr::UsdSkelInbetweenShape&();
		operator const pxr::UsdSkelInbetweenShape&() const;
#endif

		// Wrapped pxr::UsdSkelInbetweenShape functions, refer to the USD SDK documentation
	public:
		bool GetWeight(float* OutWeight) const;
		UE::FUsdAttribute GetAttr() const;
		bool IsDefined() const;

	private:
		TUniquePtr<Internal::FUsdSkelInbetweenShapeImpl> Impl;
	};
}	 // namespace UE
