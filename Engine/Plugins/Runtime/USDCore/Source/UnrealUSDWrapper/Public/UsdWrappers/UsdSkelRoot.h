// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UsdTyped.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class UsdPrim;
	class UsdSkelRoot;
PXR_NAMESPACE_CLOSE_SCOPE
#endif	  // #if USE_USD_SDK

namespace UE
{
	class FUsdAttribute;

	namespace Internal
	{
		class FUsdSkelRootImpl;
	}

	/**
	 * Minimal pxr::UsdSkelRoot wrapper for Unreal that can be used from no-rtti modules.
	 */
	class UNREALUSDWRAPPER_API FUsdSkelRoot : public FUsdTyped
	{
	public:
		FUsdSkelRoot();
		FUsdSkelRoot(const FUsdSkelRoot& Other);
		FUsdSkelRoot(FUsdSkelRoot&& Other);

		explicit FUsdSkelRoot(const FUsdPrim& Prim);

		FUsdSkelRoot& operator=(const FUsdSkelRoot& Other);
		FUsdSkelRoot& operator=(FUsdSkelRoot&& Other);

		~FUsdSkelRoot();

		explicit operator bool() const;

		// Auto conversion from/to pxr::UsdSkelRoot
	public:
#if USE_USD_SDK
		explicit FUsdSkelRoot(const pxr::UsdSkelRoot& InUsdSkelRoot);
		explicit FUsdSkelRoot(pxr::UsdSkelRoot&& InUsdSkelRoot);
		explicit FUsdSkelRoot(const pxr::UsdPrim& Prim);

		operator pxr::UsdSkelRoot&();
		operator const pxr::UsdSkelRoot&() const;
#endif	  // #if USE_USD_SDK

		  // Wrapped pxr::UsdSkelRoot functions, refer to the USD SDK documentation
	public:
		static FUsdSkelRoot Find(const UE::FUsdPrim& Prim);

	private:
		TUniquePtr<Internal::FUsdSkelRootImpl> Impl;
	};
}	 // namespace UE
