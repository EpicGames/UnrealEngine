// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/UniquePtr.h"

#if USE_USD_SDK

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class UsdPrim;
	class UsdSkelSkeletonQuery;
PXR_NAMESPACE_CLOSE_SCOPE

#endif	  // #if USE_USD_SDK

namespace UE
{
	class FUsdPrim;
	class FUsdSkelAnimQuery;

	namespace Internal
	{
		class FUsdSkelSkeletonQueryImpl;
	}

	/**
	 * Minimal pxr::UsdSkelSkeletonQuery wrapper for Unreal that can be used from no-rtti modules.
	 */
	class UNREALUSDWRAPPER_API FUsdSkelSkeletonQuery
	{
	public:
		// WARNING: Default-constructed objects are permanently invalid!
		// The default constructor is available for technical reasons, but you should create these objects
		// by calling FUsdSkelCache::GetSkelQuery(SkeletonPrim)
		FUsdSkelSkeletonQuery();
		FUsdSkelSkeletonQuery(const FUsdSkelSkeletonQuery& Other);
		FUsdSkelSkeletonQuery(FUsdSkelSkeletonQuery&& Other);
		~FUsdSkelSkeletonQuery();

		FUsdSkelSkeletonQuery& operator=(const FUsdSkelSkeletonQuery& Other);
		FUsdSkelSkeletonQuery& operator=(FUsdSkelSkeletonQuery&& Other);

		explicit operator bool() const;

		// Auto conversion from/to pxr::UsdSkelSkeletonQuery
	public:
#if USE_USD_SDK
		explicit FUsdSkelSkeletonQuery(const pxr::UsdSkelSkeletonQuery& InUsdSkelSkeletonQuery);
		explicit FUsdSkelSkeletonQuery(pxr::UsdSkelSkeletonQuery&& InUsdSkelSkeletonQuery);

		operator pxr::UsdSkelSkeletonQuery&();
		operator const pxr::UsdSkelSkeletonQuery&() const;
#endif

		// Wrapped pxr::UsdSkelSkeletonQuery functions, refer to the USD SDK documentation
	public:
		bool IsValid() const;

		UE::FUsdPrim GetPrim() const;
		UE::FUsdPrim GetSkeleton() const;
		UE::FUsdSkelAnimQuery GetAnimQuery() const;

		bool ComputeJointLocalTransforms(TArray<FTransform>& UsdSpaceTransforms, double TimeCode, bool bAtRest = false) const;

	private:
		TUniquePtr<Internal::FUsdSkelSkeletonQueryImpl> Impl;
	};
}	 // namespace UE
