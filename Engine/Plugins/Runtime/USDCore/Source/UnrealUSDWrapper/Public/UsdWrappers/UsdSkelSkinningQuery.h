// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/UniquePtr.h"

#if USE_USD_SDK

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class UsdPrim;
	class UsdSkelSkinningQuery;
PXR_NAMESPACE_CLOSE_SCOPE

#endif	  // #if USE_USD_SDK

namespace UE
{
	class FUsdPrim;
	class FUsdRelationship;

	namespace Internal
	{
		class FUsdSkelSkinningQueryImpl;
	}

	/**
	 * Minimal pxr::UsdSkelSkinningQuery wrapper for Unreal that can be used from no-rtti modules.
	 */
	class UNREALUSDWRAPPER_API FUsdSkelSkinningQuery
	{
	public:
		FUsdSkelSkinningQuery();
		FUsdSkelSkinningQuery(const FUsdSkelSkinningQuery& Other);
		FUsdSkelSkinningQuery(FUsdSkelSkinningQuery&& Other);
		~FUsdSkelSkinningQuery();

		FUsdSkelSkinningQuery& operator=(const FUsdSkelSkinningQuery& Other);
		FUsdSkelSkinningQuery& operator=(FUsdSkelSkinningQuery&& Other);

		explicit operator bool() const;

		// Auto conversion from/to pxr::UsdSkelSkinningQuery
	public:
#if USE_USD_SDK
		explicit FUsdSkelSkinningQuery(const pxr::UsdSkelSkinningQuery& InUsdSkelSkinningQuery);
		explicit FUsdSkelSkinningQuery(pxr::UsdSkelSkinningQuery&& InUsdSkelSkinningQuery);

		operator pxr::UsdSkelSkinningQuery&();
		operator const pxr::UsdSkelSkinningQuery&() const;
#endif

		// Wrapped pxr::UsdSkelSkinningQuery functions, refer to the USD SDK documentation
	public:
		bool IsValid() const;

		UE::FUsdPrim GetPrim() const;

		UE::FUsdRelationship GetBlendShapeTargetsRel() const;

		bool GetBlendShapeOrder(TArray<FString>& BlendShapes) const;

		// Note: This matrix is in USD space, retrieved as-is 
		FMatrix GetGeomBindTransform(double UsdTimeCode) const;

	private:
		TUniquePtr<Internal::FUsdSkelSkinningQueryImpl> Impl;
	};
}	 // namespace UE
