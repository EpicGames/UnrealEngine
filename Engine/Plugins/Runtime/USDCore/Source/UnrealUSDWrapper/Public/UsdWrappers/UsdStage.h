// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UniquePtr.h"

#include "UnrealUSDWrapper.h"
#include "UsdWrappers/ForwardDeclarations.h"

#define UE_API UNREALUSDWRAPPER_API

namespace UE
{
	class FSdfPath;
	class FUsdPrim;
	class FVtValue;

	namespace Internal
	{
		template<typename PtrType>
		class FUsdStageImpl;
	}

	/**
	 * Minimal pxr::UsdStage pointer wrapper for Unreal that can be used from no-rtti modules.
	 * Use the aliases FUsdStage and FUsdStageWeak instead (defined on ForwardDeclarations.h)
	 */
	template<typename PtrType>
	class FUsdStageBase
	{
	public:
		UE_API FUsdStageBase();

		UE_API FUsdStageBase(const FUsdStage& Other);
		UE_API FUsdStageBase(FUsdStage&& Other);
		UE_API FUsdStageBase(const FUsdStageWeak& Other);
		UE_API FUsdStageBase(FUsdStageWeak&& Other);

		UE_API FUsdStageBase& operator=(const FUsdStage& Other);
		UE_API FUsdStageBase& operator=(FUsdStage&& Other);
		UE_API FUsdStageBase& operator=(const FUsdStageWeak& Other);
		UE_API FUsdStageBase& operator=(FUsdStageWeak&& Other);

		UE_API ~FUsdStageBase();

		UE_API explicit operator bool() const;

		// We have to implement a templated comparison operator, or else we get ambigous
		// conversions on the Mac targets when comparing FUsdStage and FUsdStageWeak
		template<typename OtherPtrType>
		UE_API bool operator==(const FUsdStageBase<OtherPtrType>& Other) const;

		// Auto conversion from/to PtrType. We use concrete pointer types here
		// because we should also be able to convert between them
	public:
#if USE_USD_SDK
		UE_API explicit FUsdStageBase(const pxr::UsdStageRefPtr& InUsdPtr);
		UE_API explicit FUsdStageBase(pxr::UsdStageRefPtr&& InUsdPtr);
		UE_API explicit FUsdStageBase(const pxr::UsdStageWeakPtr& InUsdPtr);
		UE_API explicit FUsdStageBase(pxr::UsdStageWeakPtr&& InUsdPtr);

		UE_API operator PtrType&();
		UE_API operator const PtrType&() const;

		UE_API operator pxr::UsdStageRefPtr() const;
		UE_API operator pxr::UsdStageWeakPtr() const;
#endif	  // USE_USD_SDK

		  // Wrapped pxr::UsdStage functions, refer to the USD SDK documentation
	public:
		UE_API void LoadAndUnload(
			const TSet<UE::FSdfPath>& LoadSet,
			const TSet<UE::FSdfPath>& UnloadSet,
			EUsdLoadPolicy Policy = EUsdLoadPolicy::UsdLoadWithDescendants
		);

		/**
		 * Saves a flattened copy of the stage to the given path (e.g. "C:/Folder/FlattenedStage.usda"). Will use the corresponding file writer
		 * depending on FilePath extension. Will not alter the current stage.
		 */
		UE_API bool Export(const TCHAR* FileName, bool bAddSourceFileComment = true, const TMap<FString, FString>& FileFormatArguments = {}) const;

		UE_API FSdfLayer GetRootLayer() const;
		UE_API FSdfLayer GetSessionLayer() const;
		UE_API bool HasLocalLayer(const FSdfLayer& Layer) const;

		UE_API FUsdPrim GetPseudoRoot() const;
		UE_API FUsdPrim GetDefaultPrim() const;
		UE_API FUsdPrim GetPrimAtPath(const FSdfPath& Path) const;

		UE_API TArray<FSdfLayer> GetLayerStack(bool bIncludeSessionLayers = true) const;
		UE_API TArray<FSdfLayer> GetUsedLayers(bool bIncludeClipLayers = true) const;

		UE_API void MuteAndUnmuteLayers(const TArray<FString>& MuteLayers, const TArray<FString>& UnmuteLayers);
		UE_API bool IsLayerMuted(const FString& LayerIdentifier) const;

		UE_API bool IsEditTargetValid() const;
		UE_API void SetEditTarget(const FSdfLayer& Layer);
		UE_API FSdfLayer GetEditTarget() const;

		UE_API bool GetMetadata(const TCHAR* Key, UE::FVtValue& Value) const;
		UE_API bool HasMetadata(const TCHAR* Key) const;
		UE_API bool SetMetadata(const TCHAR* Key, const UE::FVtValue& Value) const;
		UE_API bool ClearMetadata(const TCHAR* Key) const;

		UE_API double GetStartTimeCode() const;
		UE_API double GetEndTimeCode() const;
		UE_API void SetStartTimeCode(double TimeCode);
		UE_API void SetEndTimeCode(double TimeCode);
		UE_API double GetTimeCodesPerSecond() const;
		UE_API void SetTimeCodesPerSecond(double TimeCodesPerSecond);
		UE_API double GetFramesPerSecond() const;
		UE_API void SetFramesPerSecond(double FramesPerSecond);

		UE_API void SetInterpolationType(EUsdInterpolationType InterpolationType);
		UE_API EUsdInterpolationType GetInterpolationType() const;

		UE_API void SetDefaultPrim(const FUsdPrim& Prim);

		UE_API FUsdPrim OverridePrim(const FSdfPath& Path);
		UE_API FUsdPrim DefinePrim(const FSdfPath& Path, const TCHAR* TypeName = TEXT(""));
		UE_API FUsdPrim CreateClassPrim(const FSdfPath& RootPrimPath);
		UE_API bool RemovePrim(const FSdfPath& Path);

	private:
		// So we can use the Other's Impl on copy constructor/operators
		friend FUsdStage;
		friend FUsdStageWeak;

		TUniquePtr<Internal::FUsdStageImpl<PtrType>> Impl;
	};
}	 // namespace UE

#undef UE_API