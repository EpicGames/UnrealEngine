// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class UsdPrim;
	class UsdSchemaBase;
	class UsdTyped;
PXR_NAMESPACE_CLOSE_SCOPE
#endif	  // #if USE_USD_SDK

namespace UE
{
	class FSdfPath;
	class FUsdPrim;

	namespace Internal
	{
		class FUsdTypedImpl;
	}

	/**
	 * Minimal pxr::UsdTyped wrapper for Unreal that can be used from no-rtti modules.
	 */
	class UNREALUSDWRAPPER_API FUsdTyped
	{
	public:
		FUsdTyped();
		FUsdTyped(const FUsdTyped& Other);
		FUsdTyped(FUsdTyped&& Other);

		explicit FUsdTyped(const FUsdPrim& Prim);

		FUsdTyped& operator=(const FUsdTyped& Other);
		FUsdTyped& operator=(FUsdTyped&& Other);

		~FUsdTyped();

		explicit operator bool() const;

		// Auto conversion from/to pxr::UsdTyped
	public:
#if USE_USD_SDK
		explicit FUsdTyped(const pxr::UsdTyped& InUsdTyped);
		explicit FUsdTyped(pxr::UsdTyped&& InUsdTyped);
		explicit FUsdTyped(const pxr::UsdPrim& Prim);

		operator pxr::UsdTyped&();
		operator const pxr::UsdTyped&() const;
#endif	  // #if USE_USD_SDK

		  // Wrapped pxr::UsdTyped functions, refer to the USD SDK documentation
	public:
		FSdfPath GetPath() const;
		FUsdPrim GetPrim() const;

	private:
		TUniquePtr<Internal::FUsdTypedImpl> Impl;
	};
}	 // namespace UE
