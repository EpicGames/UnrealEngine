// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Templates/UniquePtr.h"

#include "UnrealUSDWrapper.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

#include <string>

PXR_NAMESPACE_OPEN_SCOPE
	class UsdPrim;
	class UsdVariantSet;
	class UsdVariantSets;
PXR_NAMESPACE_CLOSE_SCOPE
#endif	  // #if USE_USD_SDK

namespace UE
{
	class FUsdPrim;

	namespace Internal
	{
		class FUsdVariantSetImpl;
		class FUsdVariantSetsImpl;
	}

	/**
	 * Minimal pxr::UsdVariantSet wrapper for Unreal that can be used from no-rtti modules.
	 */
	class UNREALUSDWRAPPER_API FUsdVariantSet
	{
	public:
		FUsdVariantSet();
		FUsdVariantSet(const FUsdVariantSet& Other);
		FUsdVariantSet(FUsdVariantSet&& Other);

		FUsdVariantSet& operator=(const FUsdVariantSet& Other);
		FUsdVariantSet& operator=(FUsdVariantSet&& Other);

		~FUsdVariantSet();

		explicit operator bool() const;

#if USE_USD_SDK
		// Auto conversion from/to pxr::UsdVariantSet
		explicit FUsdVariantSet(const pxr::UsdVariantSet& InUsdVariantSet);
		explicit FUsdVariantSet(pxr::UsdVariantSet&& InUsdVariantSet);
		FUsdVariantSet& operator=(const pxr::UsdVariantSet& InUsdVariantSet);
		FUsdVariantSet& operator=(pxr::UsdVariantSet&& InUsdVariantSet);

		operator pxr::UsdVariantSet();
		operator const pxr::UsdVariantSet() const;

	private:
		// FUsdVariantSet instances are only constructable by instances of
		// FUsdPrim or FUsdVariantSets.
		// Use FUsdPrim::GetVariantSet() or FUsdVariantSets::GetVariantSet()
		// to construct an instance.
		explicit FUsdVariantSet(const pxr::UsdPrim& InPrim, const std::string& InVariantSetName);
#endif	  // #if USE_USD_SDK

		  // Wrapped pxr::UsdVariantSet functions, refer to the USD SDK documentation
	public:
		bool AddVariant(const FString& VariantName, EUsdListPosition Position = EUsdListPosition::BackOfPrependList);

		TArray<FString> GetVariantNames() const;

		bool HasAuthoredVariant(const FString& VariantName) const;

		FString GetVariantSelection() const;

		bool HasAuthoredVariantSelection(FString* Value = nullptr) const;

		bool SetVariantSelection(const FString& VariantName);

		bool ClearVariantSelection();

		bool BlockVariantSelection();

		// No support currently for accessing the edit target/context for
		// an FUsdVariantSet.

		FUsdPrim GetPrim() const;

		FString GetName() const;

		bool IsValid() const;

	private:
		TUniquePtr<Internal::FUsdVariantSetImpl> Impl;

		friend class FUsdPrim;
		friend class FUsdVariantSets;
	};

	/**
	 * Minimal pxr::UsdVariantSets wrapper for Unreal that can be used from no-rtti modules.
	 */
	class UNREALUSDWRAPPER_API FUsdVariantSets
	{
	public:
		FUsdVariantSets();
		FUsdVariantSets(const FUsdVariantSets& Other);
		FUsdVariantSets(FUsdVariantSets&& Other);

		FUsdVariantSets& operator=(const FUsdVariantSets& Other);
		FUsdVariantSets& operator=(FUsdVariantSets&& Other);

		~FUsdVariantSets();

		explicit operator bool() const;

#if USE_USD_SDK
		// The pxr::UsdPrim is not accessible from an instance of
		// pxr::UsdVariantSets, so there are no facilities for
		// constructing or assigning to an FUsdVariantSets from
		// the USD native type.

		// Auto conversion to pxr::UsdVariantSets
		operator pxr::UsdVariantSets();
		operator const pxr::UsdVariantSets() const;

	private:
		// FUsdVariantSets instances are only constructable by instances of
		// FUsdPrim.
		// Use FUsdPrim::GetVariantSets() to construct an instance.
		explicit FUsdVariantSets(const pxr::UsdPrim& InPrim);
#endif	  // #if USE_USD_SDK

		  // Wrapped pxr::UsdVariantSets functions, refer to the USD SDK documentation
	public:
		FUsdVariantSet AddVariantSet(const FString& VariantSetName, EUsdListPosition Position = EUsdListPosition::BackOfPrependList);

		TArray<FString> GetNames() const;

		FUsdVariantSet operator[](const FString& VariantSetName) const
		{
			return GetVariantSet(VariantSetName);
		}

		FUsdVariantSet GetVariantSet(const FString& VariantSetName) const;

		bool HasVariantSet(const FString& VariantSetName) const;

		FString GetVariantSelection(const FString& VariantSetName) const;

		bool SetSelection(const FString& VariantSetName, const FString& VariantName);

		TMap<FString, FString> GetAllVariantSelections() const;

	private:
		TUniquePtr<Internal::FUsdVariantSetsImpl> Impl;

		friend class FUsdPrim;
	};
}	 // namespace UE
