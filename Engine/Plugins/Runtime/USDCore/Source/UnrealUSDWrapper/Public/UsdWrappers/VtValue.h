// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class VtValue;
PXR_NAMESPACE_CLOSE_SCOPE
#endif	  // #if USE_USD_SDK

namespace UE
{
	namespace Internal
	{
		class FVtValueImpl;
	}

	/**
	 * Minimal pxr::VtValue wrapper for Unreal that can be used from no-rtti modules.
	 */
	class UNREALUSDWRAPPER_API FVtValue
	{
	public:
		FVtValue();

		FVtValue(const FVtValue& Other);
		FVtValue(FVtValue&& Other);

		FVtValue& operator=(const FVtValue& Other);
		FVtValue& operator=(FVtValue&& Other);

		~FVtValue();

		bool operator==(const FVtValue& Other) const;
		bool operator!=(const FVtValue& Other) const;

		// Auto conversion from/to pxr::VtValue
		// We define a GetUsdValue() instead of `operator pxr::VtValue&` because pxr::VtValue itself has lower
		// precedence, implicit generic conversions and operators that can conflict/generate hidden bugs if we try using them
		// without having this header included (e.g. we could end up with a pxr::VtValue containing an UE::FVtValue instead)
	public:
#if USE_USD_SDK
		explicit FVtValue(const pxr::VtValue& InVtValue);
		explicit FVtValue(pxr::VtValue&& InVtValue);
		FVtValue& operator=(const pxr::VtValue& InVtValue);
		FVtValue& operator=(pxr::VtValue&& InVtValue);

		pxr::VtValue& GetUsdValue();
		const pxr::VtValue& GetUsdValue() const;
#endif	  // #if USE_USD_SDK

		  // Wrapped pxr::VtValue functions, refer to the USD SDK documentation
	public:
		FString GetTypeName() const;
		bool IsArrayValued() const;

		/** Returns UE types (FQuat, TArray<FFloat16>, etc.), performing USD to Unreal conversions when needed */
		template <typename T>
		T Get() const;

		/**
		 * Receives UE types (FQuat, TArray<FFloat16>, etc.), performing Unreal to USD conversions when needed.
		 *
		 * pxr::VtValue provides assignment operators/constructors as the main "setters", but we provide this explicit setter instead,
		 * given that an assignment operator for the UE::FVtValue wrapper class may provide ambiguous/unclear behavior in edge cases
		 * (e.g. assigning another pxr::VtValue or assigning an UE::FVtValue)
		 */
		template <typename T>
		void Set(const T& Value);

		bool IsEmpty() const;

	private:
		TUniquePtr<Internal::FVtValueImpl> Impl;
	};
}	 // namespace UE
