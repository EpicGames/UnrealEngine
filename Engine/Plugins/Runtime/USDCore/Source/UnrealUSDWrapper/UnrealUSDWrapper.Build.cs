// Copyright Epic Games, Inc. All Rights Reserved.

using System.Linq;
using System.IO;
using System.Collections.Generic;

namespace UnrealBuildTool.Rules
{
	public class UnrealUSDWrapper : ModuleRules
	{
		// Attention: When you include UnrealUSDWrapper as a dependency and include USDMemory.h, use the USD scoped allocs or IMPLEMENT_MODULE_USD
		// within your module, make sure that you call CheckAndSetupUsdSdk so that your module is properly setup for USD use.
		public static bool CheckAndSetupUsdSdk(ReadOnlyTargetRules Target, ModuleRules Module)
		{
			// Must be set to true if the USD SDK was compiled with PXR_ENABLE_RUNTIME_MEMORY_OVERLOADS=ON
			// Only relevant on Windows since on Linux/Mac, the UE allocators will always be used
			bool bEnableUEAllocatorsInUSD = true;
			bool bIsUsdSdkEnabled = EnableUsdSdk(Target, Module);
			
			// Only Windows require memory overloads in the USD SDK
			bool bPlatformNeedsUsdMemoryOverloads = Target.Platform == UnrealTargetPlatform.Win64;
			bool bEnableUsdMemoryOverloads = bIsUsdSdkEnabled && bEnableUEAllocatorsInUSD && bPlatformNeedsUsdMemoryOverloads;
			Module.PublicDefinitions.Add("ENABLE_USD_MEMORY_OVERLOADS=" + (bEnableUsdMemoryOverloads ? "1" : "0"));

			Module.PublicDefinitions.Add("USD_MERGED_MODULES=" + (Target.bMergeModules ? "1" : "0"));
			if (!Target.bMergeModules && !bEnableUsdMemoryOverloads && bPlatformNeedsUsdMemoryOverloads)
			{
				Module.PrivateDefinitions.Add("SUPPRESS_PER_MODULE_INLINE_FILE"); // This module does not use core's standard operator new/delete overloads
			}

			return bIsUsdSdkEnabled;
		}

		public UnrealUSDWrapper(ReadOnlyTargetRules Target) : base(Target)
		{
			bUseRTTI = true;

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"CinematicCamera",
					"Core",
					"CoreUObject",
					"Engine",
					"IntelTBB",
					"Projects", // For plugin manager within UnrealUSDWrapper.cpp
					"USDClasses"
				}
			);

			bool bIsUsdSdkEnabled = CheckAndSetupUsdSdk(Target, this);

			// Temporarily disabled runtime USD support until Mac and Linux dynamic linking issues are resolved
			if (bIsUsdSdkEnabled && (Target.Type == TargetType.Editor || Target.Platform == UnrealTargetPlatform.Win64))
			{
				PublicDependencyModuleNames.Add("Python3");

				PublicDefinitions.Add("USE_USD_SDK=1");
				PublicDefinitions.Add("ENABLE_USD_DEBUG_PATH=0");

				string ThirdPartyUSDDirectory = Path.Combine(ModuleDirectory, "..", "ThirdParty", "USD");
				string USDIncludeDirectory = Path.Combine(ThirdPartyUSDDirectory, "include");

				var EngineDir = Path.GetFullPath(Target.RelativeEnginePath);
				var PythonPlatformName = (Target.Platform == UnrealTargetPlatform.Win64 && Target.Architecture == UnrealArch.Arm64) ? "WinArm64" : Target.Platform.ToString();
				var PythonSourceTPSDir = Path.Combine(EngineDir, "Source", "ThirdParty", "Python3", PythonPlatformName);
				var PythonBinaryTPSDir = Path.Combine(EngineDir, "Binaries", "ThirdParty", "Python3", PythonPlatformName);

				if (Target.Platform == UnrealTargetPlatform.Win64)
				{
					PublicDefinitions.Add("USD_USES_SYSTEM_MALLOC=1");

					// Python3
					string ArchDir = (Target.Architecture == UnrealArch.Arm64) ? "arm64" : "";
					PublicSystemIncludePaths.Add(Path.Combine(PythonSourceTPSDir, "include"));
					PublicSystemLibraryPaths.Add(Path.Combine(PythonSourceTPSDir, "libs"));
					RuntimeDependencies.Add(Path.Combine("$(TargetOutputDir)", ArchDir, "python311.dll"), Path.Combine(PythonBinaryTPSDir, "python311.dll"));

					// USD
					PublicSystemIncludePaths.Add(USDIncludeDirectory);
					string USDLibDir = Path.Combine(
						ThirdPartyUSDDirectory,
						"VS" + Target.WindowsPlatform.GetVisualStudioCompilerVersionName(),
						Target.Architecture.WindowsLibDir,
						"lib");
					PublicSystemLibraryPaths.Add(USDLibDir);
					foreach (string UsdLib in Directory.EnumerateFiles(USDLibDir, "*.lib", SearchOption.AllDirectories))
					{
						PublicAdditionalLibraries.Add(UsdLib);
					}
					string USDBinDir = Path.Combine(
						ThirdPartyUSDDirectory,
						"VS" + Target.WindowsPlatform.GetVisualStudioCompilerVersionName(),
						Target.Architecture.WindowsLibDir,
						"bin");
					foreach (string UsdDll in Directory.EnumerateFiles(USDBinDir, "*.dll", SearchOption.AllDirectories))
					{
						// We can't delay-load the USD dlls as they contain data and vtables: They need to be next to the executable and implicitly linked
						RuntimeDependencies.Add(Path.Combine("$(TargetOutputDir)", Path.GetFileName(UsdDll)), UsdDll);
					}

					// If OpenUSD was rebuilt with debug information, add the .pdb files as well
					foreach (string UsdPdb in Directory.EnumerateFiles(USDBinDir, "*.pdb", SearchOption.AllDirectories))
					{
						RuntimeDependencies.Add(Path.Combine("$(TargetOutputDir)", Path.GetFileName(UsdPdb)), UsdPdb, StagedFileType.DebugNonUFS);
					}
				}
				else if (Target.Platform == UnrealTargetPlatform.Linux)
				{
					PublicDefinitions.Add("USD_USES_SYSTEM_MALLOC=0"); // USD uses tbb malloc on Linux

					// Python3
					PublicSystemIncludePaths.Add(Path.Combine(PythonSourceTPSDir, "include"));
					PublicSystemLibraryPaths.Add(Path.Combine(PythonBinaryTPSDir, "lib"));
					PrivateRuntimeLibraryPaths.Add(Path.Combine(PythonBinaryTPSDir, "bin"));
					RuntimeDependencies.Add(Path.Combine(PythonBinaryTPSDir, "bin", "python3.11"));

					// USD
					PublicSystemIncludePaths.Add(USDIncludeDirectory);
					string USDLibDir = Path.Combine(
						ThirdPartyUSDDirectory,
						"Unix",
						Target.Architecture.LinuxName,
						"lib");
					PrivateRuntimeLibraryPaths.Add(USDLibDir);
					foreach (string LibPath in Directory.EnumerateFiles(USDLibDir, "*.so", SearchOption.AllDirectories))
					{
						PublicAdditionalLibraries.Add(LibPath);
						RuntimeDependencies.Add(LibPath);
					}
					// Redirect plugInfo.json to Plugin/Binaries for the editor, but leave them pointing at the executable folder otherwise
					// (which is the default when USE_LIBRARIES_FROM_PLUGIN_FOLDER is not defined)
					if (Target.Type == TargetType.Editor && (Target.BuildEnvironment != TargetBuildEnvironment.Unique))
					{
						PublicDefinitions.Add("USE_LIBRARIES_FROM_PLUGIN_FOLDER=1");
					}
				}
				else if (Target.Platform == UnrealTargetPlatform.Mac)
				{
					PublicDefinitions.Add("USD_USES_SYSTEM_MALLOC=0");

					List<string> RuntimeModulePaths = new List<string>();

					// Python3
					PublicSystemIncludePaths.Add(Path.Combine(PythonSourceTPSDir, "include"));
					PublicSystemLibraryPaths.Add(Path.Combine(PythonBinaryTPSDir, "lib"));
					PrivateRuntimeLibraryPaths.Add(Path.Combine(PythonBinaryTPSDir, "bin"));
					RuntimeModulePaths.Add(Path.Combine(PythonBinaryTPSDir, "lib", "libpython3.11.dylib"));

					// USD
					PublicSystemIncludePaths.Add(USDIncludeDirectory);
					string USDLibDir = Path.Combine(
						ThirdPartyUSDDirectory,
						"Mac",
						"lib");
					foreach (string LibPath in Directory.EnumerateFiles(USDLibDir, "*.dylib", SearchOption.AllDirectories))
					{
						RuntimeModulePaths.Add(LibPath);
					}

					foreach (string RuntimeModulePath in RuntimeModulePaths)
					{
						if (!File.Exists(RuntimeModulePath))
						{
							string Err = string.Format("USD SDK module '{0}' not found.", RuntimeModulePath);
							System.Console.WriteLine(Err);
							throw new BuildException(Err);
						}

						PublicDelayLoadDLLs.Add(RuntimeModulePath);
						RuntimeDependencies.Add(RuntimeModulePath);
					}
					// Redirect plugInfo.json to Plugin/Binaries for the editor, but leave them pointing at the executable folder otherwise
					// (which is the default when USE_LIBRARIES_FROM_PLUGIN_FOLDER is not defined)
					if (Target.Type == TargetType.Editor && (Target.BuildEnvironment != TargetBuildEnvironment.Unique))
					{
						PublicDefinitions.Add("USE_LIBRARIES_FROM_PLUGIN_FOLDER=1");
					}
				}

				// Move UsdResources to <Target>/Binaries/ThirdParty/UsdResources. UnrealUSDWrapper.cpp will expect them to be there
				RuntimeDependencies.Add(
					Path.Combine("$(TargetOutputDir)", "..", "ThirdParty", "USD", "UsdResources", Target.Platform.ToString()),
					Path.Combine("$(PluginDir)", "Resources", "UsdResources", Target.Platform.ToString(), "...")
				);
			}
			else
			{
				PublicDefinitions.Add("USE_USD_SDK=0");
				PublicDefinitions.Add("USD_USES_SYSTEM_MALLOC=0");
			}
		}

		static bool EnableUsdSdk(ReadOnlyTargetRules Target, ModuleRules Module)
		{
			// USD SDK has been built against Python 3 and won't launch if the editor is using Python 2

			bool bEnableUsdSdk = (
				Target.WindowsPlatform.Compiler != WindowsCompiler.Intel &&
				Target.StaticAnalyzer == StaticAnalyzer.None
			);

			// Don't enable USD when running the include tool because it has issues parsing Boost headers
			if (Target.GlobalDefinitions.Contains("UE_INCLUDE_TOOL=1"))
			{
				bEnableUsdSdk = false;
			}

			if (!Target.bCompilePython)
			{
				bEnableUsdSdk = false;
			}

			// If you want to use USD in a monolithic target, you'll have to use the ANSI allocator.
			// USD always uses the ANSI C allocators directly. In a DLL UE build (so not monolithic) we can just override the operators new and delete
			// on each module with versions that use either the ANSI (so USD-compatible) allocators or the UE allocators (ModuleBoilerplate.h) when appropriate.
			// In a monolithic build we can't do that, as the primary game module will already define overrides for operator new and delete with
			// the standard UE allocators: Since we can only have one operator new/delete override on the entire monolithic executable, we can't define our own overrides.
			// Additionally, the ANSI allocator does not work properly with FMallocPoisonProxy. Consequently, FMallocPoisonProxy has to be disabled.
			// The only way around it is by forcing the ansi allocator and disabling FMallocPoisonProxy in your project's target file
			// (YourProject/Source/YourProject.Target.cs) file like this:
			//
			//		public class YourProject : TargetRules
			//		{
			//			public YourProject(TargetInfo Target) : base(Target)
			//			{
			//				...
			//				StaticAllocator = StaticAllocatorType.Ansi;
			//				GlobalDefinitions.Add("UE_USE_MALLOC_FILL_BYTES=0");
			//				...
			//			}
			//		}
			//
			// This will force the entire built executable to use the ANSI C allocators for everything (by disabling the UE overrides in ModuleBoilerplate.h) while
			// FMallocPoisonProxy is disabled, and so UE and USD allocations will be compatible.
			// Note that by that point everything will be using the USD-compatible ANSI allocators anyway, so our overrides in USDMemory.h are also disabled, as they're unnecessary.
			// Also note that we're forced to use dynamic linking for monolithic targets mainly because static linking the USD libraries disables support for user USD plugins, and secondly
			// because those static libraries would need to be linked with the --whole-archive argument, and there is currently no standard way of doing that in UE.
			bool bIsGameOrMonolithic = Target.LinkType == TargetLinkType.Monolithic || Target.Type == TargetType.Client || Target.Type == TargetType.Server || Target.Type == TargetType.Game;
			if (bEnableUsdSdk &&
				bIsGameOrMonolithic &&
				((Target.StaticAllocator != StaticAllocatorType.Ansi) || !Target.GlobalDefinitions.Contains("UE_USE_MALLOC_FILL_BYTES=0"))
			)
			{
				Module.PublicDefinitions.Add("USD_FORCE_DISABLED=1");
				bEnableUsdSdk = false;
			}
			else
			{
				Module.PublicDefinitions.Add("USD_FORCE_DISABLED=0");
			}

			return bEnableUsdSdk;
		}
	}
}
