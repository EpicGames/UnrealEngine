{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "USD Core",
	"Description": "Adds support for USD SDK, UE wrapper classes and USD conversion utilities",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "UnrealUSDWrapper",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64", "Mac", "Linux"
			]
		},
		{
			"Name": "USDUtilities",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64", "Mac", "Linux"
			]
		},
		{
			"Name": "USDClasses",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "ControlRig",
			"Enabled": true
		},
		{
			"Name": "GeometryCache",
			"Enabled": true
		},
		{
			"Name": "HairStrands",
			"Enabled": true
		},
		{
			"Name": "LiveLink",
			"Enabled": true
		}
	]
}
